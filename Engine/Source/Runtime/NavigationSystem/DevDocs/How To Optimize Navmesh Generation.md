# How to optimize navmesh generation speed?
Using a dynamic navmesh and generating navmesh at runtime can take a lot of CPU resources. Here is a list of tips and strategies that can limit that cost.

## Tips
1. Use the highest `CellHeight` and `CellSize` possible (in Project Settings ->Navigation Mesh)
    - Those parameters are used to define the size of the voxels used to generate the navmesh. The bigger they are, the less voxels are needed and the faster the tile generation runs. Be aware that increasing those will reduce the precision of the navmesh (how well it will fit the geometry shapes).


2. Limit the tile size
    - The size, depends on the game, aim for something between 32-128 cells per side - depends on the size of the changes/obstacles too
    - Agent size adds padding for rasterization, so smaller tiles size, while making changes local, can add a lot of processing. I.e. 32^2=1024, but with agent radius padding of 2 voxels (there's +1 for rounding), is (32+3+3)^2=1444.

3. Keep the nav collision simple
    - The triangles of the nav collision are the input of the generation. The less triangles, the faster the generation.

4. Keep track of what is dirtying the navmesh
    - Make sure small things that will not impact the navmesh are not marked to affect the navigation;
    - Make sure to not have objects that uselessly dirty the navmesh (like moving object at inaccessible locations);
    - Avoid dirtying huge tile areas.

## Tools

1) Locking/unlocking the navmesh generation at strategic times
    - When possible it's best to stop the automatic generation ("locking" the navmesh) before loading a bunch of assets that might dirty the navmesh and unlock the generation once it's done. This is more efficient and prevents rebuilding the same tiles several times.
    - For more information see :
https://udn.unrealengine.com/s/question/0D52L00004lufOgSAI/how-to-make-multiple-navmeshes-more-efficient

        "Making navmesh not rebuild after loading is actually pretty straightforward. You need to set your navigation system's `bInitialBuildingLocked` to true and call `ReleaseInitialBuildingLock()` once you're done loading. At that time navigation system will rebuild all parts of navmesh that have been marked dirty during loading by all other actors being loaded. To avoid that you need to clear out the accumulated information on dirtied areas. I suggest overriding `ReleaseInitialBuildingLock`() and before calling the super implementation calling `DefaultDirtyAreasController.Reset()`".

2. Multithreaded navmesh generation
    - This controlled by the `MaxSimultaneousTileGenerationJobsCount` property and limited by the amount of worker threads in `FRecastNavMeshGenerator::Init()`.

3. Using dynamic obstacles even if full dynamic generation is used
    - Instead of rebuilding the whole tile, dynamic obstacles mark the navmesh surface at the obstacle location. This is less costly and should be used when it's not needed to generate new navmesh surfaces on top of moving obstacles. 
    - The setting is on the static mesh in the "Navigation" category , it's called "Is Dynamic Obstacle".

4. If the only need for navmesh changes is the addition and removal of sublevels (no other world changes at runtime), consider using a static navmesh with navmesh data chunk streaming instead of a dynamic navmesh. That way the whole navmesh is prebuilt and only parts of it are loaded in and loaded out.
