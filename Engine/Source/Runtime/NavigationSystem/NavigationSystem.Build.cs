// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
    public class NavigationSystem : ModuleRules
    {
        public NavigationSystem(ReadOnlyTargetRules Target) : base(Target)
        {
	        CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

            PublicDependencyModuleNames.AddRange(
                new string[] {
					"Chaos",
					"Core",
                    "CoreUObject",
                    "Engine",
					"GeometryCollectionEngine",
				}
                );

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"DerivedDataCache",
					"TargetPlatform",
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"RHI",
					"RenderCore",
				}
				);

			SetupModulePhysicsSupport(Target);

            if (Target.bCompileRecast)
            {
                PrivateDependencyModuleNames.Add("Navmesh");
                PublicDefinitions.Add("WITH_RECAST=1");
                if (Target.bCompileNavmeshSegmentLinks)
                {
                    PublicDefinitions.Add("WITH_NAVMESH_SEGMENT_LINKS=1");
                }
                else
                {
                    PublicDefinitions.Add("WITH_NAVMESH_SEGMENT_LINKS=0");
                }

                if (Target.bCompileNavmeshClusterLinks)
                {
                    PublicDefinitions.Add("WITH_NAVMESH_CLUSTER_LINKS=1");
                }
                else
                {
                    PublicDefinitions.Add("WITH_NAVMESH_CLUSTER_LINKS=0");
                }
            }
            else
            {
                // Because we test WITH_RECAST in public Engine header files, we need to make sure that modules
                // that import us also have this definition set appropriately.  Recast is a private dependency
                // module, so it's definitions won't propagate to modules that import Engine.
                PublicDefinitions.Add("WITH_RECAST=0");
                PublicDefinitions.Add("WITH_NAVMESH_CLUSTER_LINKS=0");
                PublicDefinitions.Add("WITH_NAVMESH_SEGMENT_LINKS=0");
            }

            if (Target.bBuildEditor == true)
            {
                // @todo api: Only public because of WITH_EDITOR and UNREALED_API
				PublicDependencyModuleNames.Add("EditorFramework");
                PublicDependencyModuleNames.Add("UnrealEd");
                PrivateDependencyModuleNames.Add("SourceControl");
                CircularlyReferencedDependentModules.Add("UnrealEd");
            }
        }
    }
}
