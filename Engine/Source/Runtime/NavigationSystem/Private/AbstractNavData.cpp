// Copyright Epic Games, Inc. All Rights Reserved.

#include "AbstractNavData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AbstractNavData)

const FNavPathType FAbstractNavigationPath::Type;

FAbstractNavigationPath::FAbstractNavigationPath()
{
	PathType = FAbstractNavigationPath::Type;
}

INavigationQueryFilterInterface* FAbstractQueryFilter::CreateCopy() const
{
	return new FAbstractQueryFilter();
}

AAbstractNavData::AAbstractNavData(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bEditable = false;
	bListedInSceneOutliner = false;
#endif

	bCanBeMainNavData = false;
	bCanSpawnOnRebuild = false;

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		FindPathImplementation = FindPathAbstract;
		FindHierarchicalPathImplementation = FindPathAbstract;

		TestPathImplementation = TestPathAbstract;
		TestHierarchicalPathImplementation = TestPathAbstract;

		RaycastImplementationWithAdditionalResults = RaycastAbstract;

		DefaultQueryFilter->SetFilterType<FAbstractQueryFilter>();
	}
}

void AAbstractNavData::PostLoad()
{
	Super::PostLoad();
	SetFlags(RF_Transient);
	// marking as pending kill might seem an overkill, but one of the things 
	// this changes aims to achieve is to get rig of the excess number of 
	// AAbstractNavData instances. "There should be only one!"
	MarkAsGarbage();
}

FPathFindingResult AAbstractNavData::FindPathAbstract(const FNavAgentProperties& AgentProperties, const FPathFindingQuery& Query)
{
	const ANavigationData* Self = Query.NavData.Get();
	if (Self == NULL)
	{
		return ENavigationQueryResult::Error;
	}

	FPathFindingResult Result;
	if (Query.PathInstanceToFill.IsValid())
	{
		Result.Path = Query.PathInstanceToFill;
		Result.Path->ResetForRepath();
	}
	else
	{
		Result.Path = Self->CreatePathInstance<FAbstractNavigationPath>(Query);
	}

	Result.Path->GetPathPoints().Reset();
	Result.Path->GetPathPoints().Add(FNavPathPoint(Query.StartLocation));
	Result.Path->GetPathPoints().Add(FNavPathPoint(Query.EndLocation));
	Result.Path->MarkReady();
	Result.Result = ENavigationQueryResult::Success;

	return Result;
}

bool AAbstractNavData::TestPathAbstract(const FNavAgentProperties& AgentProperties, const FPathFindingQuery& Query, int32* NumVisitedNodes)
{
	return false;
}

bool AAbstractNavData::RaycastAbstract(const ANavigationData* NavDataInstance, const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, FSharedConstNavQueryFilter QueryFilter, const UObject* Querier)
{
	return RaycastAbstract(NavDataInstance, RayStart, RayEnd, HitLocation, nullptr, QueryFilter, Querier);
}

bool AAbstractNavData::RaycastAbstract(const ANavigationData* NavDataInstance, const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, FNavigationRaycastAdditionalResults* AdditionalResults, FSharedConstNavQueryFilter QueryFilter, const UObject* Querier)
{
	return false;
}

