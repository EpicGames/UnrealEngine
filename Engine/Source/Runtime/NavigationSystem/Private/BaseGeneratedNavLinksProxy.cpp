// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGeneratedNavLinksProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseGeneratedNavLinksProxy)

UBaseGeneratedNavLinksProxy::UBaseGeneratedNavLinksProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UBaseGeneratedNavLinksProxy::GetLinkData(FVector& LeftPt, FVector& RightPt, ENavLinkDirection::Type& Direction) const
{
	ensureMsgf(false, TEXT("Should not be called on a generated navlink proxy since it's not representing a single link."));
}

FNavLinkId UBaseGeneratedNavLinksProxy::GetId() const
{
	return LinkProxyId;
}

void UBaseGeneratedNavLinksProxy::UpdateLinkId(FNavLinkId ProxyId)
{
	LinkProxyId = ProxyId;
}

UObject* UBaseGeneratedNavLinksProxy::GetLinkOwner() const
{
	return Owner;
}
