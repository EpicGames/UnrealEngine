// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavAreas/NavArea.h"
#include "NavigationSystem.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavArea)

UNavArea::UNavArea(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
{
	DefaultCost = 1.f;
	FixedAreaEnteringCost = 0.f;
	DrawColor = FColor::Magenta;
	SupportedAgentsBits = 0xffffffff;
	// NOTE! AreaFlags == 0 means UNWALKABLE!
	AreaFlags = 1;
}

void UNavArea::FinishDestroy()
{
	if (HasAnyFlags(RF_ClassDefaultObject) && !IsReloadActive())
	{
		UNavigationSystemV1::RequestAreaUnregistering(GetClass());
	}

	Super::FinishDestroy();
}

void UNavArea::PostLoad()
{
	Super::PostLoad();
	RegisterArea();
}

void UNavArea::PostInitProperties()
{
	Super::PostInitProperties();
	RegisterArea();
}

void UNavArea::RegisterArea()
{
	if (!SupportedAgents.IsInitialized())
	{
		SupportedAgents.bSupportsAgent0 = bSupportsAgent0;
		SupportedAgents.bSupportsAgent1 = bSupportsAgent1;
		SupportedAgents.bSupportsAgent2 = bSupportsAgent2;
		SupportedAgents.bSupportsAgent3 = bSupportsAgent3;
		SupportedAgents.bSupportsAgent4 = bSupportsAgent4;
		SupportedAgents.bSupportsAgent5 = bSupportsAgent5;
		SupportedAgents.bSupportsAgent6 = bSupportsAgent6;
		SupportedAgents.bSupportsAgent7 = bSupportsAgent7;
		SupportedAgents.bSupportsAgent8 = bSupportsAgent8;
		SupportedAgents.bSupportsAgent9 = bSupportsAgent9;
		SupportedAgents.bSupportsAgent10 = bSupportsAgent10;
		SupportedAgents.bSupportsAgent11 = bSupportsAgent11;
		SupportedAgents.bSupportsAgent12 = bSupportsAgent12;
		SupportedAgents.bSupportsAgent13 = bSupportsAgent13;
		SupportedAgents.bSupportsAgent14 = bSupportsAgent14;
		SupportedAgents.bSupportsAgent15 = bSupportsAgent15;
		SupportedAgents.MarkInitialized();
	}

	if (HasAnyFlags(RF_ClassDefaultObject) && 
		!HasAnyFlags(RF_NeedInitialization)  // Don't register BP Area that has still not finished loaded their properties, it was also try again to register later via UNavArea::PostLoad()
		&& !IsReloadActive()
		)
	{
		UNavigationSystemV1::RequestAreaRegistering(GetClass());
	}
}

void UNavArea::Serialize(FArchive& Ar)
{
	if (Ar.IsSaving() && !SupportedAgents.IsInitialized())
	{
		SupportedAgents.MarkInitialized();
	}
		
	Super::Serialize(Ar);
}

FColor UNavArea::GetColor(UClass* AreaDefinitionClass)
{
	return AreaDefinitionClass ? AreaDefinitionClass->GetDefaultObject<UNavArea>()->DrawColor : FColor::Black;
}

void UNavArea::CopyFrom(TSubclassOf<UNavArea> AreaClass)
{
	if (AreaClass)
	{
		UNavArea* DefArea = (UNavArea*)AreaClass->GetDefaultObject();

		DefaultCost = DefArea->DefaultCost;
		FixedAreaEnteringCost = DefArea->GetFixedAreaEnteringCost();
		AreaFlags = DefArea->GetAreaFlags();
		DrawColor = DefArea->DrawColor;

		// don't copy supported agents bits
	}
}

#if WITH_EDITOR
void UNavArea::PostCDOCompiled(const FPostCDOCompiledContext& Context)
{
	Super::PostCDOCompiled(Context);
	RegisterArea();
}

void UNavArea::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName NAME_DefaultCost = GET_MEMBER_NAME_CHECKED(UNavArea, DefaultCost);
	static const FName NAME_FixedAreaEnteringCost = GET_MEMBER_NAME_CHECKED(UNavArea, FixedAreaEnteringCost);
	static const FName NAME_SupportedAgents = GET_MEMBER_NAME_CHECKED(UNavArea, SupportedAgents);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (HasAnyFlags(RF_ClassDefaultObject) && !IsReloadActive())
	{
		const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
		const FName MemberPropertyName = (PropertyChangedEvent.MemberProperty != nullptr) ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;

		if (PropertyName == NAME_DefaultCost
			|| PropertyName == NAME_FixedAreaEnteringCost
			|| MemberPropertyName == NAME_SupportedAgents)
		{
			UNavigationSystemV1::RequestAreaUnregistering(GetClass());
			RegisterArea();
		}
	}
}
#endif // WITH_EDITOR

