// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavAreas/NavAreaMeta.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavAreaMeta)

UNavAreaMeta::UNavAreaMeta(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
{
	bIsMetaArea = true;
}

