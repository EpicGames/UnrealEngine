// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavAreas/NavAreaMeta_SwitchByAgent.h"
#include "UObject/UnrealType.h"
#include "NavigationSystem.h"
#include "NavAreas/NavArea_Default.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavAreaMeta_SwitchByAgent)

#if WITH_EDITOR
bool UNavAreaMeta_SwitchByAgent::bUpdateAgentConfigCallbackRegistered = false;
#endif //#if WITH_EDITOR

UNavAreaMeta_SwitchByAgent::UNavAreaMeta_SwitchByAgent(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	Agent0Area = UNavArea_Default::StaticClass();
	Agent1Area = UNavArea_Default::StaticClass();
	Agent2Area = UNavArea_Default::StaticClass();
	Agent3Area = UNavArea_Default::StaticClass();
	Agent4Area = UNavArea_Default::StaticClass();
	Agent5Area = UNavArea_Default::StaticClass();
	Agent6Area = UNavArea_Default::StaticClass();
	Agent7Area = UNavArea_Default::StaticClass();
	Agent8Area = UNavArea_Default::StaticClass();
	Agent9Area = UNavArea_Default::StaticClass();
	Agent10Area = UNavArea_Default::StaticClass();
	Agent11Area = UNavArea_Default::StaticClass();
	Agent12Area = UNavArea_Default::StaticClass();
	Agent13Area = UNavArea_Default::StaticClass();
	Agent14Area = UNavArea_Default::StaticClass();
	Agent15Area = UNavArea_Default::StaticClass();

#if WITH_EDITOR
	if (HasAnyFlags(RF_ClassDefaultObject) && !bUpdateAgentConfigCallbackRegistered)
	{
		FCoreDelegates::OnAllModuleLoadingPhasesComplete.AddLambda([]() { GetMutableDefault<UNavAreaMeta_SwitchByAgent>()->UpdateAgentConfig(); });
		bUpdateAgentConfigCallbackRegistered = true;
	}
#endif //#if WITH_EDITOR
}

TSubclassOf<UNavAreaBase> UNavAreaMeta_SwitchByAgent::PickAreaClassForAgent(const AActor& Actor, const FNavAgentProperties& NavAgent) const
{
	const UNavigationSystemV1* DefNavSys = GetDefault<UNavigationSystemV1>();
	const int32 AgentIndex = DefNavSys ? DefNavSys->GetSupportedAgentIndex(NavAgent) : INDEX_NONE;
	TSubclassOf<UNavArea> UseAreaClass = NULL;
	
	switch (AgentIndex)
	{
	case 0: UseAreaClass = Agent0Area; break;
	case 1: UseAreaClass = Agent1Area; break;
	case 2: UseAreaClass = Agent2Area; break;
	case 3: UseAreaClass = Agent3Area; break;
	case 4: UseAreaClass = Agent4Area; break;
	case 5: UseAreaClass = Agent5Area; break;
	case 6: UseAreaClass = Agent6Area; break;
	case 7: UseAreaClass = Agent7Area; break;
	case 8: UseAreaClass = Agent8Area; break;
	case 9: UseAreaClass = Agent9Area; break;
	case 10: UseAreaClass = Agent10Area; break;
	case 11: UseAreaClass = Agent11Area; break;
	case 12: UseAreaClass = Agent12Area; break;
	case 13: UseAreaClass = Agent13Area; break;
	case 14: UseAreaClass = Agent14Area; break;
	case 15: UseAreaClass = Agent15Area; break;
	default: break;
	}

	return UseAreaClass ? UseAreaClass : FNavigationSystem::GetDefaultWalkableArea();
}

#if WITH_EDITOR
void UNavAreaMeta_SwitchByAgent::UpdateAgentConfig()
{
	const UNavigationSystemV1* DefNavSys = GetDefault<UNavigationSystemV1>();
	check(DefNavSys);

	const int32 MaxAllowedAgents = 16;
	const int32 NumAgents = FMath::Min(DefNavSys->GetSupportedAgents().Num(), MaxAllowedAgents);
	if (DefNavSys->GetSupportedAgents().Num() > MaxAllowedAgents)
	{
		UE_LOGF(LogNavigation, Error, "Navigation system supports %d agents, but only %d can be shown in %ls properties!",
			DefNavSys->GetSupportedAgents().Num(), MaxAllowedAgents, *GetClass()->GetName());
	}

	const FString CustomNameMeta = "DisplayName";
	for (int32 i = 0; i < MaxAllowedAgents; i++)
	{
		const FString PropName = FString::Printf(TEXT("Agent%dArea"), i);
		FProperty* Prop = FindFProperty<FProperty>(UNavAreaMeta_SwitchByAgent::StaticClass(), *PropName);
		check(Prop);

		if (i < NumAgents && NumAgents > 1)
		{
			Prop->SetPropertyFlags(CPF_Edit);
			Prop->SetMetaData(*CustomNameMeta, *FString::Printf(TEXT("Area Class for: %s"), *DefNavSys->GetSupportedAgents()[i].Name.ToString()));
		}
		else
		{
			Prop->ClearPropertyFlags(CPF_Edit);
		}
	}
}
#endif

