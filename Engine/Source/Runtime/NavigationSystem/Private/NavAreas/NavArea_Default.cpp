// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavAreas/NavArea_Default.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavArea_Default)

UNavArea_Default::UNavArea_Default(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

