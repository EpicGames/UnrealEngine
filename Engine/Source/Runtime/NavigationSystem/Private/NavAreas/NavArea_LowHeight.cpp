// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavAreas/NavArea_LowHeight.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavArea_LowHeight)

UNavArea_LowHeight::UNavArea_LowHeight(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
{
	DefaultCost = BIG_NUMBER;
	DrawColor = FColor(0, 0, 128);

	// can't traverse
	AreaFlags = 0;
}

