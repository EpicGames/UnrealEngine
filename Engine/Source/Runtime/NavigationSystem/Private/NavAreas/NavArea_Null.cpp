// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavAreas/NavArea_Null.h"
#include "NavMesh/RecastNavMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavArea_Null)

UNavArea_Null::UNavArea_Null(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	DefaultCost = FLT_MAX;
	AreaFlags = 0;
}

