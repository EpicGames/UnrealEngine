// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavAreas/NavArea_Obstacle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavArea_Obstacle)

UNavArea_Obstacle::UNavArea_Obstacle(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	DrawColor = FColor(127, 51, 0);	// brownish
	DefaultCost = 1000000.0f;
}

