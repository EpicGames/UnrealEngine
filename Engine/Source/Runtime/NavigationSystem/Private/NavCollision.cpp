// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavCollision.h"
#include "Serialization/MemoryWriter.h"
#include "NavigationSystem.h"
#include "Interfaces/Interface_CollisionDataProvider.h"
#include "Engine/StaticMesh.h"
#include "PrimitiveDrawingUtils.h"
#include "NavAreas/NavArea.h"
#include "AI/NavigationSystemHelpers.h"
#include "DerivedDataPluginInterface.h"
#include "DerivedDataCacheInterface.h"
#include "PhysicsEngine/BodySetup.h"
#include "ProfilingDebugging/CookStats.h"
#include "Interfaces/ITargetPlatform.h"
#include "CoreGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavCollision)

#if WITH_EDITOR
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#endif // WITH_EDITOR

static TAutoConsoleVariable<int32> CVarNavCollisionAvailable(
	TEXT("ai.NavCollisionAvailable"),
	1,
	TEXT("If set to 0 NavCollision won't be cooked and will be unavailable at runtime.\n"),
	/*ECVF_ReadOnly | */ECVF_Scalability);

#if ENABLE_COOK_STATS
namespace NavCollisionCookStats
{
	static FCookStats::FDDCResourceUsageStats UsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("NavCollision.Usage"), TEXT(""));
	});
}
#endif

static const FName NAVCOLLISION_FORMAT = "NavCollision_Chaos";

class FNavCollisionDataReader
{
public:
	FNavCollisionConvex& TriMeshCollision;
	FNavCollisionConvex& ConvexCollision;
	TNavStatArray<int32>& ConvexShapeIndices;
	FBox& Bounds;

	FNavCollisionDataReader(FByteBulkData& InBulkData, FNavCollisionConvex& InTriMeshCollision, FNavCollisionConvex& InConvexCollision, TNavStatArray<int32>& InShapeIndices, FBox& InBounds)
		: TriMeshCollision(InTriMeshCollision)
		, ConvexCollision(InConvexCollision)
		, ConvexShapeIndices(InShapeIndices)
		, Bounds(InBounds)
	{
		// Read cooked data
		uint8* DataPtr = (uint8*)InBulkData.Lock( LOCK_READ_ONLY );
		FBufferReader Ar( DataPtr, InBulkData.GetBulkDataSize(), false, true );

		uint8 bLittleEndian = true;

		Ar << bLittleEndian;
		Ar.SetByteSwapping( PLATFORM_LITTLE_ENDIAN ? !bLittleEndian : !!bLittleEndian );
		Ar << TriMeshCollision.VertexBuffer;
		Ar << TriMeshCollision.IndexBuffer;
		Ar << ConvexCollision.VertexBuffer;
		Ar << ConvexCollision.IndexBuffer;
		Ar << ConvexShapeIndices;
		Ar << Bounds;

		InBulkData.Unlock();
	}
};

//----------------------------------------------------------------------//
// FDerivedDataNavCollisionCooker
//----------------------------------------------------------------------//
class FDerivedDataNavCollisionCooker : public FDerivedDataPluginInterface
{
private:
	UNavCollision* NavCollisionInstance;
	UObject* CollisionDataProvider;
	FName Format;
	FGuid DataGuid;
	FString MeshId;

public:
	FDerivedDataNavCollisionCooker(FName InFormat, UNavCollision* InInstance);

	virtual const TCHAR* GetPluginName() const override
	{
		return TEXT("NavCollision");
	}

	virtual const TCHAR* GetVersionString() const override
	{
		return TEXT("8F4645C7A18B40C487C7E50E19CD6B6C");
	}

	virtual FString GetPluginSpecificCacheKeySuffix() const override
	{
		const uint16 Version = 15;

		return FString::Printf( TEXT("%s_%s_%s_%hu")
			, *Format.ToString()
			, *DataGuid.ToString()
			, *MeshId
			, Version
			);
	}

	virtual bool IsBuildThreadsafe() const override
	{
		return false;
	}

	virtual bool Build( TArray<uint8>& OutData ) override;
	virtual FString GetDebugContextString() const override;

	/** Return true if we can build **/
	bool CanBuild()
	{
		return true;
	}
};

FDerivedDataNavCollisionCooker::FDerivedDataNavCollisionCooker(FName InFormat, UNavCollision* InInstance)
	: NavCollisionInstance(InInstance)
	, CollisionDataProvider( NULL )
	, Format( InFormat )
{
	check(NavCollisionInstance != NULL);
	CollisionDataProvider = NavCollisionInstance->GetOuter();
	DataGuid = NavCollisionInstance->GetGuid();
	IInterface_CollisionDataProvider* CDP = Cast<IInterface_CollisionDataProvider>(CollisionDataProvider);
	if (CDP)
	{
		CDP->GetMeshId(MeshId);
	}
}

bool FDerivedDataNavCollisionCooker::Build( TArray<uint8>& OutData )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDerivedDataNavCollisionCooker::Build);

	if ((NavCollisionInstance->ConvexShapeIndices.Num() == 0) ||
		(NavCollisionInstance->GetTriMeshCollision().VertexBuffer.Num() == 0 && NavCollisionInstance->GetConvexCollision().VertexBuffer.Num() == 0))
	{
		NavCollisionInstance->GatherCollision();
	}

	FMemoryWriter Ar( OutData );
	uint8 bLittleEndian = PLATFORM_LITTLE_ENDIAN;
	Ar << bLittleEndian;
	int64 CookedMeshInfoOffset = Ar.Tell();

	Ar << NavCollisionInstance->GetMutableTriMeshCollision().VertexBuffer;
	Ar << NavCollisionInstance->GetMutableTriMeshCollision().IndexBuffer;
	Ar << NavCollisionInstance->GetMutableConvexCollision().VertexBuffer;
	Ar << NavCollisionInstance->GetMutableConvexCollision().IndexBuffer;
	Ar << NavCollisionInstance->ConvexShapeIndices;
	Ar << NavCollisionInstance->Bounds;

	// Whatever got cached return true. We want to cache 'failure' too.
	return true;
}

FString FDerivedDataNavCollisionCooker::GetDebugContextString() const
{
	if (NavCollisionInstance)
	{
		UObject* Outer = NavCollisionInstance->GetOuter();
		if (Outer)
		{
			return Outer->GetFullName();
		}
	}

	return FDerivedDataPluginInterface::GetDebugContextString();
}

namespace
{
	UNavCollisionBase* CreateNewNavCollisionInstance(UObject& Outer)
	{
		return NewObject<UNavCollision>(&Outer);
	}
}

//----------------------------------------------------------------------//
// UNavCollision
//----------------------------------------------------------------------//
UNavCollision::UNavCollision(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{	
	bGatherConvexGeometry = true;
	bHasConvexGeometry = false;
	bForceGeometryRebuild = false;
	bCreateOnClient = true;
}

void UNavCollision::PostInitProperties()
{
	Super::PostInitProperties();

	// if bCreateOnClient is false we're not even going to bind the delegate
	if (HasAnyFlags(RF_ClassDefaultObject)
		&& (GIsServer || bCreateOnClient
#if WITH_EDITOR
			|| GIsEditor
#endif
			)
		)
	{
		UNavCollisionBase::ConstructNewInstanceDelegate = UNavCollisionBase::FConstructNew::CreateStatic(&CreateNewNavCollisionInstance);
	}
}

FGuid UNavCollision::GetGuid() const
{
	return BodySetupGuid;
}

void UNavCollision::Setup(UBodySetup* BodySetup)
{
	// Create meshes from cooked data if not already done
	if (bHasConvexGeometry || BodySetup == NULL || BodySetupGuid == BodySetup->BodySetupGuid)
	{
		return;
	}

	LLM_SCOPE_BYNAME(TEXT("NavigationCollision"));

	BodySetupGuid = BodySetup->BodySetupGuid;

	// Make sure all are cleared before we start
	ClearCollision(); 
		
	// Find or create cooked navcollision data
	FByteBulkData* FormatData = GetCookedData(NAVCOLLISION_FORMAT);
	if (!bForceGeometryRebuild && FormatData)
	{
		// if it's not being already processed
		if (FormatData->IsLocked() == false)
		{
			// Create physics objects
			FNavCollisionDataReader CookedDataReader(*FormatData, TriMeshCollision, ConvexCollision, ConvexShapeIndices, Bounds);
			bHasConvexGeometry = true;
		}
	}
	else if (FPlatformProperties::RequiresCookedData() == false)
	{
		GatherCollision();
	}
}

FBox UNavCollision::GetBounds() const
{
	return Bounds;
}

void UNavCollision::GatherCollision()
{
	ClearCollision();

	UStaticMesh* StaticMeshOuter = Cast<UStaticMesh>(GetOuter());
	if (bGatherConvexGeometry && StaticMeshOuter && StaticMeshOuter->GetBodySetup())
	{
		NavigationHelper::GatherCollision(StaticMeshOuter->GetBodySetup(), this);
	}

	FKAggregateGeom SimpleGeom;
	for (int32 Idx = 0; Idx < BoxCollision.Num(); Idx++)
	{
		const FNavCollisionBox& BoxInfo = BoxCollision[Idx];

		const float X = FloatCastChecked<float>(BoxInfo.Extent.X * 2.0, UE::LWC::DefaultFloatPrecision);
		const float Y = FloatCastChecked<float>(BoxInfo.Extent.Y * 2.0, UE::LWC::DefaultFloatPrecision);
		const float Z = FloatCastChecked<float>(BoxInfo.Extent.Z * 2.0, UE::LWC::DefaultFloatPrecision);

		FKBoxElem BoxElem(X, Y, Z);

		BoxElem.SetTransform(FTransform(BoxInfo.Offset));

		SimpleGeom.BoxElems.Add(BoxElem);
	}

	// not really a cylinder, but should be close enough 
	for (int32 Idx = 0; Idx < CylinderCollision.Num(); Idx++)
	{
		const FNavCollisionCylinder& CylinderInfo = CylinderCollision[Idx];

		FKSphylElem SphylElem(CylinderInfo.Radius, CylinderInfo.Height);
		SphylElem.SetTransform(FTransform(CylinderInfo.Offset + FVector(0.f, 0.f, 0.5f*CylinderInfo.Height)));

		SimpleGeom.SphylElems.Add(SphylElem);
	}

	if (SimpleGeom.GetElementCount())
	{
		NavigationHelper::GatherCollision(SimpleGeom, *this);
	}

	bHasConvexGeometry = (TriMeshCollision.VertexBuffer.Num() > 0) || (ConvexCollision.VertexBuffer.Num() > 0);
}

void UNavCollision::ClearCollision()
{
	TriMeshCollision.VertexBuffer.Reset();
	TriMeshCollision.IndexBuffer.Reset();
	ConvexCollision.VertexBuffer.Reset();
	ConvexCollision.IndexBuffer.Reset();
	ConvexShapeIndices.Reset();
	Bounds = FBox(ForceInitToZero);

	bHasConvexGeometry = false;
}

void UNavCollision::GetNavigationModifier(FCompositeNavModifier& Modifier, const FTransform& LocalToWorld)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_NavCollision_GetNavigationModifier);

	const TSubclassOf<UNavArea> UseAreaClass = AreaClass ? AreaClass : (const TSubclassOf<UNavArea>)(FNavigationSystem::GetDefaultObstacleArea());

	// For non-dynamic obstacles with surface area, pass area class only without convex modifiers.
	if (!bIsDynamicObstacle && bUseSurfaceArea && AreaClass)
	{
		FAreaNavModifier AreaModifier;
		AreaModifier.SetAreaClass(UseAreaClass);
		Modifier.Add(AreaModifier);

		return;
	}

	// rebuild collision data if needed
	if (!bHasConvexGeometry)
	{
		GatherCollision();
	}

	const int32 NumModifiers = (TriMeshCollision.VertexBuffer.Num() ? 1 : 0) + ConvexShapeIndices.Num();
	Modifier.ReserveForAdditionalAreas(NumModifiers);

	auto AddModFunc = [&](const TNavStatArray<FVector>& VertexBuffer, const int32 FirstVertIndex, const int32 LastVertIndex)
	{
		FAreaNavModifier AreaMod;
		if (Modifier.IsPerInstanceModifier())
		{
			AreaMod.InitializePerInstanceConvex(VertexBuffer, FirstVertIndex, LastVertIndex, UseAreaClass);
		}
		else
		{
			AreaMod.InitializeConvex(VertexBuffer, FirstVertIndex, LastVertIndex, LocalToWorld, UseAreaClass);
		}
		AreaMod.SetIncludeAgentHeight(true);
		Modifier.Add(AreaMod);
	};

	int32 LastVertIndex = 0;
	for (int32 Idx = 0; Idx < ConvexShapeIndices.Num(); Idx++)
	{
		const int32 FirstVertIndex = LastVertIndex;
		LastVertIndex = ConvexShapeIndices.IsValidIndex(Idx + 1) ? ConvexShapeIndices[Idx + 1] : ConvexCollision.VertexBuffer.Num();

		// @todo this is a temp fix. A proper fix is making sure ConvexShapeIndices doesn't
		// contain any duplicates (which is the original cause of UE-52123)
		if (FirstVertIndex < LastVertIndex)
		{
			AddModFunc(ConvexCollision.VertexBuffer, FirstVertIndex, LastVertIndex);
		}
	}

	if (TriMeshCollision.VertexBuffer.Num() > 0)
	{
		AddModFunc(TriMeshCollision.VertexBuffer, 0, TriMeshCollision.VertexBuffer.Num() - 1);
	}
}

bool UNavCollision::HasSurfaceAreaClass() const
{
	return bUseSurfaceArea && AreaClass != nullptr;
}

bool UNavCollision::ExportGeometry(const FTransform& LocalToWorld, FNavigableGeometryExport& GeoExport) const
{
	if (bHasConvexGeometry)
	{
		GeoExport.ExportCustomMesh(ConvexCollision.VertexBuffer.GetData(), ConvexCollision.VertexBuffer.Num(),
			ConvexCollision.IndexBuffer.GetData(), ConvexCollision.IndexBuffer.Num(),
			LocalToWorld);

		GeoExport.ExportCustomMesh(TriMeshCollision.VertexBuffer.GetData(), TriMeshCollision.VertexBuffer.Num(),
			TriMeshCollision.IndexBuffer.GetData(), TriMeshCollision.IndexBuffer.Num(),
			LocalToWorld);
	}

	return bHasConvexGeometry;
}

void DrawCylinderHelper(FPrimitiveDrawInterface* PDI, const FMatrix& ElemTM, const FVector::FReal Radius, const FVector::FReal Height, const FColor Color)
{
	constexpr FVector::FReal AngleDelta = 2.0 * UE_DOUBLE_PI / 16.0;
	FVector X, Y, Z;

	ElemTM.GetUnitAxes(X, Y, Z);
	FVector	LastVertex = ElemTM.GetOrigin() + X * Radius;

	for(int32 SideIndex = 0;SideIndex < 16;SideIndex++)
	{
		const FVector Vertex = ElemTM.GetOrigin() +
			(X * FMath::Cos(AngleDelta * static_cast<FVector::FReal>(SideIndex + 1)) + Y * FMath::Sin(AngleDelta * static_cast<FVector::FReal>(SideIndex + 1))) * Radius;

		PDI->DrawLine(LastVertex,Vertex,Color,SDPG_World);
		PDI->DrawLine(LastVertex + Z * Height,Vertex + Z * Height,Color,SDPG_World);
		PDI->DrawLine(LastVertex,LastVertex + Z * Height,Color,SDPG_World);

		LastVertex = Vertex;
	}
}

void DrawBoxHelper(FPrimitiveDrawInterface* PDI, const FMatrix& ElemTM, const FVector& Extent, const FColor Color)
{
	FVector	B[2], P, Q;

	B[0] = Extent; // max
	B[1] = -1.0f * Extent; // min

	for( int32 i=0; i<2; i++ )
	{
		for( int32 j=0; j<2; j++ )
		{
			P.X=B[i].X; Q.X=B[i].X;
			P.Y=B[j].Y; Q.Y=B[j].Y;
			P.Z=B[0].Z; Q.Z=B[1].Z;
			PDI->DrawLine( ElemTM.TransformPosition(P), ElemTM.TransformPosition(Q), Color, SDPG_World);
			P.Y=B[i].Y; Q.Y=B[i].Y;
			P.Z=B[j].Z; Q.Z=B[j].Z;
			P.X=B[0].X; Q.X=B[1].X;
			PDI->DrawLine( ElemTM.TransformPosition(P), ElemTM.TransformPosition(Q), Color, SDPG_World);
			P.Z=B[i].Z; Q.Z=B[i].Z;
			P.X=B[j].X; Q.X=B[j].X;
			P.Y=B[0].Y; Q.Y=B[1].Y;
			PDI->DrawLine( ElemTM.TransformPosition(P), ElemTM.TransformPosition(Q), Color, SDPG_World);
		}
	}
}

void UNavCollision::DrawSimpleGeom(FPrimitiveDrawInterface* PDI, const FTransform& Transform, const FColor Color)
{
	const FMatrix ParentTM = Transform.ToMatrixWithScale();
	for (int32 i = 0; i < CylinderCollision.Num(); i++)
	{
		FMatrix ElemTM = FTranslationMatrix(CylinderCollision[i].Offset);
		ElemTM *= ParentTM;
		DrawCylinderHelper(PDI, ElemTM, CylinderCollision[i].Radius, CylinderCollision[i].Height, Color);
	}
	
	for (int32 i = 0; i < BoxCollision.Num(); i++)
	{
		FMatrix ElemTM = FTranslationMatrix(BoxCollision[i].Offset);
		ElemTM *= ParentTM;
		DrawBoxHelper(PDI, ElemTM, BoxCollision[i].Extent, Color);
	}
}


#if WITH_EDITOR
void UNavCollision::InvalidateCollision()
{
	ClearCollision();
	bForceGeometryRebuild = true;
}

void UNavCollision::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Force a geometry rebuild when bUseSurfaceArea or AreaClass changes to prevent stale cached data.
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UNavCollision, AreaClass) ||
		PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UNavCollision, bUseSurfaceArea))
	{
		InvalidateCollision();
	}
}

void UNavCollision::InvalidatePhysicsData()
{
	ClearCollision();
	CookedFormatData.FlushData();
}
#endif // WITH_EDITOR

void UNavCollision::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	const int32 VerInitial = 1;
	const int32 VerAreaClass = 2;
	const int32 VerConvexTransforms = 3;
	const int32 VerShapeGeoExport = 4;
	const int32 VerLatest = VerShapeGeoExport;

	// use magic number to determine if serialized stream has version :/
	const int32 MagicNum = 0xA237F237;
	int64 StreamStartPos = Ar.Tell();

	int32 Version = VerLatest;
	int32 MyMagicNum = MagicNum;
	Ar << MyMagicNum;

	if (MyMagicNum != MagicNum)
	{
		Version = VerInitial;
		Ar.Seek(StreamStartPos);
	}
	else
	{
		Ar << Version;
	}

	// loading a dummy GUID to have serialization not break on 
	// packages serialized before switching over UNavCollision to
	// use BodySetup's guid rather than its own one
	// motivation: not creating a new engine version
	// @NOTE could be addressed during next engine version bump
	FGuid Guid;
	Ar << Guid;
	
	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

	if (FPlatformProperties::RequiresCookedData() && !bCooked && Ar.IsLoading())
	{
		UE_LOGF(LogNavigation, Fatal, "This platform requires cooked packages, and NavCollision data was not cooked into %ls.", *GetFullName());
	}

	const bool bUseConvexCollisionVer3 = bGatherConvexGeometry || (CylinderCollision.Num() == 0 && BoxCollision.Num() == 0);
	const bool bUseConvexCollision = bGatherConvexGeometry || (BoxCollision.Num() > 0) || (CylinderCollision.Num() > 0);
	const bool bProcessCookedData = (Version >= VerShapeGeoExport) ? bUseConvexCollision : bUseConvexCollisionVer3;

	if (bCooked && bProcessCookedData)
	{
		if (Ar.IsCooking())
		{
			FName Format = NAVCOLLISION_FORMAT;
			GetCookedData(Format); // Get the data from the DDC or build it

			TArray<FName> ActualFormatsToSave;
			ActualFormatsToSave.Add(Format);
			CookedFormatData.Serialize(Ar, this, &ActualFormatsToSave);
		}
		else
		{
			CookedFormatData.Serialize(Ar, this);
		}
	}

	if (Version >= VerAreaClass)
	{
		Ar << AreaClass;
	}

	if (Version < VerShapeGeoExport && Ar.IsLoading() && GIsEditor)
	{
		bForceGeometryRebuild = true;
	}
}

void UNavCollision::PostLoad()
{
	Super::PostLoad();

	// Our owner needs to be post-loaded before us else they may not have loaded
	// their data yet.
	UObject* Outer = GetOuter();
	if (Outer)
	{
		Outer->ConditionalPostLoad();

		UStaticMesh* StaticMeshOuter = Cast<UStaticMesh>(Outer);
		
		// It's OK to skip this in case of StaticMesh pending compilation because it is also
		// called by UStaticMesh::CreateNavCollision at the end of UStaticMesh's PostLoad.
		if (StaticMeshOuter != nullptr && !StaticMeshOuter->IsCompiling())
		{
			Setup(StaticMeshOuter->GetBodySetup());
		}
	}
}

FByteBulkData* UNavCollision::GetCookedData(FName Format)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UNavCollision::GetCookedData);

	const bool bUseConvexCollision = bGatherConvexGeometry || (BoxCollision.Num() > 0) || (CylinderCollision.Num() > 0);
	if (IsTemplate() || !bUseConvexCollision)
	{
		return nullptr;
	}
	
	bool bContainedData = CookedFormatData.Contains(Format);
	FByteBulkData* Result = &CookedFormatData.GetFormat(Format);

	if (!bContainedData && CVarNavCollisionAvailable.GetValueOnAnyThread() != 0)
	{
		if (FPlatformProperties::RequiresCookedData())
		{
			UE_LOGF(LogNavigation, Error, "Attempt to build nav collision data for %ls when we are unable to. This platform requires cooked packages.", *GetPathName());
			return nullptr;
		}
		
		TArray<uint8> OutData;
		FDerivedDataNavCollisionCooker* DerivedNavCollisionData = new FDerivedDataNavCollisionCooker(Format, this);
		if (DerivedNavCollisionData->CanBuild())
		{
			bool bDataWasBuilt = false;
			COOK_STAT(auto Timer = NavCollisionCookStats::UsageStats.TimeSyncWork());
			if (GetDerivedDataCacheRef().GetSynchronous(DerivedNavCollisionData, OutData, &bDataWasBuilt))
			{
				COOK_STAT(Timer.AddHitOrMiss(bDataWasBuilt ? FCookStats::CallStats::EHitOrMiss::Miss : FCookStats::CallStats::EHitOrMiss::Hit, OutData.Num()));
				if (OutData.Num())
				{
					Result->Lock(LOCK_READ_WRITE);
					FMemory::Memcpy(Result->Realloc(OutData.Num()), OutData.GetData(), OutData.Num());
					Result->Unlock();
				}
			}
		}
	}

	UE_CLOGF(!Result, LogNavigation, Error, "Failed to read CoockedDataFormat for %ls.", *GetPathName());
	return (Result && Result->GetBulkDataSize() > 0) ? Result : nullptr; // we don't return empty bulk data...but we save it to avoid thrashing the DDC
}

void UNavCollision::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);
	
	if (CookedFormatData.Contains(NAVCOLLISION_FORMAT))
	{
		const FByteBulkData& FmtData = CookedFormatData.GetFormat(NAVCOLLISION_FORMAT);
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(FmtData.GetBulkDataSize());
	}
}

bool UNavCollision::NeedsLoadForTargetPlatform(const class ITargetPlatform* TargetPlatform) const
{
#if WITH_EDITOR
	const UDeviceProfile* DeviceProfile = UDeviceProfileManager::Get().FindProfile(TargetPlatform->IniPlatformName());
	if (DeviceProfile)
	{
		int32 CVarNavCollisionAvailableVal = 1;
		if (DeviceProfile->GetConsolidatedCVarValue(TEXT("ai.NavCollisionAvailable"), CVarNavCollisionAvailableVal))
		{
			return CVarNavCollisionAvailableVal != 0;
		}
	}
#endif // WITH_EDITOR

	return true;
}

void UNavCollision::CopyUserSettings(const UNavCollision& OtherData)
{
	CylinderCollision = OtherData.CylinderCollision;
	BoxCollision = OtherData.BoxCollision;
	AreaClass = OtherData.AreaClass;
	bIsDynamicObstacle = OtherData.bIsDynamicObstacle;
	bUseSurfaceArea = OtherData.bUseSurfaceArea;
	bGatherConvexGeometry = OtherData.bGatherConvexGeometry;
}
