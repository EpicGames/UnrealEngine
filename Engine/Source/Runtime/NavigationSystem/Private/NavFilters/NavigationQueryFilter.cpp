// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavFilters/NavigationQueryFilter.h"
#include "NavigationSystem.h"
#include "Engine/Engine.h"
#include "NavAreas/NavArea.h"
#include "NavigationData.h"
#include "EngineGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationQueryFilter)

//----------------------------------------------------------------------//
// UNavigationQueryFilter
//----------------------------------------------------------------------//
UNavigationQueryFilter::UNavigationQueryFilter(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	IncludeFlags.Packed = 0xffff;
	ExcludeFlags.Packed = 0;
	bInstantiateForQuerier = false;
	bIsMetaFilter = false;
}

FSharedConstNavQueryFilter UNavigationQueryFilter::GetQueryFilter(const ANavigationData& NavData, const UObject* Querier) const
{
	if (bIsMetaFilter && Querier != nullptr)
	{
		TSubclassOf<UNavigationQueryFilter> SimpleFilterClass = GetSimpleFilterForAgent(*Querier);
		if (*SimpleFilterClass)
		{
			const UNavigationQueryFilter* DefFilterOb = SimpleFilterClass.GetDefaultObject();
			check(DefFilterOb);
			if (DefFilterOb->bIsMetaFilter == false)
			{
				return DefFilterOb->GetQueryFilter(NavData, Querier);
			}
		}
	}
	
	// the default, simple filter implementation
	FSharedConstNavQueryFilter SharedFilter = bInstantiateForQuerier ? nullptr : NavData.GetQueryFilter(GetClass());
	if (!SharedFilter.IsValid())
	{
		// Clone the default filter so we get search nodes and other fields
		FSharedNavQueryFilter NavFilter = NavData.GetDefaultQueryFilter()->GetCopy();

		InitializeFilter(NavData, Querier, *NavFilter.Get());

		SharedFilter = NavFilter;
		if (!bInstantiateForQuerier)
		{
			const_cast<ANavigationData&>(NavData).StoreQueryFilter(GetClass(), SharedFilter);
		}
	}

	return SharedFilter;
}

void UNavigationQueryFilter::InitializeFilter(const ANavigationData& NavData, const UObject* Querier, FNavigationQueryFilter& Filter) const
{
	// apply overrides
	for (int32 i = 0; i < Areas.Num(); i++)
	{
		const FNavigationFilterArea& AreaData = Areas[i];
		
		const int32 AreaId = NavData.GetAreaID(AreaData.AreaClass);
		if (AreaId == INDEX_NONE)
		{
			continue;
		}

		if (AreaData.bIsExcluded)
		{
			Filter.SetExcludedArea(IntCastChecked<uint8>(AreaId));
		}
		else
		{
			if (AreaData.bOverrideTravelCost)
			{
				Filter.SetAreaCost(IntCastChecked<uint8>(AreaId), FMath::Max(1.0f, AreaData.TravelCostOverride));
			}

			if (AreaData.bOverrideEnteringCost)
			{
				Filter.SetFixedAreaEnteringCost(IntCastChecked<uint8>(AreaId), FMath::Max(0.0f, AreaData.EnteringCostOverride));
			}
		}
	}

	// apply flags
	Filter.SetIncludeFlags(IncludeFlags.Packed);
	Filter.SetExcludeFlags(ExcludeFlags.Packed);
}

FSharedConstNavQueryFilter UNavigationQueryFilter::GetQueryFilter(const ANavigationData& NavData, TSubclassOf<UNavigationQueryFilter> FilterClass)
{
	if (FilterClass)
	{
		const UNavigationQueryFilter* DefFilterOb = FilterClass.GetDefaultObject();
		// no way we have not default object here
		check(DefFilterOb);
		return DefFilterOb->GetQueryFilter(NavData, nullptr);
	}

	return nullptr;
}

FSharedConstNavQueryFilter UNavigationQueryFilter::GetQueryFilter(const ANavigationData& NavData, const UObject* Querier, TSubclassOf<UNavigationQueryFilter> FilterClass)
{
	if (FilterClass)
	{
		const UNavigationQueryFilter* DefFilterOb = FilterClass.GetDefaultObject();
		// no way we have not default object here
		check(DefFilterOb);
		return DefFilterOb->GetQueryFilter(NavData, Querier);
	}

	return nullptr;
}

void UNavigationQueryFilter::AddTravelCostOverride(TSubclassOf<UNavArea> AreaClass, float TravelCost)
{
	int32 Idx = FindAreaOverride(AreaClass);
	if (Idx == INDEX_NONE)
	{
		FNavigationFilterArea AreaData;
		AreaData.AreaClass = AreaClass;

		Idx = Areas.Add(AreaData);
	}

	Areas[Idx].bOverrideTravelCost = true;
	Areas[Idx].TravelCostOverride = TravelCost;
}

void UNavigationQueryFilter::AddEnteringCostOverride(TSubclassOf<UNavArea> AreaClass, float EnteringCost)
{
	int32 Idx = FindAreaOverride(AreaClass);
	if (Idx == INDEX_NONE)
	{
		FNavigationFilterArea AreaData;
		AreaData.AreaClass = AreaClass;

		Idx = Areas.Add(AreaData);
	}

	Areas[Idx].bOverrideEnteringCost = true;
	Areas[Idx].EnteringCostOverride = EnteringCost;
}

void UNavigationQueryFilter::AddExcludedArea(TSubclassOf<UNavArea> AreaClass)
{
	int32 Idx = FindAreaOverride(AreaClass);
	if (Idx == INDEX_NONE)
	{
		FNavigationFilterArea AreaData;
		AreaData.AreaClass = AreaClass;

		Idx = Areas.Add(AreaData);
	}

	Areas[Idx].bIsExcluded = true;
}

int32 UNavigationQueryFilter::FindAreaOverride(TSubclassOf<UNavArea> AreaClass) const
{
	for (int32 i = 0; i < Areas.Num(); i++)
	{
		if (Areas[i].AreaClass == AreaClass)
		{
			return i;
		}
	}

	return INDEX_NONE;
}

#if WITH_EDITOR
void UNavigationQueryFilter::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// remove cached filter settings from existing NavigationSystems
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(Context.World());
		if (NavSys)
		{
			NavSys->ResetCachedFilter(GetClass());
		}
	}
}
#endif

