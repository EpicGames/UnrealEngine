// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavFilters/RecastFilter_UseDefaultArea.h"
#include "Templates/Casts.h"
#include "NavMesh/RecastNavMesh.h"
#include "NavMesh/RecastQueryFilter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RecastFilter_UseDefaultArea)

URecastFilter_UseDefaultArea::URecastFilter_UseDefaultArea(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

void URecastFilter_UseDefaultArea::InitializeFilter(const ANavigationData& NavData, const UObject* Querier, FNavigationQueryFilter& Filter) const
{
#if WITH_RECAST
	Filter.SetFilterImplementation(dynamic_cast<const INavigationQueryFilterInterface*>(ARecastNavMesh::GetNamedFilter(ERecastNamedFilter::FilterOutAreas)));
#endif // WITH_RECAST

	Super::InitializeFilter(NavData, Querier, Filter);
}

