// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavGraph/NavGraphGenerator.h"

//----------------------------------------------------------------------//
// FNavGraphGenerator
//----------------------------------------------------------------------//
FNavGraphGenerator::FNavGraphGenerator(ANavigationGraph* InDestNavGraph)
{

}

FNavGraphGenerator::~FNavGraphGenerator()
{

}

void FNavGraphGenerator::Init()
{

}

void FNavGraphGenerator::CleanUpIntermediateData()
{

}

void FNavGraphGenerator::UpdateBuilding()
{

}

