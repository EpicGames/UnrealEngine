// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "GameFramework/Volume.h"
#include "AI/NavDataGenerator.h"

class ANavigationGraph;

/**
 * Class that handles generation of the ANavigationGraph data
 */
class FNavGraphGenerator : public FNavDataGenerator
{
public:
	FNavGraphGenerator(ANavigationGraph* InDestNavGraph);
	virtual ~FNavGraphGenerator();

private:
	/** Prevent copying. */
	FNavGraphGenerator(FNavGraphGenerator const& NoCopy) { check(0); };
	FNavGraphGenerator& operator=(FNavGraphGenerator const& NoCopy) { check(0); return *this; }

private:
	// Performs initial setup of member variables so that generator is ready to do its thing from this point on
	void Init();
	void CleanUpIntermediateData();
	void UpdateBuilding();

private:
	/** Bounding geometry definition. */
	TArray<AVolume const*> InclusionVolumes;

	FCriticalSection GraphChangingLock;
};
