// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavGraph/NavigationGraph.h"
#include "EngineUtils.h"
#include "NavigationSystem.h"
#include "NavGraph/NavGraphGenerator.h"
#include "NavNodeInterface.h"
#include "NavGraph/NavigationGraphNodeComponent.h"
#include "NavGraph/NavigationGraphNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationGraph)

//----------------------------------------------------------------------//
// FNavGraphNode
//----------------------------------------------------------------------//
FNavGraphNode::FNavGraphNode() 
	: Owner(nullptr)
{
	Edges.Reserve(InitialEdgesCount);
}

//----------------------------------------------------------------------//
// UNavigationGraphNodeComponent
//----------------------------------------------------------------------//
UNavigationGraphNodeComponent::UNavigationGraphNodeComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UNavigationGraphNodeComponent::BeginDestroy()
{
	Super::BeginDestroy();
	
	if (PrevNodeComponent != NULL)
	{
		PrevNodeComponent->NextNodeComponent = NextNodeComponent;
	}

	if (NextNodeComponent != NULL)
	{
		NextNodeComponent->PrevNodeComponent = PrevNodeComponent;
	}

	NextNodeComponent = NULL;
	PrevNodeComponent = NULL;
}

//----------------------------------------------------------------------//
// ANavigationGraphNode
//----------------------------------------------------------------------//
ANavigationGraphNode::ANavigationGraphNode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

//----------------------------------------------------------------------//
// ANavigationGraph
//----------------------------------------------------------------------//

ANavigationGraph::ANavigationGraph(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		NavDataGenerator = MakeShareable(new FNavGraphGenerator(this));
	}
}

