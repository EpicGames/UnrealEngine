// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavLinkComponent.h"
#include "NavLinkRenderingProxy.h"
#include "AI/NavigationSystemHelpers.h"
#include "AI/Navigation/NavigationRelevantData.h"
#include "NavigationSystemTypes.h"
#include "NavAreas/NavArea_Default.h"
#include "Engine/CollisionProfile.h"
#include "NavLinkTrivial.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavLinkComponent)


UNavLinkComponent::UNavLinkComponent(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	Mobility = EComponentMobility::Stationary;
	BodyInstance.SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	SetGenerateOverlapEvents(false);

	bHasCustomNavigableGeometry = EHasCustomNavigableGeometry::EvenIfNotCollidable;
	bCanEverAffectNavigation = true;
	bNavigationRelevant = true;

	FNavigationLink DefLink;
	DefLink.SetAreaClass(UNavArea_Default::StaticClass());

	Links.Add(DefLink);
}

FBoxSphereBounds UNavLinkComponent::CalcBounds(const FTransform &LocalToWorld) const
{
	FBox LocalBounds(ForceInit);
	for (int32 Idx = 0; Idx < Links.Num(); Idx++)
	{
		LocalBounds += Links[Idx].Left;
		LocalBounds += Links[Idx].Right;
	}

	const FBox WorldBounds = LocalBounds.TransformBy(LocalToWorld);
	return FBoxSphereBounds(WorldBounds);
}

void UNavLinkComponent::GetNavigationData(FNavigationRelevantData& Data) const
{
	NavigationHelper::ProcessNavLinkAndAppend(&Data.Modifiers, NavigationHelper::FNavLinkOwnerData(*this), Links);
}

bool UNavLinkComponent::IsNavigationRelevant() const
{
	return Links.Num() > 0;
}

bool UNavLinkComponent::GetNavigationLinksArray(TArray<FNavigationLink>& OutLink, TArray<FNavigationSegmentLink>& OutSegments) const
{
	OutLink.Append(Links);
	return OutLink.Num() > 0;
}

FPrimitiveSceneProxy* UNavLinkComponent::CreateSceneProxy()
{
	return new FNavLinkRenderingProxy(this);
}

#if WITH_EDITOR

void UNavLinkComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UNavLinkComponent, Links))
	{
		InitializeLinksAreaClasses();
	}
}

void UNavLinkComponent::PostEditUndo()
{
	Super::PostEditUndo();

	InitializeLinksAreaClasses();
}

void UNavLinkComponent::PostEditImport()
{
	Super::PostEditImport();

	InitializeLinksAreaClasses();
}

#endif // WITH_EDITOR

void UNavLinkComponent::OnRegister()
{
	Super::OnRegister();

	InitializeLinksAreaClasses();
}

void UNavLinkComponent::InitializeLinksAreaClasses()
{
	for (FNavigationLink& Link : Links)
	{
		Link.InitializeAreaClass();
	}
}

//----------------------------------------------------------------------//
// UNavLinkTrivial
//----------------------------------------------------------------------//
UNavLinkTrivial::UNavLinkTrivial(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FNavigationLink& Link = Links[Links.Add(FNavigationLink(FVector(0, 100, 0), FVector(0, -100, 0)))];
}

