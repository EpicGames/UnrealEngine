// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavLinkCustomComponent.h"
#include "TimerManager.h"
#include "GameFramework/Pawn.h"
#include "CollisionQueryParams.h"
#include "WorldCollision.h"
#include "Engine/World.h"
#include "GameFramework/Controller.h"
#include "NavigationSystem.h"
#include "NavAreas/NavArea_Null.h"
#include "NavAreas/NavArea_Default.h"
#include "AI/NavigationModifier.h"
#include "NavigationOctree.h"
#include "AI/NavigationSystemHelpers.h"
#include "AI/Navigation/PathFollowingAgentInterface.h"
#include "Engine/OverlapResult.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavLinkCustomComponent)

#if WITH_EDITOR
namespace UE::Navigation::LinkCustomComponent::Private
{
	void OnNavAreaRegistrationChanged(UNavLinkCustomComponent& CustomComponent, const UWorld& World, const UClass* NavAreaClass)
	{
		if (NavAreaClass && (NavAreaClass == CustomComponent.GetLinkAreaClass() || NavAreaClass == CustomComponent.GetObstacleAreaClass()) && &World == CustomComponent.GetWorld())
		{
			CustomComponent.RefreshNavigationModifiers();
		}
	}
} // UE::Navigation::LinkCustomComponent::Private
#endif // WITH_EDITOR

UNavLinkCustomComponent::UNavLinkCustomComponent(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	NavLinkUserId = 0;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	LinkRelativeStart = FVector(70, 0, 0);
	LinkRelativeEnd = FVector(-70, 0, 0);
	LinkDirection = ENavLinkDirection::BothWays;
	EnabledAreaClass = UNavArea_Default::StaticClass();
	DisabledAreaClass = UNavArea_Null::StaticClass();
	ObstacleAreaClass = UNavArea_Null::StaticClass();
	ObstacleExtent = FVector(50, 50, 50);
	bLinkEnabled = true;
	bNotifyWhenEnabled = false;
	bNotifyWhenDisabled = false;
	bCreateBoxObstacle = false;
	BroadcastRadius = 0.0f;
	BroadcastChannel = ECC_Pawn;
	BroadcastInterval = 0.0f;

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		const FString PathName = GetPathName();
		AuxiliaryCustomLinkId = FNavLinkAuxiliaryId::GenerateUniqueAuxiliaryId(*PathName);
	}
}

void UNavLinkCustomComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::NavigationLinkID32To64)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		CustomLinkId = FNavLinkId(NavLinkUserId);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void UNavLinkCustomComponent::PostLoad()
{
	Super::PostLoad();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	INavLinkCustomInterface::UpdateUniqueId(CustomLinkId);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#if WITH_EDITOR
// This function is only called if GIsEditor == true for non default objects components that are registered.
void UNavLinkCustomComponent::OnNavAreaRegistered(const UWorld& World, const UClass* NavAreaClass)
{
	UE::Navigation::LinkCustomComponent::Private::OnNavAreaRegistrationChanged(*this, World, NavAreaClass);
}

// This function is only called if GIsEditor == true for non default objects components that are registered.
void UNavLinkCustomComponent::OnNavAreaUnregistered(const UWorld& World, const UClass* NavAreaClass)
{
	UE::Navigation::LinkCustomComponent::Private::OnNavAreaRegistrationChanged(*this, World, NavAreaClass);
}
#endif // WITH_EDITOR

TStructOnScope<FActorComponentInstanceData> UNavLinkCustomComponent::GetComponentInstanceData() const
{
	TStructOnScope<FActorComponentInstanceData> InstanceData = MakeStructOnScope<FActorComponentInstanceData, FNavLinkCustomInstanceData>(this);
	FNavLinkCustomInstanceData* NavLinkCustomInstanceData = InstanceData.Cast<FNavLinkCustomInstanceData>();
	NavLinkCustomInstanceData->CustomLinkId = CustomLinkId;
	NavLinkCustomInstanceData->AuxiliaryCustomLinkId = AuxiliaryCustomLinkId;
	return InstanceData;
}

void UNavLinkCustomComponent::ApplyComponentInstanceData(FNavLinkCustomInstanceData* NavLinkData)
{
	check(NavLinkData);

	if (CustomLinkId != NavLinkData->CustomLinkId
		|| AuxiliaryCustomLinkId != NavLinkData->AuxiliaryCustomLinkId)
	{
		// Registered component has its link registered in the navigation system. 
		// In such case, we need to unregister current Id and register with the Id from the instance data.
		const bool bIsLinkRegistrationUpdateRequired = IsRegistered();

		if (bIsLinkRegistrationUpdateRequired)
		{
			UNavigationSystemV1::RequestCustomLinkUnregistering(*this, this);
		}

		AuxiliaryCustomLinkId = NavLinkData->AuxiliaryCustomLinkId;
		CustomLinkId = NavLinkData->CustomLinkId;

		if (bIsLinkRegistrationUpdateRequired)
		{
			UNavigationSystemV1::RequestCustomLinkRegistering(*this, this);
		}
	}
}

#if WITH_EDITOR
void UNavLinkCustomComponent::PostEditImport()
{
	Super::PostEditImport();

	// Generate a new AuxiliarLinkUserId and set CustomLinkId to Invalid, this is then inline with the constructor in this regard.
	// CustomLinkId is set to a valid Id later in OnRegister().
	const FString PathName = GetPathName();
	AuxiliaryCustomLinkId = FNavLinkAuxiliaryId::GenerateUniqueAuxiliaryId(*PathName);

	CustomLinkId = FNavLinkId::Invalid;
}
#endif // WITH_EDITOR

void UNavLinkCustomComponent::GetLinkData(FVector& LeftPt, FVector& RightPt, ENavLinkDirection::Type& Direction) const
{
	LeftPt = LinkRelativeStart;
	RightPt = LinkRelativeEnd;
	Direction = LinkDirection;
}

void UNavLinkCustomComponent::GetSupportedAgents(FNavAgentSelector& OutSupportedAgents) const
{
	OutSupportedAgents = SupportedAgents;
}

TSubclassOf<UNavArea> UNavLinkCustomComponent::GetLinkAreaClass() const
{
	return bLinkEnabled ? EnabledAreaClass : DisabledAreaClass;
}

FNavLinkAuxiliaryId UNavLinkCustomComponent::GetAuxiliaryId() const
{
	return AuxiliaryCustomLinkId;
}

FNavLinkId UNavLinkCustomComponent::GetId() const
{
	return CustomLinkId;
}

void UNavLinkCustomComponent::UpdateLinkId(FNavLinkId NewUniqueId)
{
	if (NewUniqueId == CustomLinkId)
	{
		return;
	}

#if WITH_EDITOR
	UE_CLOGF(NewUniqueId.IsLegacyId(), LogNavLink, Verbose, "%s Navlink using LinkUserId id %llu [%ls] is being updated with old style link ID. Please move to the new system. See FNavLinkId::GenerateUniqueId()", __FUNCTION__, CustomLinkId.GetId(), *GetFullName());
#endif

	CustomLinkId = NewUniqueId;

#if WITH_EDITOR
	Modify(true);
#endif
}

bool UNavLinkCustomComponent::IsLinkPathfindingAllowed(const UObject* Querier) const
{
	return true;
}

bool UNavLinkCustomComponent::OnLinkMoveStarted(UObject* PathComp, const FVector& DestPoint)
{
	MovingAgents.Add(MakeWeakObjectPtr(PathComp));

	if (OnMoveReachedLink.IsBound())
	{
		OnMoveReachedLink.Execute(this, PathComp, DestPoint);
		return true;
	}

	return false;
}

void UNavLinkCustomComponent::OnLinkMoveFinished(UObject* PathComp)
{
	MovingAgents.Remove(MakeWeakObjectPtr(PathComp));
}

void UNavLinkCustomComponent::GetNavigationData(FNavigationRelevantData& Data) const
{
	TArray<FNavigationLink> NavLinks;
	FNavigationLink LinkMod = GetLinkModifier();
	LinkMod.MaxFallDownLength = 0.f;
	LinkMod.LeftProjectHeight = 0.f;
	NavLinks.Add(LinkMod);
	NavigationHelper::ProcessNavLinkAndAppend(&Data.Modifiers, GetOwner(), NavLinks);

	if (bCreateBoxObstacle)
	{
		Data.Modifiers.Add(FAreaNavModifier(FBox::BuildAABB(ObstacleOffset, ObstacleExtent), GetOwner()->GetTransform(), ObstacleAreaClass));
	}
}

void UNavLinkCustomComponent::CalcAndCacheBounds() const
{
	Bounds = FBox(ForceInit);
	Bounds += GetStartPoint();
	Bounds += GetEndPoint();

	if (bCreateBoxObstacle)
	{
		FBox ObstacleBounds = FBox::BuildAABB(ObstacleOffset, ObstacleExtent);
		Bounds += ObstacleBounds.TransformBy(GetOwner()->GetTransform());
	}
}

void UNavLinkCustomComponent::OnRegister()
{
	Super::OnRegister();

	// Actor::GetActorInstanceGuid() is only available when WITH_EDITOR is valid.
#if WITH_EDITOR
	// Do not convert old Ids.
	if (CustomLinkId == FNavLinkId::Invalid || CustomLinkId.IsLegacyId() == false)
	{
		// Either this is a freshly spawned component or a component loaded after AuxiliaryCustomLinkId has been saved (and is a new style Id), so we can safely use this to generate a new id.
		// 
		const AActor* Owner = GetOwner();
		checkf(Owner, TEXT("We must have an Owner here as we need it to create a unique id."));

		const FNavLinkId OldCustomLinkId = CustomLinkId;

		// We calculate the CustomLinkId deterministically every time we call OnRegister(), as Level Instances with Actors with UNavLinkCustomComponents can not serialize data for each 
		// duplicated UNavLinkCustomComponent in different Level Instances (in the editor world). 
		// For cooked data this is not a problem as the Level Instances are expanded in to actual instances of actors and components so we can cook the in game value for the CustomLinkId.
		CustomLinkId = FNavLinkId::GenerateUniqueId(AuxiliaryCustomLinkId, Owner->GetActorInstanceGuid());
		UE_LOGF(LogNavLink, VeryVerbose, "%s navlink id generated %llu [%ls].", __FUNCTION__, CustomLinkId.GetId(), *GetFullName());

		if (OldCustomLinkId != CustomLinkId)
		{
			Modify(true);

			// Baked navmesh has off-mesh connections with the old ID — rebuild the tile
			// so they get regenerated with the new one.
			if (UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld()))
			{
				const FBox LinkBounds = UNavigationSystemV1::ComputeCustomLinkBounds(*this);
				if (LinkBounds.IsValid && !LinkBounds.GetSize().IsNearlyZero())
				{
					NavSys->AddDirtyArea(LinkBounds, ENavigationDirtyFlag::Geometry);
				}
			}
		}
	}
#else // !WITH_EDITOR
	// There is an edge case here for run time only level instances in that they will have a valid CustomLinkId but it will be from the level instance so Ids will get duplicated
	// in different level instances. Currently baked in nav mesh is not supported for these level instances and its anticipated nav mesh will need to be dynamically regenerated
	// when the level instance is spawned. This means the CustomLinkId does not need to be deterministic as its not relating to anything generated outside of the run time.
	// This clash gets handled via RequestCustomLinkRegistering() in UNavigationSystemV1::RegisterCustomLink(), not in the next section of code!

	// This will only be true for freshly spawned components in game that are not in level instances.
	if (CustomLinkId == FNavLinkId::Invalid)
	{
		CustomLinkId = FNavLinkId::GenerateUniqueId(AuxiliaryCustomLinkId, FGuid::NewGuid());

		UE_LOGF(LogNavLink, VeryVerbose, "%s incremental navlink id generated %llu [%ls].", __FUNCTION__, CustomLinkId.GetId(), *GetFullName());
	}
#endif // WITH_EDITOR

	UNavigationSystemV1::RequestCustomLinkRegistering(*this, this);

#if WITH_EDITOR
	if (GIsEditor && !HasAnyFlags(RF_ClassDefaultObject))
	{
		OnNavAreaRegisteredDelegateHandle = UNavigationSystemBase::OnNavAreaRegisteredDelegate().AddUObject(this, &UNavLinkCustomComponent::OnNavAreaRegistered);
		OnNavAreaUnregisteredDelegateHandle = UNavigationSystemBase::OnNavAreaUnregisteredDelegate().AddUObject(this, &UNavLinkCustomComponent::OnNavAreaUnregistered);
	}
#endif // WITH_EDITOR 
}

void UNavLinkCustomComponent::OnUnregister()
{
	Super::OnUnregister();

#if WITH_EDITOR
	if (GIsEditor && !HasAnyFlags(RF_ClassDefaultObject))
	{
		UNavigationSystemBase::OnNavAreaRegisteredDelegate().Remove(OnNavAreaRegisteredDelegateHandle);
		UNavigationSystemBase::OnNavAreaUnregisteredDelegate().Remove(OnNavAreaUnregisteredDelegateHandle);
	}
#endif // WITH_EDITOR 

	UNavigationSystemV1::RequestCustomLinkUnregistering(*this, this);
}

void UNavLinkCustomComponent::SetLinkData(const FVector& RelativeStart, const FVector& RelativeEnd, ENavLinkDirection::Type Direction)
{
	LinkRelativeStart = RelativeStart;
	LinkRelativeEnd = RelativeEnd;
	LinkDirection = Direction;

	// Link start and end positions have changed, we need to update the bounds
	UpdateNavigationBounds();

	RefreshNavigationModifiers();
}

FNavigationLink UNavLinkCustomComponent::GetLinkModifier() const
{
	return INavLinkCustomInterface::GetModifier(this);
}

void UNavLinkCustomComponent::SetEnabledArea(TSubclassOf<UNavArea> AreaClass)
{
	EnabledAreaClass = AreaClass;
	if (IsNavigationRelevant() && bLinkEnabled)
	{
		UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
		if (NavSys)
		{
			NavSys->UpdateCustomLink(this);
		}
	}
}

void UNavLinkCustomComponent::SetDisabledArea(TSubclassOf<UNavArea> AreaClass)
{
	DisabledAreaClass = AreaClass;
	if (IsNavigationRelevant() && !bLinkEnabled)
	{
		UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
		if (NavSys)
		{
			NavSys->UpdateCustomLink(this);
		}
	}
}

void UNavLinkCustomComponent::AddNavigationObstacle(TSubclassOf<UNavArea> AreaClass, const FVector& BoxExtent, const FVector& BoxOffset)
{
	ObstacleOffset = BoxOffset;
	ObstacleExtent = BoxExtent;
	ObstacleAreaClass = AreaClass;
	bCreateBoxObstacle = true;

	RefreshNavigationModifiers();
}

void UNavLinkCustomComponent::ClearNavigationObstacle()
{
	ObstacleAreaClass = NULL;
	bCreateBoxObstacle = false;

	RefreshNavigationModifiers();
}

void UNavLinkCustomComponent::SetEnabled(bool bNewEnabled)
{
	if (bLinkEnabled != bNewEnabled)
	{
		bLinkEnabled = bNewEnabled;

		UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
		if (NavSys)
		{
			NavSys->UpdateCustomLink(this);
		}

		if (GetWorld())
		{
			GetWorld()->GetTimerManager().ClearTimer(TimerHandle_BroadcastStateChange);

			if ((bLinkEnabled && bNotifyWhenEnabled) || (!bLinkEnabled && bNotifyWhenDisabled))
			{
				BroadcastStateChange();
			}
		}

		RefreshNavigationModifiers();
	}
}

void UNavLinkCustomComponent::SetMoveReachedLink(FOnMoveReachedLink const& InDelegate)
{
	OnMoveReachedLink = InDelegate;
}

bool UNavLinkCustomComponent::HasMovingAgents() const
{
	for (int32 i = 0; i < MovingAgents.Num(); i++)
	{
		if (MovingAgents[i].IsValid())
		{
			return true;
		}
	}

	return false;
}

void UNavLinkCustomComponent::SetBroadcastData(float Radius, ECollisionChannel TraceChannel, float Interval)
{
	BroadcastRadius = Radius;
	BroadcastChannel = TraceChannel;
	BroadcastInterval = Interval;
}

void UNavLinkCustomComponent::SendBroadcastWhenEnabled(bool bEnabled)
{
	bNotifyWhenEnabled = bEnabled;
}

void UNavLinkCustomComponent::SendBroadcastWhenDisabled(bool bEnabled)
{
	bNotifyWhenDisabled = bEnabled;
}

void UNavLinkCustomComponent::SetBroadcastFilter(FBroadcastFilter const& InDelegate)
{
	OnBroadcastFilter = InDelegate;
}

void UNavLinkCustomComponent::CollectNearbyAgents(TArray<UObject*>& NotifyList)
{
	AActor* MyOwner = GetOwner();
	if (BroadcastRadius < KINDA_SMALL_NUMBER || MyOwner == NULL)
	{
		return;
	}

	FCollisionQueryParams Params(SCENE_QUERY_STAT(SmartLinkBroadcastTrace), false, MyOwner);
	TArray<FOverlapResult> OverlapsL, OverlapsR;

	const FVector LocationL = GetStartPoint();
	const FVector LocationR = GetEndPoint();
	const FVector::FReal LinkDistSq = (LocationL - LocationR).SizeSquared();
	const FVector::FReal DistThresholdSq = FMath::Square(BroadcastRadius * 0.25);
	if (LinkDistSq > DistThresholdSq)
	{
		GetWorld()->OverlapMultiByChannel(OverlapsL, LocationL, FQuat::Identity, BroadcastChannel, FCollisionShape::MakeSphere(BroadcastRadius), Params);
		GetWorld()->OverlapMultiByChannel(OverlapsR, LocationR, FQuat::Identity, BroadcastChannel, FCollisionShape::MakeSphere(BroadcastRadius), Params);
	}
	else
	{
		const FVector MidPoint = (LocationL + LocationR) * 0.5;
		GetWorld()->OverlapMultiByChannel(OverlapsL, MidPoint, FQuat::Identity, BroadcastChannel, FCollisionShape::MakeSphere(BroadcastRadius), Params);
	}

	TArray<AController*> ControllerList;
	for (int32 i = 0; i < OverlapsL.Num(); i++)
	{
		APawn* MovingPawn = OverlapsL[i].OverlapObjectHandle.FetchActor<APawn>();
		if (MovingPawn && MovingPawn->GetController())
		{
			ControllerList.Add(MovingPawn->GetController());
		}
	}
	for (int32 i = 0; i < OverlapsR.Num(); i++)
	{
		APawn* MovingPawn = OverlapsR[i].OverlapObjectHandle.FetchActor<APawn>();
		if (MovingPawn && MovingPawn->GetController())
		{
			ControllerList.Add(MovingPawn->GetController());
		}
	}

	for (AController* Controller : ControllerList)
	{
		IPathFollowingAgentInterface* PFAgent = Controller->GetPathFollowingAgent();
		UObject* PFAgentObject = Cast<UObject>(PFAgent);
		if (PFAgentObject)
		{
			NotifyList.Add(PFAgentObject);
		}
	}
}

void UNavLinkCustomComponent::BroadcastStateChange()
{
	TArray<UObject*> NearbyAgents;

	CollectNearbyAgents(NearbyAgents);
	OnBroadcastFilter.ExecuteIfBound(this, NearbyAgents);

	if (BroadcastInterval > 0.0f)
	{
		GetWorld()->GetTimerManager().SetTimer(TimerHandle_BroadcastStateChange, this, &UNavLinkCustomComponent::BroadcastStateChange, BroadcastInterval);
	}
}

FVector UNavLinkCustomComponent::GetStartPoint() const
{
	return GetOwner()->GetTransform().TransformPosition(LinkRelativeStart);
}

FVector UNavLinkCustomComponent::GetEndPoint() const
{
	return GetOwner()->GetTransform().TransformPosition(LinkRelativeEnd);
}

