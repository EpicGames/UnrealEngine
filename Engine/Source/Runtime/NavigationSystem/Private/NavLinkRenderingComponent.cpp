// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavLinkRenderingComponent.h"
#include "EngineGlobals.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "NavigationSystem.h"
#include "Engine/Engine.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "MeshElementCollector.h"
#include "Engine/CollisionProfile.h"
#include "PrimitiveDrawingUtils.h"
#include "AI/Navigation/NavLinkDefinition.h"
#include "NavLinkRenderingProxy.h"
#include "NavLinkHostInterface.h"
#include "NavMesh/RecastNavMesh.h"
#include "SceneView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavLinkRenderingComponent)

//----------------------------------------------------------------------//
// UNavLinkRenderingComponent
//----------------------------------------------------------------------//
UNavLinkRenderingComponent::UNavLinkRenderingComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// properties

	// Allows updating in game, while optimizing rendering for the case that it is not modified
	Mobility = EComponentMobility::Stationary;

	BodyInstance.SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);

	bIsEditorOnly = true;

	SetGenerateOverlapEvents(false);
}

FBoxSphereBounds UNavLinkRenderingComponent::CalcBounds(const FTransform& InLocalToWorld) const
{
	AActor* LinkOwnerActor = GetOwner();
	if (LinkOwnerActor != NULL)
	{
		FBox BoundingBox(ForceInit);

		INavLinkHostInterface* LinkOwnerHost = Cast<INavLinkHostInterface>(LinkOwnerActor);
		if (LinkOwnerHost != NULL)
		{
			TArray<TSubclassOf<UNavLinkDefinition> > NavLinkClasses;
			TArray<FNavigationLink> SimpleLinks;
			TArray<FNavigationSegmentLink> DummySegmentLinks;

			if (LinkOwnerHost->GetNavigationLinksClasses(NavLinkClasses))
			{
				for (int32 NavLinkClassIdx = 0; NavLinkClassIdx < NavLinkClasses.Num(); ++NavLinkClassIdx)
				{
					if (NavLinkClasses[NavLinkClassIdx] != NULL)
					{
						const TArray<FNavigationLink>& Links = UNavLinkDefinition::GetLinksDefinition(NavLinkClasses[NavLinkClassIdx]);
						for (const auto& Link : Links)
						{
							BoundingBox += Link.Left;
							BoundingBox += Link.Right;
						}
					}
				}
			}
			if (LinkOwnerHost->GetNavigationLinksArray(SimpleLinks, DummySegmentLinks))
			{
				for (const auto& Link : SimpleLinks)
				{
					BoundingBox += Link.Left;
					BoundingBox += Link.Right;
				}
			}
		}

		// BoundingBox is in actor space. Incorporate provided InLocalToWorld transform via component space.
		const FTransform ActorToWorld = LinkOwnerActor->ActorToWorld();
		const FTransform WorldToComponent = GetComponentTransform().Inverse();
		return FBoxSphereBounds(BoundingBox).TransformBy(ActorToWorld * WorldToComponent * InLocalToWorld);
	}

	return FBoxSphereBounds(ForceInitToZero);
}

FPrimitiveSceneProxy* UNavLinkRenderingComponent::CreateSceneProxy()
{
	return new FNavLinkRenderingProxy(this);
}

#if WITH_EDITOR
bool UNavLinkRenderingComponent::ComponentIsTouchingSelectionBox(const FBox& InSelBBox, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const
{
	// NavLink rendering components not treated as 'selectable' in editor
	return false;
}

bool UNavLinkRenderingComponent::ComponentIsTouchingSelectionFrustum(const FConvexVolume& InFrustum, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const
{
	// NavLink rendering components not treated as 'selectable' in editor
	return false;
}
#endif

//----------------------------------------------------------------------//
// FNavLinkRenderingProxy
//----------------------------------------------------------------------//
FNavLinkRenderingProxy::FNavLinkRenderingProxy(const UPrimitiveComponent* InComponent)
	: FPrimitiveSceneProxy(InComponent)
{
	LinkOwnerActor = InComponent->GetOwner();
	LinkOwnerHost = Cast<INavLinkHostInterface>((UPrimitiveComponent*)InComponent);

	if (LinkOwnerHost == nullptr)
	{
		LinkOwnerHost = Cast<INavLinkHostInterface>(LinkOwnerActor);
	}

	if (LinkOwnerActor != NULL && LinkOwnerHost != NULL)
	{
		const FTransform LinkOwnerLocalToWorld = LinkOwnerActor->ActorToWorld();
		TArray<TSubclassOf<UNavLinkDefinition> > NavLinkClasses;
		LinkOwnerHost->GetNavigationLinksClasses(NavLinkClasses);

		for (int32 NavLinkClassIdx = 0; NavLinkClassIdx < NavLinkClasses.Num(); ++NavLinkClassIdx)
		{
			if (NavLinkClasses[NavLinkClassIdx] != NULL)
			{
				StorePointLinks(LinkOwnerLocalToWorld, UNavLinkDefinition::GetLinksDefinition(NavLinkClasses[NavLinkClassIdx]));
				StoreSegmentLinks(LinkOwnerLocalToWorld, UNavLinkDefinition::GetSegmentLinksDefinition(NavLinkClasses[NavLinkClassIdx]));
			}
		}

		TArray<FNavigationLink> PointLinks;
		TArray<FNavigationSegmentLink> SegmentLinks;
		if (LinkOwnerHost->GetNavigationLinksArray(PointLinks, SegmentLinks))
		{
			StorePointLinks(LinkOwnerLocalToWorld, PointLinks);
			StoreSegmentLinks(LinkOwnerLocalToWorld, SegmentLinks);
		}
	}
}

SIZE_T FNavLinkRenderingProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

void FNavLinkRenderingProxy::StorePointLinks(const FTransform& InLocalToWorld, const TArray<FNavigationLink>& LinksArray)
{
	OffMeshPointLinks.Reserve(OffMeshPointLinks.Num() + LinksArray.Num());
	for (const FNavigationLink& Link : LinksArray)
	{
		OffMeshPointLinks.Emplace(InLocalToWorld, Link);
	}
}

void FNavLinkRenderingProxy::StoreSegmentLinks(const FTransform& InLocalToWorld, const TArray<FNavigationSegmentLink>& LinksArray)
{
	OffMeshSegmentLinks.Reserve(OffMeshSegmentLinks.Num() + LinksArray.Num());
	for (const FNavigationSegmentLink& Link : LinksArray)
	{
		OffMeshSegmentLinks.Emplace(InLocalToWorld, Link);
	}
}

void FNavLinkRenderingProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const 
{
	if (LinkOwnerActor && LinkOwnerActor->GetWorld())
	{
		const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<const UNavigationSystemV1>(LinkOwnerActor->GetWorld());
		TArray<float> StepHeights;
		uint32 AgentMask = 0;
		if (NavSys != NULL)
		{
			StepHeights.Reserve(NavSys->NavDataSet.Num());
			for(int32 DataIndex = 0; DataIndex < NavSys->NavDataSet.Num(); ++DataIndex)
			{
				const ARecastNavMesh* NavMesh = Cast<const ARecastNavMesh>(NavSys->NavDataSet[DataIndex]);

				if (NavMesh != NULL)
				{
					AgentMask = NavMesh->IsDrawingEnabled() ? AgentMask | (1 << DataIndex) : AgentMask;
#if WITH_RECAST
					const float AgentMaxStepHeight = NavMesh->GetAgentMaxStepHeight(ENavigationDataResolution::Default);
					if (AgentMaxStepHeight > 0 && NavMesh->IsDrawingEnabled())
					{
						StepHeights.Add(AgentMaxStepHeight);
					}
#endif // WITH_RECAST
				}
			}
		}

		static const FColor RadiusColor(150, 160, 150, 48);
		FMaterialRenderProxy* const MeshColorInstance = &Collector.AllocateOneFrameResource<FColoredMaterialRenderProxy>(GEngine->DebugMeshMaterial->GetRenderProxy(), RadiusColor);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				FNavLinkRenderingProxy::GetLinkMeshes(OffMeshPointLinks, OffMeshSegmentLinks, StepHeights, MeshColorInstance, ViewIndex, Collector, AgentMask);
			}
		}
	}
}

void FNavLinkRenderingProxy::GetLinkMeshes(const TArray<FNavLinkDrawing>& OffMeshPointLinks, const TArray<FNavLinkSegmentDrawing>& OffMeshSegmentLinks, TArray<float>& StepHeights, FMaterialRenderProxy* const MeshColorInstance, int32 ViewIndex, FMeshElementCollector& Collector, uint32 AgentMask)
{
	static const FColor LinkColor(0,0,166);
	static const float LinkArcThickness = 3.f;
	static const float LinkArcHeight = 0.4f;

	if (StepHeights.Num() == 0)
	{
		StepHeights.Add(FNavigationSystem::FallbackAgentHeight / 2);
	}

	FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);

	for (int32 LinkIndex = 0; LinkIndex < OffMeshPointLinks.Num(); ++LinkIndex)
	{
		const FNavLinkDrawing& Link = OffMeshPointLinks[LinkIndex];
		if ((Link.SupportedAgentsBits & AgentMask) == 0)
		{
			continue;
		}
		const FVector::FReal RealSegments = FMath::Max(LinkArcHeight * (Link.Right - Link.Left).Size() / 10., 8.);
		check(RealSegments >= 0 && RealSegments <= (FVector::FReal)TNumericLimits<uint32>::Max());
		const uint32 Segments = static_cast<uint32>(RealSegments);
		DrawArc(PDI, Link.Left, Link.Right, LinkArcHeight, Segments, Link.Color, SDPG_World, 3.5f);
		const FVector VOffset(0,0,FVector::Dist(Link.Left, Link.Right)*1.333f);

		switch (Link.Direction)
		{
		case ENavLinkDirection::LeftToRight:
			DrawArrowHead(PDI, Link.Right, Link.Left+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			break;
		case ENavLinkDirection::RightToLeft:
			DrawArrowHead(PDI, Link.Left, Link.Right+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			break;
		case ENavLinkDirection::BothWays:
		default:
			DrawArrowHead(PDI, Link.Right, Link.Left+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			DrawArrowHead(PDI, Link.Left, Link.Right+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			break;
		}

		// draw snap-spheres on both ends
		if (Link.SnapHeight < 0)
		{
			for (int32 StepHeightIndex = 0; StepHeightIndex < StepHeights.Num(); ++StepHeightIndex)
			{
				GetCylinderMesh(Link.Right, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, StepHeights[StepHeightIndex], 10, MeshColorInstance, SDPG_World, ViewIndex, Collector);
				GetCylinderMesh(Link.Left, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, StepHeights[StepHeightIndex], 10, MeshColorInstance, SDPG_World, ViewIndex, Collector);
			}
		}
		else
		{
			GetCylinderMesh(Link.Right, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, Link.SnapHeight, 10, MeshColorInstance, SDPG_World, ViewIndex, Collector);
			GetCylinderMesh(Link.Left, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, Link.SnapHeight, 10, MeshColorInstance, SDPG_World, ViewIndex, Collector);
		}
	}

	static const float SegmentArcHeight = 0.25f;
	for (int32 LinkIndex = 0; LinkIndex < OffMeshSegmentLinks.Num(); ++LinkIndex)
	{
		const FNavLinkSegmentDrawing& Link = OffMeshSegmentLinks[LinkIndex];
		if ((Link.SupportedAgentsBits & AgentMask) == 0)
		{
			continue;
		}
		const FVector::FReal RealSegmentsStart = FMath::Max(SegmentArcHeight * (Link.RightStart - Link.LeftStart).Size() / 10., 8.);
		const FVector::FReal RealSegmentsEnd= FMath::Max(SegmentArcHeight * (Link.RightEnd - Link.LeftEnd).Size() / 10., 8.);
		check(RealSegmentsStart >= 0 && RealSegmentsStart <= (FVector::FReal)TNumericLimits<uint32>::Max());
		check(RealSegmentsEnd >= 0 && RealSegmentsEnd <= (FVector::FReal)TNumericLimits<uint32>::Max());
		const uint32 SegmentsStart = static_cast<uint32>(RealSegmentsStart);
		const uint32 SegmentsEnd = static_cast<uint32>(RealSegmentsEnd);
		DrawArc(PDI, Link.LeftStart, Link.RightStart, SegmentArcHeight, SegmentsStart, Link.Color, SDPG_World, 3.5f);
		DrawArc(PDI, Link.LeftEnd, Link.RightEnd, SegmentArcHeight, SegmentsEnd, Link.Color, SDPG_World, 3.5f);
		const FVector VOffset(0,0,FVector::Dist(Link.LeftStart, Link.RightStart)*1.333f);

		switch (Link.Direction)
		{
		case ENavLinkDirection::LeftToRight:
			DrawArrowHead(PDI, Link.RightStart, Link.LeftStart+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			DrawArrowHead(PDI, Link.RightEnd, Link.LeftEnd+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			break;
		case ENavLinkDirection::RightToLeft:
			DrawArrowHead(PDI, Link.LeftStart, Link.RightStart+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			DrawArrowHead(PDI, Link.LeftEnd, Link.RightEnd+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			break;
		case ENavLinkDirection::BothWays:
		default:
			DrawArrowHead(PDI, Link.RightStart, Link.LeftStart+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			DrawArrowHead(PDI, Link.RightEnd, Link.LeftEnd+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			DrawArrowHead(PDI, Link.LeftStart, Link.RightStart+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			DrawArrowHead(PDI, Link.LeftEnd, Link.RightEnd+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			break;
		}

		// draw snap-spheres on both ends
		if (Link.SnapHeight < 0)
		{
			for (int32 StepHeightIndex = 0; StepHeightIndex < StepHeights.Num(); ++StepHeightIndex)
			{
				GetCylinderMesh(Link.RightStart, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, StepHeights[StepHeightIndex], 10, MeshColorInstance, SDPG_World, ViewIndex, Collector);
				GetCylinderMesh(Link.RightEnd, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, StepHeights[StepHeightIndex], 10, MeshColorInstance, SDPG_World, ViewIndex, Collector);
				GetCylinderMesh(Link.LeftStart, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, StepHeights[StepHeightIndex], 10, MeshColorInstance, SDPG_World, ViewIndex, Collector);
				GetCylinderMesh(Link.LeftEnd, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, StepHeights[StepHeightIndex], 10, MeshColorInstance, SDPG_World, ViewIndex, Collector);
			}
		}
		else
		{
			GetCylinderMesh(Link.RightStart, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, Link.SnapHeight, 10, MeshColorInstance, SDPG_World, ViewIndex, Collector);
			GetCylinderMesh(Link.RightEnd, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, Link.SnapHeight, 10, MeshColorInstance, SDPG_World, ViewIndex, Collector);
			GetCylinderMesh(Link.LeftStart, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, Link.SnapHeight, 10, MeshColorInstance, SDPG_World, ViewIndex, Collector);
			GetCylinderMesh(Link.LeftEnd, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, Link.SnapHeight, 10, MeshColorInstance, SDPG_World, ViewIndex, Collector);
		}
	}
}

void FNavLinkRenderingProxy::DrawLinks(FPrimitiveDrawInterface* PDI, TArray<FNavLinkDrawing>& OffMeshPointLinks, TArray<FNavLinkSegmentDrawing>& OffMeshSegmentLinks, TArray<float>& StepHeights, FMaterialRenderProxy* const MeshColorInstance, uint32 AgentMask)
{
	static const FColor LinkColor(0,0,166);
	static const float LinkArcThickness = 3.f;
	static const float LinkArcHeight = 0.4f;

	if (StepHeights.Num() == 0)
	{
		StepHeights.Add(FNavigationSystem::FallbackAgentHeight / 2);
	}

	for (int32 LinkIndex = 0; LinkIndex < OffMeshPointLinks.Num(); ++LinkIndex)
	{
		const FNavLinkDrawing& Link = OffMeshPointLinks[LinkIndex];
		if ((Link.SupportedAgentsBits & AgentMask) == 0)
		{
			continue;
		}

		const FVector::FReal RealSegments = FMath::Max(LinkArcHeight * (Link.Right - Link.Left).Size() / 10., 8.);
		check(RealSegments >= 0 && RealSegments <= (FVector::FReal)TNumericLimits<uint32>::Max());
		const uint32 Segments = static_cast<uint32>(RealSegments);
		DrawArc(PDI, Link.Left, Link.Right, LinkArcHeight, Segments, Link.Color, SDPG_World, 3.5f);
		const FVector VOffset(0,0,FVector::Dist(Link.Left, Link.Right)*1.333f);

		switch (Link.Direction)
		{
		case ENavLinkDirection::LeftToRight:
			DrawArrowHead(PDI, Link.Right, Link.Left+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			break;
		case ENavLinkDirection::RightToLeft:
			DrawArrowHead(PDI, Link.Left, Link.Right+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			break;
		case ENavLinkDirection::BothWays:
		default:
			DrawArrowHead(PDI, Link.Right, Link.Left+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			DrawArrowHead(PDI, Link.Left, Link.Right+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			break;
		}

		// draw snap-spheres on both ends
		if (Link.SnapHeight < 0)
		{
			for (int32 StepHeightIndex = 0; StepHeightIndex < StepHeights.Num(); ++StepHeightIndex)
			{
				DrawCylinder(PDI, Link.Right, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, StepHeights[StepHeightIndex], 10, MeshColorInstance, SDPG_World);
				DrawCylinder(PDI, Link.Left, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, StepHeights[StepHeightIndex], 10, MeshColorInstance, SDPG_World);
			}
		}
		else
		{
			DrawCylinder(PDI, Link.Right, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, Link.SnapHeight, 10, MeshColorInstance, SDPG_World);
			DrawCylinder(PDI, Link.Left, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, Link.SnapHeight, 10, MeshColorInstance, SDPG_World);
		}
	}

	static const float SegmentArcHeight = 0.25f;
	for (int32 LinkIndex = 0; LinkIndex < OffMeshSegmentLinks.Num(); ++LinkIndex)
	{
		const FNavLinkSegmentDrawing& Link = OffMeshSegmentLinks[LinkIndex];
		if ((Link.SupportedAgentsBits & AgentMask) == 0)
		{
			continue;
		}

		const FVector::FReal RealSegmentsStart = FMath::Max(SegmentArcHeight * (Link.RightStart - Link.LeftStart).Size() / 10., 8.);
		const FVector::FReal RealSegmentsEnd = FMath::Max(SegmentArcHeight * (Link.RightEnd - Link.LeftEnd).Size() / 10., 8.);
		check(RealSegmentsStart >= 0 && RealSegmentsStart <= (FVector::FReal)TNumericLimits<uint32>::Max());
		check(RealSegmentsEnd >= 0 && RealSegmentsEnd <= (FVector::FReal)TNumericLimits<uint32>::Max());
		const uint32 SegmentsStart = static_cast<uint32>(RealSegmentsStart);
		const uint32 SegmentsEnd = static_cast<uint32>(RealSegmentsEnd);
		DrawArc(PDI, Link.LeftStart, Link.RightStart, SegmentArcHeight, SegmentsStart, Link.Color, SDPG_World, 3.5f);
		DrawArc(PDI, Link.LeftEnd, Link.RightEnd, SegmentArcHeight, SegmentsEnd, Link.Color, SDPG_World, 3.5f);
		const FVector VOffset(0,0,FVector::Dist(Link.LeftStart, Link.RightStart)*1.333f);

		switch (Link.Direction)
		{
		case ENavLinkDirection::LeftToRight:
			DrawArrowHead(PDI, Link.RightStart, Link.LeftStart+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			DrawArrowHead(PDI, Link.RightEnd, Link.LeftEnd+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			break;
		case ENavLinkDirection::RightToLeft:
			DrawArrowHead(PDI, Link.LeftStart, Link.RightStart+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			DrawArrowHead(PDI, Link.LeftEnd, Link.RightEnd+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			break;
		case ENavLinkDirection::BothWays:
		default:
			DrawArrowHead(PDI, Link.RightStart, Link.LeftStart+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			DrawArrowHead(PDI, Link.RightEnd, Link.LeftEnd+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			DrawArrowHead(PDI, Link.LeftStart, Link.RightStart+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			DrawArrowHead(PDI, Link.LeftEnd, Link.RightEnd+VOffset, 30.f, Link.Color, SDPG_World, 3.5f);
			break;
		}

		// draw snap-spheres on both ends
		if (Link.SnapHeight < 0)
		{
			for (int32 StepHeightIndex = 0; StepHeightIndex < StepHeights.Num(); ++StepHeightIndex)
			{
				DrawCylinder(PDI, Link.RightStart, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, StepHeights[StepHeightIndex], 10, MeshColorInstance, SDPG_World);
				DrawCylinder(PDI, Link.RightEnd, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, StepHeights[StepHeightIndex], 10, MeshColorInstance, SDPG_World);
				DrawCylinder(PDI, Link.LeftStart, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, StepHeights[StepHeightIndex], 10, MeshColorInstance, SDPG_World);
				DrawCylinder(PDI, Link.LeftEnd, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, StepHeights[StepHeightIndex], 10, MeshColorInstance, SDPG_World);
			}
		}
		else
		{
			DrawCylinder(PDI, Link.RightStart, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, Link.SnapHeight, 10, MeshColorInstance, SDPG_World);
			DrawCylinder(PDI, Link.RightEnd, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, Link.SnapHeight, 10, MeshColorInstance, SDPG_World);
			DrawCylinder(PDI, Link.LeftStart, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, Link.SnapHeight, 10, MeshColorInstance, SDPG_World);
			DrawCylinder(PDI, Link.LeftEnd, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), Link.SnapRadius, Link.SnapHeight, 10, MeshColorInstance, SDPG_World);
		}
	}
}

FPrimitiveViewRelevance FNavLinkRenderingProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = IsShown(View) && IsSelected() && (View && View->Family && View->Family->EngineShowFlags.Navigation);
	Result.bDynamicRelevance = true;
	// ideally the TranslucencyRelevance should be filled out by the material, here we do it conservative
	Result.bSeparateTranslucency = Result.bNormalTranslucency = IsShown(View);
	Result.bShadowRelevance = IsShadowCast(View);
	Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
	return Result;
}

uint32 FNavLinkRenderingProxy::GetMemoryFootprint( void ) const 
{ 
	return( sizeof( *this ) + GetAllocatedSize() ); 
}

uint32 FNavLinkRenderingProxy::GetAllocatedSize( void ) const 
{ 
	return IntCastChecked<uint32>(FPrimitiveSceneProxy::GetAllocatedSize() + OffMeshPointLinks.GetAllocatedSize() + OffMeshSegmentLinks.GetAllocatedSize());
}

