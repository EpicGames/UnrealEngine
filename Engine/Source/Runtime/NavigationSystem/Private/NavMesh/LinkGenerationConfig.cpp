// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavMesh/LinkGenerationConfig.h"
#include "BaseGeneratedNavLinksProxy.h"
#include "NavAreas/NavArea_Default.h"

#if WITH_RECAST
#include "Detour/DetourNavLinkBuilderConfig.h"
#endif //WITH_RECAST

// Deprecated FNavLinkGenerationJumpDownConfig

// Refer to header for more information

#include UE_INLINE_GENERATED_CPP_BY_NAME(LinkGenerationConfig)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
// Deprecated
FNavLinkGenerationJumpDownConfig::FNavLinkGenerationJumpDownConfig()
{
	DownDirectionAreaClass = UNavArea_Default::StaticClass();
	UpDirectionAreaClass = UNavArea_Default::StaticClass();
}

// Deprecated
bool FNavLinkGenerationJumpDownConfig::Serialize(FArchive& Ar)
{
	UScriptStruct* const Struct = StaticStruct();
	Struct->SerializeTaggedProperties(Ar, (uint8*)this, Struct, nullptr);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		if (AreaClass_DEPRECATED)
		{
			DownDirectionAreaClass = AreaClass_DEPRECATED;
			UpDirectionAreaClass = AreaClass_DEPRECATED;
			AreaClass_DEPRECATED = nullptr;
		}
	}
#endif //WITH_EDITORONLY_DATA

	return true;
}

#if WITH_RECAST
// Deprecated
void FNavLinkGenerationJumpDownConfig::CopyToDetourConfig(dtNavLinkBuilderJumpDownConfig& OutDetourConfig) const
{
	OutDetourConfig.enabled = bEnabled;
	OutDetourConfig.jumpLength = JumpLength;
	OutDetourConfig.jumpDistanceFromEdge = JumpDistanceFromEdge;
	OutDetourConfig.jumpMaxDepth = JumpMaxDepth;
	OutDetourConfig.jumpHeight = JumpHeight;
	OutDetourConfig.jumpEndsHeightTolerance	= JumpEndsHeightTolerance;
	OutDetourConfig.samplingSeparationFactor = SamplingSeparationFactor;
	OutDetourConfig.filterDistanceThreshold = FilterDistanceThreshold;
	OutDetourConfig.linkBuilderFlags = LinkBuilderFlags;

	if (LinkProxy)
	{
		OutDetourConfig.linkUserId = LinkProxy->GetId().GetId();	
	}
}
#endif //WITH_RECAST
PRAGMA_ENABLE_DEPRECATION_WARNINGS


// Refer to header for more information
PRAGMA_DISABLE_DEPRECATION_WARNINGS
FNavLinkGenerationJumpConfig::FNavLinkGenerationJumpConfig()
{
	DownDirectionAreaClass = UNavArea_Default::StaticClass();
	UpDirectionAreaClass = UNavArea_Default::StaticClass();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool FNavLinkGenerationJumpConfig::Serialize(FArchive& Ar)
{
	UScriptStruct* const Struct = StaticStruct();
	Struct->SerializeTaggedProperties(Ar, (uint8*)this, Struct, nullptr);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (AreaClass_DEPRECATED)
		{
			DownDirectionAreaClass = AreaClass_DEPRECATED;
			UpDirectionAreaClass = AreaClass_DEPRECATED;
			AreaClass_DEPRECATED = nullptr;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif //WITH_EDITORONLY_DATA

	return true;
}

#if WITH_RECAST

void FNavLinkGenerationJumpConfig::CopyToDetourConfig(dtNavLinkBuilderJumpConfig& OutDetourConfig) const
{
	OutDetourConfig.enabled = bEnabled;
	OutDetourConfig.jumpLength = JumpLength;
	OutDetourConfig.jumpDistanceFromEdge = JumpDistanceFromEdge;
	OutDetourConfig.jumpMaxDepth = JumpMaxDepth;
	OutDetourConfig.jumpHeight = JumpHeight;
	OutDetourConfig.jumpEndsHeightTolerance	= JumpEndsHeightTolerance;
	OutDetourConfig.samplingSeparationFactor = SamplingSeparationFactor;
	OutDetourConfig.filterDistanceThreshold = FilterDistanceThreshold;
	OutDetourConfig.linkBuilderFlags = LinkBuilderFlags;

	if (LinkProxy)
	{
		OutDetourConfig.linkUserId = LinkProxy->GetId().GetId();	
	}
}

#endif //WITH_RECAST
