// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavMesh/NavMeshBoundsVolume.h"
#include "NavigationSystem.h"
#include "Engine/CollisionProfile.h"
#include "Components/BrushComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavMeshBoundsVolume)

#if WITH_EDITOR
#include "ActorFactories/ActorFactory.h"
#include "Editor.h"
#endif // WITH_EDITOR

namespace UE::Navigation::Private
{
	static void ApplyNavMeshBoundsCollision(UBrushComponent* Brush, const bool bAllowOverlap)
	{
		if (!Brush)
		{
			return;
		}

		if (bAllowOverlap)
		{
			Brush->SetCollisionProfileName(UCollisionProfile::OverlapAll_ProfileName);
		}
		else
		{
			Brush->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
		}
	}
}

ANavMeshBoundsVolume::ANavMeshBoundsVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	GetBrushComponent()->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	GetBrushComponent()->Mobility = EComponentMobility::Static;

	BrushColor = FColor(200, 200, 200, 255);
	SupportedAgents.MarkInitialized();

	bColored = true;

	bAllowPhysicsOverlap = false;

#if WITH_EDITORONLY_DATA
	bIsSpatiallyLoaded = false;
#endif
}

#if WITH_EDITOR

void ANavMeshBoundsVolume::PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	if (GIsEditor && NavSys)
	{
		const FName PropName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : FName();
		const FName MemberName = (PropertyChangedEvent.MemberProperty != nullptr) ? PropertyChangedEvent.MemberProperty->GetFName() : FName();

		if (PropName == GET_MEMBER_NAME_CHECKED(ABrush, BrushBuilder)
			|| MemberName == GET_MEMBER_NAME_CHECKED(ANavMeshBoundsVolume, SupportedAgents)
			|| MemberName == USceneComponent::GetRelativeLocationPropertyName()
			|| MemberName == USceneComponent::GetRelativeRotationPropertyName()
			|| MemberName == USceneComponent::GetRelativeScale3DPropertyName())
		{
			NavSys->OnNavigationBoundsUpdated(this);
		}
	}

	if (PropertyChangedEvent.Property != nullptr)
	{
		static const FName NAME_bAllowPhysicsOverlap(TEXT("bAllowPhysicsOverlap"));
		if (PropertyChangedEvent.Property->GetFName() == NAME_bAllowPhysicsOverlap)
		{
			UE::Navigation::Private::ApplyNavMeshBoundsCollision(GetBrushComponent(), bAllowPhysicsOverlap);
		}
	}
}

void ANavMeshBoundsVolume::PostEditUndo()
{
	Super::PostEditUndo();
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	if (GIsEditor && NavSys)
	{
		NavSys->OnNavigationBoundsUpdated(this);
	}

	UE::Navigation::Private::ApplyNavMeshBoundsCollision(GetBrushComponent(), bAllowPhysicsOverlap);
}

void ANavMeshBoundsVolume::OnPostEngineInit()
{
	if (GEditor)
	{
		const TArray<UActorFactory*>& ActorFactories = GEditor->ActorFactories;
		for (UActorFactory* Factory : ActorFactories)
		{
			// For ANavMeshBoundsVolume, do not use placement extent so that the volume embeds into the surface.
			// When flush with the surface, the collision might not be in the volume and the navmesh might not generate.
			const TSubclassOf<AActor> ActorClass = Factory->NewActorClass;
			if (ActorClass != nullptr && ActorClass->IsChildOf(ANavMeshBoundsVolume::StaticClass()))
			{
				Factory->bUsePlacementExtent = false;
			}
		}
	}
}

#endif // WITH_EDITOR

void ANavMeshBoundsVolume::PostRegisterAllComponents() 
{
	Super::PostRegisterAllComponents();
	
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	if (NavSys && GetLocalRole() == ROLE_Authority)
	{
		NavSys->OnNavigationBoundsAdded(this);
	}

	UE::Navigation::Private::ApplyNavMeshBoundsCollision(GetBrushComponent(), bAllowPhysicsOverlap);
}

void ANavMeshBoundsVolume::PostUnregisterAllComponents() 
{
	Super::PostUnregisterAllComponents();
	
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	if (NavSys && GetLocalRole() == ROLE_Authority)
	{
		NavSys->OnNavigationBoundsRemoved(this);
	}
}

