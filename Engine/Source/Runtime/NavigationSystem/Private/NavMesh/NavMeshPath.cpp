// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavMesh/NavMeshPath.h"
#include "EngineStats.h"
#include "EngineGlobals.h"
#include "AI/Navigation/NavAgentInterface.h"
#include "NavigationSystem.h"
#include "Engine/Engine.h"
#include "Engine/Canvas.h"
#include "DrawDebugHelpers.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "NavMesh/RecastNavMesh.h"
#include "NavAreas/NavArea.h"
#include "Debug/DebugDrawService.h"
#include "Algo/Reverse.h"

#define DEBUG_DRAW_OFFSET 0
#define PATH_OFFSET_KEEP_VISIBLE_POINTS 1


//----------------------------------------------------------------------//
// FNavMeshPath
//----------------------------------------------------------------------//
const FNavPathType FNavMeshPath::Type;
	
FNavMeshPath::FNavMeshPath()
	: bWantsStringPulling(true)
	, bWantsPathCorridor(false)
{
	PathType = FNavMeshPath::Type;
	InternalResetNavMeshPath();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FNavMeshPath::~FNavMeshPath() = default;
FNavMeshPath::FNavMeshPath(const FNavMeshPath&) = default;
FNavMeshPath::FNavMeshPath(FNavMeshPath&& Other) = default;
FNavMeshPath& FNavMeshPath::operator=(const FNavMeshPath& Other) = default;
FNavMeshPath& FNavMeshPath::operator=(FNavMeshPath&& Other) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FNavMeshPath::ResetForRepath()
{
	Super::ResetForRepath();
	InternalResetNavMeshPath();
}

void FNavMeshPath::InternalResetNavMeshPath()
{
	PathCorridor.Reset();
	PathCorridorCost.Reset();
	CustomNavLinkIds.Reset();
	PathCorridorEdges.Reset();

	bCorridorEdgesGenerated = false;
	bDynamic = false;
	bStringPulled = false;

	// keep:
	// - bWantsStringPulling
	// - bWantsPathCorridor
}

FVector::FReal FNavMeshPath::GetStringPulledLength(const int32 StartingPoint) const
{
	if (IsValid() == false || StartingPoint >= PathPoints.Num())
	{
		return 0.f;
	}

	FVector::FReal TotalLength = 0.f;
	const FNavPathPoint* PrevPoint = PathPoints.GetData() + StartingPoint;
	const FNavPathPoint* PathPoint = PrevPoint + 1;

	for (int32 PathPointIndex = StartingPoint + 1; PathPointIndex < PathPoints.Num(); ++PathPointIndex, ++PathPoint, ++PrevPoint)
	{
		TotalLength += FVector::Dist(PrevPoint->Location, PathPoint->Location);
	}

	return TotalLength;
}

FVector::FReal FNavMeshPath::GetPathCorridorLength(const int32 StartingEdge) const
{
	if (bCorridorEdgesGenerated == false)
	{
		return 0.f;
	}
	else if (StartingEdge >= PathCorridorEdges.Num())
	{
		return StartingEdge == 0 && PathPoints.Num() > 1 ? FVector::Dist(PathPoints[0].Location, PathPoints[PathPoints.Num()-1].Location) : 0.;
	}
	
	const FNavigationPortalEdge* PrevEdge = PathCorridorEdges.GetData() + StartingEdge;
	const FNavigationPortalEdge* CorridorEdge = PrevEdge + 1;
	FVector PrevEdgeMiddle = PrevEdge->GetMiddlePoint();

	FVector::FReal TotalLength = StartingEdge == 0 ? FVector::Dist(PathPoints[0].Location, PrevEdgeMiddle)
		: FVector::Dist(PrevEdgeMiddle, PathCorridorEdges[StartingEdge - 1].GetMiddlePoint());

	for (int32 PathPolyIndex = StartingEdge + 1; PathPolyIndex < PathCorridorEdges.Num(); ++PathPolyIndex, ++PrevEdge, ++CorridorEdge)
	{
		const FVector CurrentEdgeMiddle = CorridorEdge->GetMiddlePoint();
		TotalLength += FVector::Dist(CurrentEdgeMiddle, PrevEdgeMiddle);
		PrevEdgeMiddle = CurrentEdgeMiddle;
	}
	// @todo add distance to last point here!
	return TotalLength;
}

const TArray<FNavigationPortalEdge>& FNavMeshPath::GeneratePathCorridorEdges() const
{
#if WITH_RECAST
	// mz@todo the underlying recast function queries the navmesh a portal at a time, 
	// which is a waste of performance. A batch-query function has to be added.
	const int32 CorridorLength = PathCorridor.Num();
	if (CorridorLength != 0 && IsInGameThread() && NavigationDataUsed.IsValid())
	{
		const ARecastNavMesh* MyOwner = Cast<ARecastNavMesh>(GetNavigationDataUsed());
		if (MyOwner)
		{
			MyOwner->GetEdgesForPathCorridor(&PathCorridor, &PathCorridorEdges);
			bCorridorEdgesGenerated = (PathCorridorEdges.Num() > 0);
		}
	}
#endif // WITH_RECAST
	return PathCorridorEdges;
}

void FNavMeshPath::PerformStringPulling(const FVector& StartLoc, const FVector& EndLoc)
{
#if WITH_RECAST
	const ARecastNavMesh* MyOwner = Cast<ARecastNavMesh>(GetNavigationDataUsed());
	if (::IsValid(MyOwner) && PathCorridor.Num())
	{
		bStringPulled = MyOwner->FindStraightPath(StartLoc, EndLoc, PathCorridor, PathPoints, &CustomNavLinkIds);
	}
#endif	// WITH_RECAST
}


#if DEBUG_DRAW_OFFSET
	UWorld* GInternalDebugWorld_ = NULL;
#endif

namespace
{
	struct FPathPointInfo
	{
		FPathPointInfo() 
		{

		}
		FPathPointInfo( const FNavPathPoint& InPoint, const FVector& InEdgePt0, const FVector& InEdgePt1) 
			: Point(InPoint)
			, EdgePt0(InEdgePt0)
			, EdgePt1(InEdgePt1) 
		{ 
			/** Empty */ 
		}

		FNavPathPoint Point;
		FVector EdgePt0;
		FVector EdgePt1;
	};

	FORCEINLINE bool CheckVisibility(const FPathPointInfo* StartPoint, const FPathPointInfo* EndPoint,  TArray<FNavigationPortalEdge>& PathCorridorEdges, FVector::FReal OffsetDistannce, FPathPointInfo* LastVisiblePoint)
	{
		FVector IntersectionPoint = FVector::ZeroVector;
		FVector StartTrace = StartPoint->Point.Location;
		FVector EndTrace = EndPoint->Point.Location;

		// find closest edge to StartPoint
		FVector::FReal BestDistance = TNumericLimits<FVector::FReal>::Max();
		FNavigationPortalEdge* CurrentEdge = NULL;

		FVector::FReal BestEndPointDistance = TNumericLimits<FVector::FReal>::Max();
		FNavigationPortalEdge* EndPointEdge = NULL;
		for (int32 EdgeIndex =0; EdgeIndex < PathCorridorEdges.Num(); ++EdgeIndex)
		{
			FVector::FReal DistToEdge = TNumericLimits<FVector::FReal>::Max();
			FNavigationPortalEdge* Edge = &PathCorridorEdges[EdgeIndex];
			if (BestDistance > FMath::Square(KINDA_SMALL_NUMBER))
			{
				DistToEdge= FMath::PointDistToSegmentSquared(StartTrace, Edge->Left, Edge->Right);
				if (DistToEdge < BestDistance)
				{
					BestDistance = DistToEdge;
					CurrentEdge = Edge;
#if DEBUG_DRAW_OFFSET
					DrawDebugLine( GInternalDebugWorld_, Edge->Left, Edge->Right, FColor::White, true );
#endif
				}
			}

			if (BestEndPointDistance > FMath::Square(KINDA_SMALL_NUMBER))
			{
				DistToEdge= FMath::PointDistToSegmentSquared(EndTrace, Edge->Left, Edge->Right);
				if (DistToEdge < BestEndPointDistance)
				{
					BestEndPointDistance = DistToEdge;
					EndPointEdge = Edge;
				}
			}
		}

		if (CurrentEdge == NULL || EndPointEdge == NULL )
		{
			LastVisiblePoint->Point.Location = FVector::ZeroVector;
			return false;
		}


		if (BestDistance <= FMath::Square(KINDA_SMALL_NUMBER))
		{
			CurrentEdge++;
		}

		if (CurrentEdge == EndPointEdge)
		{
			return true;
		}

		const FVector RayNormal = (StartTrace-EndTrace) .GetSafeNormal() * OffsetDistannce;
		StartTrace = StartTrace + RayNormal;
		EndTrace = EndTrace - RayNormal;

		bool bIsVisible = true;
#if DEBUG_DRAW_OFFSET
		DrawDebugLine( GInternalDebugWorld_, StartTrace, EndTrace, FColor::Yellow, true );
#endif
		const FNavigationPortalEdge* LaseEdge = &PathCorridorEdges[PathCorridorEdges.Num()-1];
		while (CurrentEdge <= EndPointEdge)
		{
			FVector Left = CurrentEdge->Left;
			FVector Right = CurrentEdge->Right;

#if DEBUG_DRAW_OFFSET
			DrawDebugLine( GInternalDebugWorld_, Left, Right, FColor::White, true );
#endif
			bool bIntersected = FMath::SegmentIntersection2D(Left, Right, StartTrace, EndTrace, IntersectionPoint);
			if ( !bIntersected)
			{
				const FVector::FReal EdgeHalfLength = (CurrentEdge->Left - CurrentEdge->Right).Size() * 0.5f;
				const FVector::FReal Distance = FMath::Min(OffsetDistannce, EdgeHalfLength) *  0.1f;
				Left = CurrentEdge->Left + Distance * (CurrentEdge->Right - CurrentEdge->Left).GetSafeNormal();
				Right = CurrentEdge->Right + Distance * (CurrentEdge->Left - CurrentEdge->Right).GetSafeNormal();
				FVector ClosestPointOnRay, ClosestPointOnEdge;
				FMath::SegmentDistToSegment(StartTrace, EndTrace, Right, Left, ClosestPointOnRay, ClosestPointOnEdge);
#if DEBUG_DRAW_OFFSET
				DrawDebugSphere( GInternalDebugWorld_, ClosestPointOnEdge, 10, 8, FColor::Red, true );
#endif
				LastVisiblePoint->Point.Location = ClosestPointOnEdge;
				LastVisiblePoint->EdgePt0= CurrentEdge->Left ;
				LastVisiblePoint->EdgePt1= CurrentEdge->Right ;
				return false;
			}
#if DEBUG_DRAW_OFFSET
			DrawDebugSphere( GInternalDebugWorld_, IntersectionPoint, 8, 8, FColor::White, true );
#endif
			CurrentEdge++;
			bIsVisible = true;
		}

		return bIsVisible;
	}
}

void FNavMeshPath::ApplyFlags(int32 NavDataFlags)
{
	if (NavDataFlags & ERecastPathFlags::SkipStringPulling)
	{
		bWantsStringPulling = false;
	}

	if (NavDataFlags & ERecastPathFlags::GenerateCorridor)
	{
		bWantsPathCorridor = true;
	}
}

void AppendPathPointsHelper(TArray<FNavPathPoint>& PathPoints, const TArray<FPathPointInfo>& SourcePoints, int32 Index)
{
	if (SourcePoints.IsValidIndex(Index) && SourcePoints[Index].Point.NodeRef != 0)
	{
		PathPoints.Add(SourcePoints[Index].Point);
	}
}

void FNavMeshPath::OffsetFromCorners(FVector::FReal Distance)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_OffsetFromCorners);

	const ARecastNavMesh* MyOwner = Cast<ARecastNavMesh>(GetNavigationDataUsed());
	if (MyOwner == nullptr || PathPoints.Num() == 0 || PathPoints.Num() > 100)
	{
		// skip it, there is not need to offset that path from performance point of view
		return;
	}

#if DEBUG_DRAW_OFFSET
	GInternalDebugWorld_ = MyOwner->GetWorld();
	FlushDebugStrings(GInternalDebugWorld_);
	FlushPersistentDebugLines(GInternalDebugWorld_);
#endif

	if (bCorridorEdgesGenerated == false)
	{
		GeneratePathCorridorEdges(); 
	}
	const FVector::FReal DistanceSq = Distance * Distance;
	int32 CurrentEdge = 0;
	bool bNeedToCopyResults = false;
	int32 SingleNodePassCount = 0;

	FNavPathPoint* PathPoint = PathPoints.GetData();
	// it's possible we'll be inserting points into the path, so we need to buffer the result
	TArray<FPathPointInfo> FirstPassPoints;
	FirstPassPoints.Reserve(PathPoints.Num() + 2);
	FirstPassPoints.Add(FPathPointInfo(*PathPoint, FVector::ZeroVector, FVector::ZeroVector));
	++PathPoint;

	// for every point on path find a related corridor edge
	for (int32 PathNodeIndex = 1; PathNodeIndex < PathPoints.Num()-1 && CurrentEdge < PathCorridorEdges.Num();)
	{
		if (FNavMeshNodeFlags(PathPoint->Flags).PathFlags & RECAST_STRAIGHTPATH_OFFMESH_CONNECTION)
		{
			// put both ends
			FirstPassPoints.Add(FPathPointInfo(*PathPoint, FVector(0), FVector(0)));
			FirstPassPoints.Add(FPathPointInfo(*(PathPoint+1), FVector(0), FVector(0)));
			PathNodeIndex += 2;
			PathPoint += 2;
			continue;
		}

		int32 CloserPoint = -1;
		const FNavigationPortalEdge* Edge = &PathCorridorEdges[CurrentEdge];
		for (int32 EdgeIndex = CurrentEdge; EdgeIndex < PathCorridorEdges.Num(); ++Edge, ++EdgeIndex)
		{
			const FVector::FReal DistToSequence = FMath::PointDistToSegmentSquared(PathPoint->Location, Edge->Left, Edge->Right);
			if (DistToSequence <= FMath::Square(KINDA_SMALL_NUMBER))
			{
				const FVector::FReal LeftDistanceSq = FVector::DistSquared(PathPoint->Location, Edge->Left);
				const FVector::FReal RightDistanceSq = FVector::DistSquared(PathPoint->Location, Edge->Right);
				if (LeftDistanceSq > DistanceSq && RightDistanceSq > DistanceSq)
				{
					++CurrentEdge;
				}
				else
				{
					CloserPoint = LeftDistanceSq < RightDistanceSq ? 0 : 1;
					CurrentEdge = EdgeIndex;
				}
				break;
			}
		}

		if (CloserPoint >= 0)
		{
			bNeedToCopyResults = true;

			Edge = &PathCorridorEdges[CurrentEdge];
			const FVector::FReal ActualOffset = FPlatformMath::Min(Edge->GetLength()/2, Distance);

			FNavPathPoint NewPathPoint = *PathPoint;
			// apply offset 

			const FVector EdgePt0 = Edge->GetPoint(CloserPoint);
			const FVector EdgePt1 = Edge->GetPoint((CloserPoint+1)%2);
			const FVector EdgeDir = EdgePt1 - EdgePt0;
			const FVector EdgeOffset = EdgeDir.GetSafeNormal() * ActualOffset;
			NewPathPoint.Location = EdgePt0 + EdgeOffset;
			// update NodeRef (could be different if this is n-th pass on the same PathPoint
			NewPathPoint.NodeRef = Edge->ToRef;
			FirstPassPoints.Add(FPathPointInfo(NewPathPoint, EdgePt0, EdgePt1));

			// if we've found a matching edge it's possible there's also another one there using the same edge. 
			// that's why we need to repeat the process with the same path point and next edge
			++CurrentEdge;

			// we need to know if we did more than one iteration on a given point
			// if so then we should not add that point in following "else" statement
			++SingleNodePassCount;
		}
		else
		{
			if (SingleNodePassCount == 0)
			{
				// store unchanged
				FirstPassPoints.Add(FPathPointInfo(*PathPoint, FVector(0), FVector(0)));
			}
			else
			{
				SingleNodePassCount = 0;
			}

			++PathNodeIndex;
			++PathPoint;
		}
	}

	if (bNeedToCopyResults)
	{
		if (FirstPassPoints.Num() < 3 || !MyOwner->bUseBetterOffsetsFromCorners)
		{
			FNavPathPoint EndPt = PathPoints.Last();

			PathPoints.Reset();
			for (int32 Index=0; Index < FirstPassPoints.Num(); ++Index)
			{
				PathPoints.Add(FirstPassPoints[Index].Point);
			}

			PathPoints.Add(EndPt);
			return;
		}

		TArray<FNavPathPoint> DestinationPathPoints;
		DestinationPathPoints.Reserve(FirstPassPoints.Num() + 2);

		// don't forget the last point
		FirstPassPoints.Add(FPathPointInfo(PathPoints[PathPoints.Num()-1], FVector::ZeroVector, FVector::ZeroVector));

		int32 StartPointIndex = 0;
		int32 LastVisiblePointIndex = 0;
		int32 TestedPointIndex = 1;
		int32 LastPointIndex = FirstPassPoints.Num()-1;

		const int32 MaxSteps = 200;
		for (int32 StepsLeft = MaxSteps; StepsLeft >= 0; StepsLeft--)
		{ 
			if (StartPointIndex == TestedPointIndex || StepsLeft == 0)
			{
				// something went wrong, or exceeded limit of steps (= went even more wrong)
				DestinationPathPoints.Reset();
				break;
			}

			const FNavMeshNodeFlags LastVisibleFlags(FirstPassPoints[LastVisiblePointIndex].Point.Flags);
			const FNavMeshNodeFlags StartPointFlags(FirstPassPoints[StartPointIndex].Point.Flags);
			bool bWantsVisibilityInsert = true;

			if (StartPointFlags.PathFlags & RECAST_STRAIGHTPATH_OFFMESH_CONNECTION) 
			{
				AppendPathPointsHelper(DestinationPathPoints, FirstPassPoints, StartPointIndex);
				AppendPathPointsHelper(DestinationPathPoints, FirstPassPoints, StartPointIndex + 1);

				StartPointIndex++;
				LastVisiblePointIndex = StartPointIndex;
				TestedPointIndex = LastVisiblePointIndex + 1;
				
				// skip inserting new points
				bWantsVisibilityInsert = false;
			}
			
			bool bVisible = false; 
			if (((LastVisibleFlags.PathFlags & RECAST_STRAIGHTPATH_OFFMESH_CONNECTION) == 0) && (StartPointFlags.Area == LastVisibleFlags.Area))
			{
				FPathPointInfo LastVisiblePoint;
				bVisible = CheckVisibility( &FirstPassPoints[StartPointIndex], &FirstPassPoints[TestedPointIndex], PathCorridorEdges, Distance, &LastVisiblePoint );
				if (!bVisible)
				{
					if (LastVisiblePoint.Point.Location.IsNearlyZero())
					{
						DestinationPathPoints.Reset();
						break;
					}
					else if (StartPointIndex == LastVisiblePointIndex)
					{
						/** add new point only if we don't see our next location otherwise use last visible point*/
						LastVisiblePoint.Point.Flags = FirstPassPoints[LastVisiblePointIndex].Point.Flags;
						LastVisiblePointIndex = FirstPassPoints.Insert( LastVisiblePoint, StartPointIndex+1 );
						LastPointIndex = FirstPassPoints.Num()-1;

						// TODO: potential infinite loop - keeps inserting point without visibility
					}
				}
			}

			if (bWantsVisibilityInsert)
			{
				if (bVisible) 
				{ 
#if PATH_OFFSET_KEEP_VISIBLE_POINTS
					AppendPathPointsHelper(DestinationPathPoints, FirstPassPoints, StartPointIndex);
					LastVisiblePointIndex = TestedPointIndex;
					StartPointIndex = LastVisiblePointIndex;
					TestedPointIndex++;
#else
					LastVisiblePointIndex = TestedPointIndex;
					TestedPointIndex++;
#endif
				} 
				else
				{ 
					AppendPathPointsHelper(DestinationPathPoints, FirstPassPoints, StartPointIndex);
					StartPointIndex = LastVisiblePointIndex;
					TestedPointIndex = LastVisiblePointIndex + 1;
				} 
			}

			// if reached end of path, add current and last points to close it and leave loop
			if (TestedPointIndex > LastPointIndex) 
			{
				AppendPathPointsHelper(DestinationPathPoints, FirstPassPoints, StartPointIndex);
				AppendPathPointsHelper(DestinationPathPoints, FirstPassPoints, LastPointIndex);
				break; 
			} 
		} 

		if (DestinationPathPoints.Num())
		{
			PathPoints = DestinationPathPoints;
		}
	}
}

bool FNavMeshPath::IsPathSegmentANavLink(const int32 PathSegmentStartIndex) const
{
	return PathPoints.IsValidIndex(PathSegmentStartIndex)
		&& FNavMeshNodeFlags(PathPoints[PathSegmentStartIndex].Flags).IsNavLink();
}

void FNavMeshPath::DebugDraw(const ANavigationData* NavData, const FColor PathColor, UCanvas* Canvas, const bool bPersistent, const float LifeTime, const uint32 NextPathPointIndex) const
{
	Super::DebugDraw(NavData, PathColor, Canvas, bPersistent, LifeTime, NextPathPointIndex);

#if WITH_RECAST && ENABLE_DRAW_DEBUG
	const ARecastNavMesh* RecastNavMesh = Cast<const ARecastNavMesh>(NavData);		
	const TArray<FNavigationPortalEdge>& Edges = GetPathCorridorEdges();
	const int32 CorridorEdgesCount = Edges.Num();
	const UWorld* World = NavData->GetWorld();

	for (int32 EdgeIndex = 0; EdgeIndex < CorridorEdgesCount; ++EdgeIndex)
	{
		DrawDebugLine(World, Edges[EdgeIndex].Left + NavigationDebugDrawing::PathOffset, Edges[EdgeIndex].Right + NavigationDebugDrawing::PathOffset
			, FColor::Blue, bPersistent, LifeTime, /*DepthPriority*/0
			, /*Thickness*/NavigationDebugDrawing::PathLineThickness);
	}

	if (Canvas && RecastNavMesh && RecastNavMesh->bDrawLabelsOnPathNodes)
	{
		UFont* RenderFont = GEngine->GetSmallFont();
		for (int32 VertIdx = 0; VertIdx < PathPoints.Num(); ++VertIdx)
		{
			// draw box at vert
			FVector const VertLoc = PathPoints[VertIdx].Location 
				+ FVector(0, 0, NavigationDebugDrawing::PathNodeBoxExtent.Z*2)
				+ NavigationDebugDrawing::PathOffset;
			const FVector ScreenLocation = Canvas->Project(VertLoc);

			FNavMeshNodeFlags NodeFlags(PathPoints[VertIdx].Flags);
			const UClass* NavAreaClass = RecastNavMesh->GetAreaClass(NodeFlags.Area);

			Canvas->DrawText(RenderFont, FString::Printf(TEXT("%d: %s"), VertIdx, *GetNameSafe(NavAreaClass)), UE_REAL_TO_FLOAT(ScreenLocation.X), UE_REAL_TO_FLOAT(ScreenLocation.Y));
		}
	}
#endif // WITH_RECAST && ENABLE_DRAW_DEBUG
}

bool FNavMeshPath::ContainsWithSameEnd(const FNavMeshPath* Other) const
{
	if (PathCorridor.Num() < Other->PathCorridor.Num())
	{
		return false;
	}

	const NavNodeRef* ThisPathNode = &PathCorridor[PathCorridor.Num()-1];
	const NavNodeRef* OtherPathNode = &Other->PathCorridor[Other->PathCorridor.Num()-1];
	bool bAreTheSame = true;

	for (int32 NodeIndex = Other->PathCorridor.Num() - 1; NodeIndex >= 0 && bAreTheSame; --NodeIndex, --ThisPathNode, --OtherPathNode)
	{
		bAreTheSame = *ThisPathNode == *OtherPathNode;
	}	

	return bAreTheSame;
}

namespace
{
	FORCEINLINE
	bool CheckIntersectBetweenPoints(const FBox& Box, const FVector* AgentExtent, const FVector& Start, const FVector& End)
	{
		if (FVector::DistSquared(Start, End) > SMALL_NUMBER)
		{
			const FVector Direction = (End - Start);

			FVector HitLocation, HitNormal;
			float HitTime;

			// If we have a valid AgentExtent, then we use an extent box to represent the path
			// Otherwise we use a line to represent the path
			if ((AgentExtent && FMath::LineExtentBoxIntersection(Box, Start, End, *AgentExtent, HitLocation, HitNormal, HitTime)) ||
				(!AgentExtent && FMath::LineBoxIntersection(Box, Start, End, Direction)))
			{
				return true;
			}
		}

		return false;
	}
}
bool FNavMeshPath::DoesPathIntersectBoxImplementation(const FBox& Box, const FVector& StartLocation, uint32 StartingIndex, int32* IntersectingSegmentIndex, FVector* AgentExtent) const
{
	bool bIntersects = false;	
	const TArray<FNavigationPortalEdge>& CorridorEdges = GetPathCorridorEdges();
	const uint32 NumCorridorEdges = CorridorEdges.Num();

	// if we have a valid corridor, but the index is out of bounds, we could
	// be checking just the last point, but that would be inconsistent with 
	// FNavMeshPath::DoesPathIntersectBoxImplementation implementation
	// so in this case we just say "Nope, doesn't intersect"
	if (NumCorridorEdges <= 0 || StartingIndex > NumCorridorEdges)
	{
		return false;
	}

	// note that it's a bit simplified. It works
	FVector Start = StartLocation;
	if (CorridorEdges.IsValidIndex(StartingIndex))
	{
		// make sure that Start is initialized correctly when testing from the middle of path (StartingIndex > 0)
		if (CorridorEdges.IsValidIndex(StartingIndex - 1))
		{
			const FNavigationPortalEdge& Edge = CorridorEdges[StartingIndex - 1];
			Start = Edge.Right + (Edge.Left - Edge.Right) / 2 + (AgentExtent ? FVector(0.f, 0.f, AgentExtent->Z) : FVector::ZeroVector);
		}

		for (uint32 PortalIndex = StartingIndex; PortalIndex < NumCorridorEdges; ++PortalIndex)
		{
			const FNavigationPortalEdge& Edge = CorridorEdges[PortalIndex];
			const FVector End = Edge.Right + (Edge.Left - Edge.Right) / 2 + (AgentExtent ? FVector(0.f, 0.f, AgentExtent->Z) : FVector::ZeroVector);
			
			if (CheckIntersectBetweenPoints(Box, AgentExtent, Start, End))
			{
				bIntersects = true;
				if (IntersectingSegmentIndex != NULL)
				{
					*IntersectingSegmentIndex = PortalIndex;
				}
				break;
			}

			Start = End;
		}

		// test the last portal->path end line. 
		if (bIntersects == false)
		{
			ensure(PathPoints.Num() >= 2);
			const FVector End = PathPoints.Last().Location + (AgentExtent ? FVector(0.f, 0.f, AgentExtent->Z) : FVector::ZeroVector);

			if (CheckIntersectBetweenPoints(Box, AgentExtent, Start, End))
			{
				bIntersects = true;
				if (IntersectingSegmentIndex != NULL)
				{
					*IntersectingSegmentIndex = NumCorridorEdges;
				}
			}
		}
	}
	else if (NumCorridorEdges > 0 && StartingIndex == NumCorridorEdges) //at last polygon, just after last edge so direct line check 
	{
		const FVector End = PathPoints.Last().Location + (AgentExtent ? FVector(0.f, 0.f, AgentExtent->Z) : FVector::ZeroVector);
			
		if (CheckIntersectBetweenPoints(Box, AgentExtent, Start, End))
		{
			bIntersects = true;
			if (IntersectingSegmentIndex != NULL)
			{
				*IntersectingSegmentIndex = CorridorEdges.Num();
			}
		}
	}
	
	// just check if path's end is inside the tested box
	if (bIntersects == false && Box.IsInside(PathPoints.Last().Location))
	{
		bIntersects = true;
		if (IntersectingSegmentIndex != NULL)
		{
			*IntersectingSegmentIndex = CorridorEdges.Num();
		}
	}

	return bIntersects;
}

bool FNavMeshPath::DoesIntersectBox(const FBox& Box, uint32 StartingIndex, int32* IntersectingSegmentIndex, FVector* AgentExtent) const
{
	if (IsStringPulled())
	{
		return Super::DoesIntersectBox(Box, StartingIndex, IntersectingSegmentIndex);
	}

	bool bParametersValid = true;
	FVector StartLocation = PathPoints[0].Location;

	const TArray<FNavigationPortalEdge>& CorridorEdges = GetPathCorridorEdges();
	if (StartingIndex < uint32(CorridorEdges.Num()))
	{
		StartLocation = CorridorEdges[StartingIndex].Right + (CorridorEdges[StartingIndex].Left - CorridorEdges[StartingIndex].Right) / 2;
		++StartingIndex;
	}
	else if (StartingIndex > uint32(CorridorEdges.Num()))
	{
		bParametersValid = false;
	}
	// else will be handled by DoesPathIntersectBoxImplementation

	return bParametersValid && DoesPathIntersectBoxImplementation(Box, StartLocation, StartingIndex, IntersectingSegmentIndex, AgentExtent);
}

bool FNavMeshPath::DoesIntersectBox(const FBox& Box, const FVector& AgentLocation, uint32 StartingIndex, int32* IntersectingSegmentIndex, FVector* AgentExtent) const
{
	if (IsStringPulled())
	{
		return Super::DoesIntersectBox(Box, AgentLocation, StartingIndex, IntersectingSegmentIndex, AgentExtent);
	}

	return DoesPathIntersectBoxImplementation(Box, AgentLocation, StartingIndex, IntersectingSegmentIndex, AgentExtent);
}

bool FNavMeshPath::GetNodeFlags(int32 NodeIdx, FNavMeshNodeFlags& Flags) const
{
	bool bResult = false;

	if (IsStringPulled())
	{
		if (PathPoints.IsValidIndex(NodeIdx))
		{
			Flags = FNavMeshNodeFlags(PathPoints[NodeIdx].Flags);
			bResult = true;
		}
	}
	else
	{
		if (PathCorridor.IsValidIndex(NodeIdx))
		{
#if WITH_RECAST
			const ARecastNavMesh* MyOwner = Cast<ARecastNavMesh>(GetNavigationDataUsed());
			if (MyOwner)
			{
				MyOwner->GetPolyFlags(PathCorridor[NodeIdx], Flags);
				bResult = true;
			}
#endif	// WITH_RECAST
		}
	}

	return bResult;
}

FVector FNavMeshPath::GetSegmentDirection(uint32 SegmentEndIndex) const
{
	if (IsStringPulled())
	{
		return Super::GetSegmentDirection(SegmentEndIndex);
	}
	
	FVector Result = FNavigationSystem::InvalidLocation;
	const TArray<FNavigationPortalEdge>& Corridor = GetPathCorridorEdges();

	if (Corridor.Num() > 0 && PathPoints.Num() > 1)
	{
		if (Corridor.IsValidIndex(SegmentEndIndex))
		{
			if (SegmentEndIndex > 0)
			{
				Result = (Corridor[SegmentEndIndex].GetMiddlePoint() - Corridor[SegmentEndIndex - 1].GetMiddlePoint()).GetSafeNormal();
			}
			else
			{
				Result = (Corridor[0].GetMiddlePoint() - GetPathPoints()[0].Location).GetSafeNormal();
			}
		}
		else if (SegmentEndIndex >= uint32(Corridor.Num()))
		{
			// in this special case return direction of last segment
			Result = (Corridor[Corridor.Num() - 1].GetMiddlePoint() - GetPathPoints()[0].Location).GetSafeNormal();
		}
	}

	return Result;
}

void FNavMeshPath::Invert()
{
	Algo::Reverse(PathPoints);
	Algo::Reverse(PathCorridor);
	Algo::Reverse(PathCorridorCost);
	if (bCorridorEdgesGenerated)
	{
		Algo::Reverse(PathCorridorEdges);
	}
}

#if ENABLE_VISUAL_LOG

void FNavMeshPath::DescribeSelfToVisLog(FVisualLogEntry* Snapshot) const
{
	if (Snapshot == nullptr)
	{
		return;
	}

	if (IsStringPulled())
	{
		// draw path points only for string pulled paths
		Super::DescribeSelfToVisLog(Snapshot);
	}

	// draw corridor
#if WITH_RECAST
	FVisualLogShapeElement CorridorPoly(EVisualLoggerShapeElement::Polygon);
	CorridorPoly.SetColor(FColorList::Cyan.WithAlpha(100));
	CorridorPoly.Category = LogNavigation.GetCategoryName();
	CorridorPoly.Verbosity = ELogVerbosity::Verbose;
	CorridorPoly.Points.Reserve(PathCorridor.Num() * 6);

	const FVector CorridorOffset = NavigationDebugDrawing::PathOffset * 1.25f;
	int32 NumAreaMark = 1;

	ARecastNavMesh* NavMesh = Cast<ARecastNavMesh>(GetNavigationDataUsed());
	if (NavMesh == nullptr)
	{
		return;
	}
	NavMesh->BeginBatchQuery();

	TArray<FVector> Verts;
	for (int32 Idx = 0; Idx < PathCorridor.Num(); Idx++)
	{
		const int32 AreaID = IntCastChecked<int32>(NavMesh->GetPolyAreaID(PathCorridor[Idx]));
		const UClass* AreaClass = NavMesh->GetAreaClass(AreaID);
		
		Verts.Reset();
		const bool bPolyResult = NavMesh->GetPolyVerts(PathCorridor[Idx], Verts);
		if (!bPolyResult || Verts.Num() == 0)
		{
			// probably invalidated polygon, etc. (time sensitive and rare to reproduce issue)
			continue;
		}

		const UNavArea* DefArea = AreaClass ? ((UClass*)AreaClass)->GetDefaultObject<UNavArea>() : NULL;
		const TSubclassOf<UNavAreaBase> DefaultWalkableArea = FNavigationSystem::GetDefaultWalkableArea();
		const FColor PolygonColor = AreaClass != DefaultWalkableArea ? (DefArea ? DefArea->DrawColor : NavMesh->GetConfig().Color) : FColorList::Cyan;

		CorridorPoly.SetColor(PolygonColor.WithAlpha(100));
		CorridorPoly.Points.Reset();
		CorridorPoly.Points.Append(Verts);
		Snapshot->ElementsToDraw.Add(CorridorPoly);

		if (AreaClass && AreaClass != DefaultWalkableArea)
		{
			FVector CenterPt = FVector::ZeroVector;
			for (int32 VIdx = 0; VIdx < Verts.Num(); VIdx++)
			{
				CenterPt += Verts[VIdx];
			}
			CenterPt /= Verts.Num();

			FVisualLogShapeElement AreaMarkElem(EVisualLoggerShapeElement::Segment);
			AreaMarkElem.SetColor(FColorList::Orange);
			AreaMarkElem.Category = LogNavigation.GetCategoryName();
			AreaMarkElem.Verbosity = ELogVerbosity::Verbose;
			AreaMarkElem.Thickness = 2;
			AreaMarkElem.Description = AreaClass->GetName();

			AreaMarkElem.Points.Add(CenterPt + CorridorOffset);
			AreaMarkElem.Points.Add(CenterPt + CorridorOffset + FVector(0,0,100 + static_cast<double>(NumAreaMark) * 50));
			Snapshot->ElementsToDraw.Add(AreaMarkElem);

			NumAreaMark = (NumAreaMark + 1) % 5;
		}
	}

	NavMesh->FinishBatchQuery();
	//Snapshot->ElementsToDraw.Add(CorridorElem);
#endif
}

FString FNavMeshPath::GetDescription() const
{
	return FString::Printf(TEXT("NotifyPathUpdate points:%d corridor length %d valid:%s")
		, PathPoints.Num()
		, PathCorridor.Num()
		, IsValid() ? TEXT("yes") : TEXT("no"));
}

#endif // ENABLE_VISUAL_LOG
