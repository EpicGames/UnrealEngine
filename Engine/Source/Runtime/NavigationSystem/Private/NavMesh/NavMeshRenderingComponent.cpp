// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	NavMeshRenderingComponent.cpp: A component that renders a navmesh.
 =============================================================================*/

#include "NavMesh/NavMeshRenderingComponent.h"
#include "Materials/Material.h"
#include "NavigationSystem.h"
#include "Engine/GameViewportClient.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Materials/MaterialRenderProxy.h"
#include "NavigationOctree.h"
#include "NavMesh/RecastHelpers.h"
#include "NavMesh/RecastNavMesh.h"
#include "NavMesh/RecastNavMeshGenerator.h"
#include "Debug/DebugDrawService.h"
#include "SceneInterface.h"
#include "PrimitiveDrawingUtils.h"
#include "TimerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavMeshRenderingComponent)

#if RECAST_INTERNAL_DEBUG_DATA
#include "NavMesh/RecastInternalDebugData.h"
#endif

#if WITH_EDITOR
#include "Editor.h"
#include "EditorViewportClient.h"
#endif

static constexpr FColor NavMeshRenderColor_Recast_TriangleEdges(255, 255, 255);
static constexpr FColor NavMeshRenderColor_Recast_TileEdges(16, 16, 16, 32);
static constexpr FColor NavMeshRenderColor_RecastMesh(140, 255, 0, 164);
static constexpr FColor NavMeshRenderColor_TileBounds(255, 255, 64, 255);
static constexpr FColor NavMeshRenderColor_PathCollidingGeom(255, 255, 255, 40);
static constexpr FColor NavMeshRenderColor_RecastTileBeingRebuilt(255, 0, 0, 64);
static constexpr FColor NavMeshRenderColor_OffMeshConnectionInvalid(64, 64, 64);
static const FColor NavMeshRenderColor_PolyForbidden(FColorList::Black);

static constexpr float DefaultEdges_LineThickness = 0.0f;
static constexpr float TileResolution_LineThickness = 5.f;
static constexpr float PolyEdges_LineThickness = 1.1f;
static constexpr float NavMeshEdges_LineThickness = 4.f;
static constexpr float LinkLines_LineThickness = 2.0f;
static constexpr float GeneratedLinkLines_LineThickness = 4.0f;
static constexpr float ClusterLinkLines_LineThickness = 2.0f;
static constexpr float LinkLines_UnidirectionalArcVerticalSeparationDistance = 3.0f;

namespace FNavMeshRenderingHelpers
{
	void DrawDebugBox(FPrimitiveDrawInterface* PDI, FVector const& Center, FVector const& Box, FColor const& Color)
	{
		// no debug line drawing on dedicated server
		if (PDI != nullptr)
		{
			PDI->DrawLine(Center + FVector(Box.X, Box.Y, Box.Z), Center + FVector(Box.X, -Box.Y, Box.Z), Color, SDPG_World);
			PDI->DrawLine(Center + FVector(Box.X, -Box.Y, Box.Z), Center + FVector(-Box.X, -Box.Y, Box.Z), Color, SDPG_World);
			PDI->DrawLine(Center + FVector(-Box.X, -Box.Y, Box.Z), Center + FVector(-Box.X, Box.Y, Box.Z), Color, SDPG_World);
			PDI->DrawLine(Center + FVector(-Box.X, Box.Y, Box.Z), Center + FVector(Box.X, Box.Y, Box.Z), Color, SDPG_World);

			PDI->DrawLine(Center + FVector(Box.X, Box.Y, -Box.Z), Center + FVector(Box.X, -Box.Y, -Box.Z), Color, SDPG_World);
			PDI->DrawLine(Center + FVector(Box.X, -Box.Y, -Box.Z), Center + FVector(-Box.X, -Box.Y, -Box.Z), Color, SDPG_World);
			PDI->DrawLine(Center + FVector(-Box.X, -Box.Y, -Box.Z), Center + FVector(-Box.X, Box.Y, -Box.Z), Color, SDPG_World);
			PDI->DrawLine(Center + FVector(-Box.X, Box.Y, -Box.Z), Center + FVector(Box.X, Box.Y, -Box.Z), Color, SDPG_World);

			PDI->DrawLine(Center + FVector(Box.X, Box.Y, Box.Z), Center + FVector(Box.X, Box.Y, -Box.Z), Color, SDPG_World);
			PDI->DrawLine(Center + FVector(Box.X, -Box.Y, Box.Z), Center + FVector(Box.X, -Box.Y, -Box.Z), Color, SDPG_World);
			PDI->DrawLine(Center + FVector(-Box.X, -Box.Y, Box.Z), Center + FVector(-Box.X, -Box.Y, -Box.Z), Color, SDPG_World);
			PDI->DrawLine(Center + FVector(-Box.X, Box.Y, Box.Z), Center + FVector(-Box.X, Box.Y, -Box.Z), Color, SDPG_World);
		}
	}

	bool LineInView(const FVector& Start, const FVector& End, const FSceneView* View)
	{
		if (FVector::DistSquaredXY(Start, View->ViewMatrices.GetViewOrigin()) > ARecastNavMesh::GetDrawDistanceSq() ||
			FVector::DistSquaredXY(End,   View->ViewMatrices.GetViewOrigin()) > ARecastNavMesh::GetDrawDistanceSq())
		{
			return false;
		}

		for (int32 PlaneIdx = 0; PlaneIdx < View->ViewFrustum.Planes.Num(); ++PlaneIdx)
		{
			const FPlane& CurPlane = View->ViewFrustum.Planes[PlaneIdx];
			if (CurPlane.PlaneDot(Start) > 0.f && CurPlane.PlaneDot(End) > 0.f)
			{
				return false;
			}
		}

		return true;
	}

	bool PointInView(const FVector& Position, const FSceneView* View)
	{
		if (FVector::DistSquaredXY(Position, View->ViewMatrices.GetViewOrigin()) > ARecastNavMesh::GetDrawDistanceSq())
		{
			return false;
		}

		for (int32 PlaneIdx = 0; PlaneIdx < View->ViewFrustum.Planes.Num(); ++PlaneIdx)
		{
			const FPlane& CurPlane = View->ViewFrustum.Planes[PlaneIdx];
			if (CurPlane.PlaneDot(Position) > 0.f)
			{
				return false;
			}
		}

		return true;
	}

	bool LineInCorrectDistance(const FVector& Start, const FVector& End, const FSceneView* View, FVector::FReal CorrectDistance = -1.)
	{
		const FVector::FReal MaxDistanceSq = (CorrectDistance > 0.) ? FMath::Square(CorrectDistance) : ARecastNavMesh::GetDrawDistanceSq();
		return	FVector::DistSquaredXY(Start, View->ViewMatrices.GetViewOrigin()) < MaxDistanceSq &&
				FVector::DistSquaredXY(End, View->ViewMatrices.GetViewOrigin()) < MaxDistanceSq;
	}

	FVector EvalArc(const FVector& Org, const FVector& Dir, const FVector::FReal h, const FVector::FReal u)
	{
		FVector Pt = Org + Dir * u;
		Pt.Z += h * (1 - (u * 2 - 1)*(u * 2 - 1));

		return Pt;
	}

	void CacheArc(TArray<FDebugRenderSceneProxy::FDebugLine>& DebugLines, const FVector& Start, const FVector& End, const FVector::FReal Height, const uint32 Segments, const FLinearColor& Color, float LineThickness = 0, float VerticalOffset = 0)
	{
		if (Segments == 0)
		{
			return;
		}

		const FVector::FReal ArcPtsScale = 1. / (FVector::FReal)Segments;
		const FVector Dir = End - Start;
		const FVector::FReal Length = Dir.Size();
		const FVector::FReal VerticalOffsetSign = FMath::Sign(Dir.Z);

		FVector Prev = Start;
		for (uint32 i = 1; i <= Segments; ++i)
		{
			const FVector::FReal u = (FVector::FReal)i * ArcPtsScale;
			FVector Pt = EvalArc(Start, Dir, Length*Height, u);
			Pt.Z += (i < Segments) ? VerticalOffsetSign * VerticalOffset : 0.0;

			DebugLines.Add(FDebugRenderSceneProxy::FDebugLine(Prev, Pt, Color.ToFColor(true), LineThickness));
			Prev = Pt;
		}
	}

	void CacheArrowHead(TArray<FDebugRenderSceneProxy::FDebugLine>& DebugLines, const FVector& Tip, const FVector& Origin, const FVector::FReal Size, const FLinearColor& Color, float LineThickness = 0)
	{
		const FVector Az(0.0, 1.0, 0.0);
		const FVector Ay = (Origin - Tip).GetSafeNormal();
		const FVector Ax = FVector::CrossProduct(Az, Ay);

		DebugLines.Add(FDebugRenderSceneProxy::FDebugLine(Tip, FVector(Tip.X + Ay.X*Size + Ax.X*Size / 3, Tip.Y + Ay.Y*Size + Ax.Y*Size / 3, Tip.Z + Ay.Z*Size + Ax.Z*Size / 3), Color.ToFColor(true), LineThickness));
		DebugLines.Add(FDebugRenderSceneProxy::FDebugLine(Tip, FVector(Tip.X + Ay.X*Size - Ax.X*Size / 3, Tip.Y + Ay.Y*Size - Ax.Y*Size / 3, Tip.Z + Ay.Z*Size - Ax.Z*Size / 3), Color.ToFColor(true), LineThickness));
	}

	void CacheLink(TArray<FDebugRenderSceneProxy::FDebugLine>& DebugLines, const FVector V0, const FVector V1, const FColor LinkColor, const uint8 LinkDirection, const bool bIsGenerated)
	{
		const float LineThickness = bIsGenerated ? GeneratedLinkLines_LineThickness : LinkLines_LineThickness;
		const float ArcVerticalOffset = LinkDirection ? 0.0f : 0.5f * (LinkLines_UnidirectionalArcVerticalSeparationDistance + LineThickness);

		FNavMeshRenderingHelpers::CacheArc(DebugLines, V0, V1, 0.4f, 4, LinkColor, LineThickness, ArcVerticalOffset);

		const FVector VOffset(0, 0, FVector::Dist(V0, V1) * 1.333f);
		FNavMeshRenderingHelpers::CacheArrowHead(DebugLines, V1, V0 + VOffset, 30.f, LinkColor, LineThickness);
		if (LinkDirection)
		{
			FNavMeshRenderingHelpers::CacheArrowHead(DebugLines, V0, V1 + VOffset, 30.f, LinkColor, LineThickness);
		}
	}

	void DrawWireCylinder(TArray<FDebugRenderSceneProxy::FDebugLine>& DebugLines, const FVector& Base, const FVector& X, const FVector& Y, const FVector& Z, FColor Color, FVector::FReal Radius, FVector::FReal HalfHeight, int32 NumSides, uint8 DepthPriority, float LineThickness = 0)
	{
		const FVector::FReal AngleDelta = 2.0 * PI / static_cast<FVector::FReal>(NumSides);
		FVector	LastVertex = Base + X * Radius;

		for (int32 SideIndex = 0; SideIndex < NumSides; SideIndex++)
		{
			const FVector Vertex = Base +
				(X * FMath::Cos(AngleDelta * static_cast<FVector::FReal>(SideIndex + 1)) + Y * FMath::Sin(AngleDelta * static_cast<FVector::FReal>(SideIndex + 1))) * Radius;

			DebugLines.Add(FDebugRenderSceneProxy::FDebugLine(LastVertex - Z * HalfHeight, Vertex - Z * HalfHeight, Color));
			DebugLines.Add(FDebugRenderSceneProxy::FDebugLine(LastVertex + Z * HalfHeight, Vertex + Z * HalfHeight, Color));
			DebugLines.Add(FDebugRenderSceneProxy::FDebugLine(LastVertex - Z * HalfHeight, LastVertex + Z * HalfHeight, Color));

			LastVertex = Vertex;
		}
	}

	inline uint8 GetBit(int32 v, uint8 bit)
	{
		return static_cast<uint8>((v & (1 << bit)) >> bit);
	}

	FColor GetClusterColor(int32 Idx)
	{
		const uint8 r = 1 + GetBit(Idx, 1) + GetBit(Idx, 3) * 2;
		const uint8 g = 1 + GetBit(Idx, 2) + GetBit(Idx, 4) * 2;
		const uint8 b = 1 + GetBit(Idx, 0) + GetBit(Idx, 5) * 2;
		return FColor(r * 63, g * 63, b * 63, 164);
	}

	FColor DarkenColor(const FColor& Base)
	{
		const uint32 Col = Base.DWColor();
		return FColor(((Col >> 1) & 0x007f7f7f) | (Col & 0xff000000));
	}

	FColor SemiDarkenColor(const FColor& Base)
	{
		const uint32 Col = Base.DWColor();
		// 1/2 can be too dark(DarkenColor) - use 3/4 here.
		const uint32 QuarterCol = ((Col >> 2) & 0x003f3f3f);
		const uint32 HalfCol = ((Col >> 1) & 0x007f7f7f);
		const uint32 ThreeQuarterCol = QuarterCol + HalfCol;
		return FColor(ThreeQuarterCol | (Col & 0xff000000));
	}

	void AddVertex(FNavMeshSceneProxyData::FDebugMeshData& MeshData, const FVector& Pos, const FColor Color)
	{
		FDynamicMeshVertex* Vertex = new(MeshData.Vertices) FDynamicMeshVertex;
		Vertex->Position = (FVector3f)Pos;
		Vertex->TextureCoordinate[0] = FVector2f::ZeroVector;
		Vertex->TangentX = FVector(1.0f, 0.0f, 0.0f);
		Vertex->TangentZ = FVector(0.0f, 1.0f, 0.0f);
		// store the sign of the determinant in TangentZ.W (-1=-128,+1=127)
		Vertex->TangentZ.Vector.W = 127;
		Vertex->Color = Color;
	}

	void AddTriangleIndices(FNavMeshSceneProxyData::FDebugMeshData& MeshData, int32 V0, int32 V1, int32 V2)
	{
		MeshData.Indices.Add(V0);
		MeshData.Indices.Add(V1);
		MeshData.Indices.Add(V2);
	}

	void AddCluster(TArray<FNavMeshSceneProxyData::FDebugMeshData>& MeshBuilders, const TArray<int32>& MeshIndices, const TArray<FVector>& MeshVerts, const FColor Color, const FVector DrawOffset)
	{
		if (MeshIndices.Num() == 0)
		{
			return;
		}

		FNavMeshSceneProxyData::FDebugMeshData DebugMeshData;
		for (int32 VertIdx = 0; VertIdx < MeshVerts.Num(); ++VertIdx)
		{
			FNavMeshRenderingHelpers::AddVertex(DebugMeshData, MeshVerts[VertIdx] + DrawOffset, Color);
		}
		for (int32 TriIdx = 0; TriIdx < MeshIndices.Num(); TriIdx += 3)
		{
			FNavMeshRenderingHelpers::AddTriangleIndices(DebugMeshData, MeshIndices[TriIdx], MeshIndices[TriIdx + 1], MeshIndices[TriIdx + 2]);
		}

		DebugMeshData.ClusterColor = Color;
		MeshBuilders.Add(DebugMeshData);
	}

	void AddRecastGeometry(TArray<FVector>& OutVertexBuffer, TArray<uint32>& OutIndexBuffer, const FVector::FReal* Coords, int32 NumVerts, const int32* Faces, int32 NumFaces, const FTransform& Transform = FTransform::Identity)
	{
		const int32 VertIndexBase = OutVertexBuffer.Num();
		for (int32 VertIdx = 0; VertIdx < NumVerts * 3; VertIdx += 3)
		{
			OutVertexBuffer.Add(Transform.TransformPosition(Recast2UnrealPoint(&Coords[VertIdx])));
		}

		const int32 FirstNewFaceVertexIndex = OutIndexBuffer.Num();
		const uint32 NumIndices = NumFaces * 3;
		OutIndexBuffer.AddUninitialized(NumIndices);
		for (uint32 Index = 0; Index < NumIndices; ++Index)
		{
			OutIndexBuffer[FirstNewFaceVertexIndex + Index] = VertIndexBase + Faces[Index];
		}
	}

	inline bool HasFlag(int32 Flags, ENavMeshDetailFlags TestFlag)
	{
		return (Flags & (1 << static_cast<int32>(TestFlag))) != 0;
	}

#if WITH_RECAST
	int32 GetDetailFlags(const ARecastNavMesh* NavMesh)
	{
		return (NavMesh == nullptr) ? 0 : 0 |
			(NavMesh->bDrawTriangleEdges ? (1 << static_cast<int32>(ENavMeshDetailFlags::TriangleEdges)) : 0) |
			(NavMesh->bDrawPolyEdges ? (1 << static_cast<int32>(ENavMeshDetailFlags::PolyEdges)) : 0) |
			(NavMesh->bDrawFilledPolys ? (1 << static_cast<int32>(ENavMeshDetailFlags::FilledPolys)) : 0) |
			(NavMesh->bDrawNavMeshEdges ? (1 << static_cast<int32>(ENavMeshDetailFlags::BoundaryEdges)) : 0) |
			(NavMesh->bDrawTileBounds ? (1 << static_cast<int32>(ENavMeshDetailFlags::TileBounds)) : 0) |
			(NavMesh->bDrawTileResolutions ? (1 << static_cast<int32>(ENavMeshDetailFlags::TileResolutions)) : 0) |
			(NavMesh->bDrawPathCollidingGeometry ? (1 << static_cast<int32>(ENavMeshDetailFlags::PathCollidingGeometry)) : 0) |
			(NavMesh->bDrawTileLabels ? (1 << static_cast<int32>(ENavMeshDetailFlags::TileLabels)) : 0) |
			(NavMesh->bDrawPolygonLabels ? (1 << static_cast<int32>(ENavMeshDetailFlags::PolygonLabels)) : 0) |
			(NavMesh->bDrawDefaultPolygonCost ? (1 << static_cast<int32>(ENavMeshDetailFlags::PolygonCost)) : 0) |
			(NavMesh->bDrawPolygonFlags ? (1 << static_cast<int32>(ENavMeshDetailFlags::PolygonFlags)) : 0) |
			(NavMesh->bDrawPolygonAreaIDs ? (1 << static_cast<int32>(ENavMeshDetailFlags::PolygonAreaIDs)) : 0) |
			(NavMesh->bDrawLabelsOnPathNodes ? (1 << static_cast<int32>(ENavMeshDetailFlags::PathLabels)) : 0) |
			(NavMesh->bDrawNavLinks ? (1 << static_cast<int32>(ENavMeshDetailFlags::NavLinks)) : 0) |
			(NavMesh->bDrawFailedNavLinks ? (1 << static_cast<int32>(ENavMeshDetailFlags::FailedNavLinks)) : 0) |
			(NavMesh->bDrawClusters ? (1 << static_cast<int32>(ENavMeshDetailFlags::Clusters)) : 0) |
			(NavMesh->bDrawOctree ? (1 << static_cast<int32>(ENavMeshDetailFlags::NavOctree)) : 0) |
			(NavMesh->bDrawOctreeDetails ? (1 << static_cast<int32>(ENavMeshDetailFlags::NavOctreeDetails)) : 0) |
			(NavMesh->bDrawMarkedForbiddenPolys ? (1 << static_cast<int32>(ENavMeshDetailFlags::MarkForbiddenPolys)) : 0) |
			(NavMesh->bDrawTileBuildTimes ? (1 << static_cast<int32>(ENavMeshDetailFlags::TileBuildTimes)) : 0) |
			(NavMesh->bDrawTileBuildTimesHeatMap ? (1 << static_cast<int32>(ENavMeshDetailFlags::TileBuildTimesHeatMap)) : 0) |
			(NavMesh->bDrawOnScreenInformation ? (1 << static_cast<int32>(ENavMeshDetailFlags::OnScreenInformation)) : 0);
	}
#endif // WITH_RECAST

}

//////////////////////////////////////////////////////////////////////////
// FNavMeshSceneProxyData

void FNavMeshSceneProxyData::Reset()
{
	MeshBuilders.Reset();
	ThickLineItems.Reset();
	TileEdgeLines.Reset();
	NavMeshEdgeLines.Reset();
	NavLinkLines.Reset();
	ClusterLinkLines.Reset();
	AuxLines.Reset();
	AuxPoints.Reset();
	AuxBoxes.Reset();
	DebugLabels.Reset();
	OctreeBounds.Reset();
	Bounds.Init();

	bNeedsNewData = true;
	bDataGathered = false;
	NavDetailFlags = 0;
}

void FNavMeshSceneProxyData::Serialize(FArchive& Ar)
{
	int32 NumMeshBuilders = MeshBuilders.Num();
	Ar << NumMeshBuilders;
	if (Ar.IsLoading())
	{
		MeshBuilders.SetNum(NumMeshBuilders);
	}

	for (int32 Idx = 0; Idx < NumMeshBuilders; Idx++)
	{
		FDebugMeshData& MeshBuilder = MeshBuilders[Idx];

		int32 NumVerts = MeshBuilder.Vertices.Num();
		Ar << NumVerts;
		if (Ar.IsLoading())
		{
			MeshBuilder.Vertices.SetNum(NumVerts);
		}

		for (int32 VertIdx = 0; VertIdx < NumVerts; VertIdx++)
		{
			FVector3f SerializedVert = MeshBuilder.Vertices[VertIdx].Position;
			Ar << SerializedVert;

			if (Ar.IsLoading())
			{
				MeshBuilder.Vertices[VertIdx] = FDynamicMeshVertex(SerializedVert);
			}
		}

		Ar << MeshBuilder.Indices;
		Ar << MeshBuilder.ClusterColor;
	}

	TArray<FDebugRenderSceneProxy::FDebugLine>* LineArraysToSerialize[] = { &ThickLineItems, &TileEdgeLines, &NavMeshEdgeLines, &NavLinkLines, &ClusterLinkLines, &AuxLines };
	for (int32 ArrIdx = 0; ArrIdx < UE_ARRAY_COUNT(LineArraysToSerialize); ArrIdx++)
	{
		int32 NumItems = LineArraysToSerialize[ArrIdx]->Num();
		Ar << NumItems;
		if (Ar.IsLoading())
		{
			LineArraysToSerialize[ArrIdx]->Reset(NumItems);
			LineArraysToSerialize[ArrIdx]->AddUninitialized(NumItems);
		}

		for (int32 Idx = 0; Idx < NumItems; Idx++)
		{
			Ar << (*LineArraysToSerialize[ArrIdx])[Idx].Thickness;
			Ar << (*LineArraysToSerialize[ArrIdx])[Idx].Start;
			Ar << (*LineArraysToSerialize[ArrIdx])[Idx].End;
			Ar << (*LineArraysToSerialize[ArrIdx])[Idx].Color;
		}
	}

	int32 NumPoints = AuxPoints.Num();
	Ar << NumPoints;
	if (Ar.IsLoading())
	{
		FDebugPoint TmpPoint(FVector::ZeroVector, FColor::Black, 0.0f);
		AuxPoints.Reserve(NumPoints);
		for (int32 Idx = 0; Idx < NumPoints; Idx++)
		{
			Ar << TmpPoint.Position;
			Ar << TmpPoint.Color;
			Ar << TmpPoint.Size;

			AuxPoints.Add(TmpPoint);
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < NumPoints; Idx++)
		{
			Ar << AuxPoints[Idx].Position;
			Ar << AuxPoints[Idx].Color;
			Ar << AuxPoints[Idx].Size;
		}
	}

	int32 NumBoxes = AuxBoxes.Num();
	Ar << NumBoxes;
	if (Ar.IsLoading())
	{
		FDebugRenderSceneProxy::FDebugBox TmpBox = FDebugRenderSceneProxy::FDebugBox(FBox(), FColor());
		AuxBoxes.Reserve(NumBoxes);
		for (int32 Idx = 0; Idx < NumBoxes; Idx++)
		{	
			Ar << TmpBox.Box;
			Ar << TmpBox.Color;
			Ar << TmpBox.Transform;

			AuxBoxes.Add(TmpBox);
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < NumBoxes; Idx++)
		{
			Ar << AuxBoxes[Idx].Box;
			Ar << AuxBoxes[Idx].Color;
			Ar << AuxBoxes[Idx].Transform;
		}
	}

	int32 NumLabels = DebugLabels.Num();
	Ar << NumLabels;
	if (Ar.IsLoading())
	{
		DebugLabels.SetNum(NumLabels);
	}

	for (int32 Idx = 0; Idx < NumLabels; Idx++)
	{
		Ar << DebugLabels[Idx].Location;
		Ar << DebugLabels[Idx].Text;
	}

	int32 NumBounds = OctreeBounds.Num();
	Ar << NumBounds;
	if (Ar.IsLoading())
	{
		OctreeBounds.SetNum(NumBounds);
	}

	for (int32 Idx = 0; Idx < NumBounds; Idx++)
	{
		Ar << OctreeBounds[Idx].Center;
		Ar << OctreeBounds[Idx].Extent;
	}

	Ar << Bounds;
	Ar << NavMeshDrawOffset;
	Ar << NavDetailFlags;

	int32 BitFlags = ((bDataGathered ? 1 : 0) << 0) | ((bNeedsNewData ? 1 : 0) << 1);
	Ar << BitFlags;
	bDataGathered = (BitFlags & (1 << 0)) != 0;
	bNeedsNewData = (BitFlags & (1 << 1)) != 0;
}

uint32 FNavMeshSceneProxyData::GetAllocatedSize() const
{
	return IntCastChecked<uint32>(
		MeshBuilders.GetAllocatedSize() +
		ThickLineItems.GetAllocatedSize() +
		TileEdgeLines.GetAllocatedSize() +
		NavMeshEdgeLines.GetAllocatedSize() +
		NavLinkLines.GetAllocatedSize() +
		ClusterLinkLines.GetAllocatedSize() +
		AuxLines.GetAllocatedSize() +
		AuxPoints.GetAllocatedSize() +
		AuxBoxes.GetAllocatedSize() +
		DebugLabels.GetAllocatedSize() +
		OctreeBounds.GetAllocatedSize());
}

#if WITH_RECAST

// Deprecated
void FNavMeshSceneProxyData::GatherData(const ARecastNavMesh* NavMesh, int32 InNavDetailFlags, const TArray<int32>& TileSet)
{
}

void FNavMeshSceneProxyData::GatherData(const ARecastNavMesh* NavMesh, int32 InNavDetailFlags, const TArray<FNavTileRef>& TileSet)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_GatherDebugDrawingGeometry);
	Reset();

	NavDetailFlags = InNavDetailFlags;
	if (NavMesh && NavDetailFlags)
	{
		bNeedsNewData = false;
		bDataGathered = true;

		NavMeshDrawOffset.Z = NavMesh->DrawOffset;

		FRecastDebugGeometry NavMeshGeometry;
		NavMeshGeometry.bGatherPolyEdges = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::PolyEdges);
		NavMeshGeometry.bGatherNavMeshEdges = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::BoundaryEdges);
		NavMeshGeometry.bMarkForbiddenPolys = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::MarkForbiddenPolys);
		NavMeshGeometry.bGatherTileBuildTimesHeatMap = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::TileBuildTimesHeatMap);
		const bool bGatherTileBuildTimes = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::TileBuildTimes);
		const bool bDisplayOnScreenInformation = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::OnScreenInformation);

#if RECAST_INTERNAL_DEBUG_DATA
		// Colors for tile build times heat map
		auto LerpColor = [](FColor A, FColor B, float T) -> FColor
		{
			return FColor(
				static_cast<uint8>(FMath::RoundToInt(float(A.R) * (1.f - T) + float(B.R) * T)),
				static_cast<uint8>(FMath::RoundToInt(float(A.G) * (1.f - T) + float(B.G) * T)),
				static_cast<uint8>(FMath::RoundToInt(float(A.B) * (1.f - T) + float(B.B) * T)),
				static_cast<uint8>(FMath::RoundToInt(float(A.A) * (1.f - T) + float(B.A) * T)));
		};

		TArray<FColor> TileBuildTimeColors;
		const TMap<FIntPoint, FRecastInternalDebugData>* DebugDataMap = NavMesh->GetDebugDataMap();

		double TotalTileBuildTime = 0.;
		double AverageBuildTime = 0.;
		double AverageCompLayersBuildTime = 0.;
		double AverageNavLayersBuildTime = 0.;
		double AverageLinkBuildTime = 0.;
		double TotalBuildLinkTime = 0.;
		if (bGatherTileBuildTimes || NavMeshGeometry.bGatherTileBuildTimesHeatMap)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_GatherDebugDrawing_MaxTileBuildTime);
			
			// Find min and max tile build time
			NavMeshGeometry.MinTileBuildTime = DBL_MAX;
			NavMeshGeometry.MaxTileBuildTime = 0.;
			if (DebugDataMap)
			{
				for(const TPair<FIntPoint, FRecastInternalDebugData>& Pair : *DebugDataMap)
				{
					NavMeshGeometry.MaxTileBuildTime = FMath::Max(NavMeshGeometry.MaxTileBuildTime, Pair.Value.BuildTime);
					NavMeshGeometry.MinTileBuildTime = FMath::Min(NavMeshGeometry.MinTileBuildTime, Pair.Value.BuildTime);
					
					AverageBuildTime += Pair.Value.BuildTime;
					AverageCompLayersBuildTime += Pair.Value.BuildCompressedLayerTime;
					AverageNavLayersBuildTime += Pair.Value.BuildNavigationDataTime;
					AverageLinkBuildTime += Pair.Value.BuildLinkTime;
				}
			}

			TotalTileBuildTime = AverageBuildTime;
			TotalBuildLinkTime = AverageLinkBuildTime;
			if (DebugDataMap->Num() != 0)
			{
				AverageBuildTime /= DebugDataMap->Num();
				AverageCompLayersBuildTime /= DebugDataMap->Num();
				AverageNavLayersBuildTime /= DebugDataMap->Num();
				AverageLinkBuildTime /= DebugDataMap->Num();
			}
		}

		if(NavMeshGeometry.bGatherTileBuildTimesHeatMap)
		{
			TileBuildTimeColors.AddDefaulted(FRecastDebugGeometry::BuildTimeBucketsCount);
			for (int32 Index = 0; Index < FRecastDebugGeometry::BuildTimeBucketsCount; Index++)
			{
				const float LerpValue = (float)Index / (FRecastDebugGeometry::BuildTimeBucketsCount-1);
				TileBuildTimeColors[Index] = LerpColor(FColor::Blue.WithAlpha(140), FColor::Red.WithAlpha(140), LerpValue);
			}
		}
#endif // RECAST_INTERNAL_DEBUG_DATA

		if (bDisplayOnScreenInformation)
		{
			// On screen information
			QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_GatherDebugDrawing_2DLabels);

			auto GetPartitioningString = [](const ERecastPartitioning::Type Type) -> FString
			{
				return Type == ERecastPartitioning::Monotone ? TEXT("Monotone") :
					Type == ERecastPartitioning::Watershed ? TEXT("Watershed") : Type == ERecastPartitioning::ChunkyMonotone ? TEXT("ChunkyMonotone") : TEXT("Unknown");
			}; 
			
			// Navmesh
			const ERuntimeGenerationType Mode = NavMesh->GetRuntimeGenerationMode();
			const FString GenerationMode = Mode == ERuntimeGenerationType::Static ? TEXT("Static") :
				(Mode == ERuntimeGenerationType::Dynamic ? TEXT("Dynamic") : (Mode == ERuntimeGenerationType::DynamicModifiersOnly ? TEXT("DynamicModifersOnly") : TEXT("Unknown")));
			
			if (NavMesh->NeedsRebuild())
			{
				DebugLabels.Add(FDebugText(FString::Printf(TEXT("%s"), TEXT("*** NAVMESH NEEDS TO BE REBUILT ***"))));	
			}
			DebugLabels.Add(FDebugText(FString::Printf(TEXT("%s (%s%s)"), *NavMesh->GetName(), NavMesh->bIsWorldPartitioned ? TEXT("WP ") : TEXT(""), *GenerationMode)));
			DebugLabels.Add(FDebugText(FString::Printf(TEXT("AgentRadius %0.1f, AgentHeight %0.1f"), NavMesh->AgentRadius, NavMesh->AgentHeight)));
			DebugLabels.Add(FDebugText(FString::Printf(
				TEXT("CellSizes %0.1f/%0.1f/%0.1f, CellHeights %0.1f/%0.1f/%0.1f, AgentMaxStepHeight %0.1f/%0.1f/%0.1f (low/default/high)"),
				NavMesh->GetCellSize(ENavigationDataResolution::Low), NavMesh->GetCellSize(ENavigationDataResolution::Default), NavMesh->GetCellSize(ENavigationDataResolution::High),
				NavMesh->GetCellHeight(ENavigationDataResolution::Low), NavMesh->GetCellHeight(ENavigationDataResolution::Default), NavMesh->GetCellHeight(ENavigationDataResolution::High),
				NavMesh->GetAgentMaxStepHeight(ENavigationDataResolution::Low), NavMesh->GetAgentMaxStepHeight(ENavigationDataResolution::Default), NavMesh->GetAgentMaxStepHeight(ENavigationDataResolution::High)
				)));
			DebugLabels.Add(FDebugText(FString::Printf(TEXT("Region part %s, Layer part %s"), *GetPartitioningString(NavMesh->RegionPartitioning), *GetPartitioningString(NavMesh->LayerPartitioning))));
			DebugLabels.Add(FDebugText(TEXT(""))); // empty line

			if (NavMesh->GetGenerator() && !NavMesh->GetActiveTileSet().IsEmpty())
			{
				DebugLabels.Add(FDebugText(FString::Printf(TEXT("Active tiles: %i"), NavMesh->GetActiveTileSet().Num())));	
			}	
			
			// Navigation system
			if (const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(NavMesh->GetWorld()))
			{
				DebugLabels.Add(FDebugText(FString::Printf(TEXT("NavData count: %i"), NavSys->NavDataSet.Num())));
				DebugLabels.Add(FDebugText(FString::Printf(TEXT("MainNavData: %s"), NavSys->MainNavData ? *NavSys->MainNavData->GetName() : TEXT("none"))));
				DebugLabels.Add(FDebugText(FString::Printf(TEXT("Custom NavLinks count: %i"), NavSys->GetNumCustomLinks())));

#if WITH_NAVMESH_CLUSTER_LINKS
				DebugLabels.Add(FDebugText(FString::Printf(TEXT("Using cluster links"))));
#endif // WITH_NAVMESH_CLUSTER_LINKS

				if (NavSys->IsActiveTilesGenerationEnabled()) // Checks bGenerateNavigationOnlyAroundNavigationInvokers
				{
					DebugLabels.Add(FDebugText(FString::Printf(TEXT("Invoker Locations: %i"), NavSys->GetInvokerLocations().Num())));	
				}
				
				const int32 Running = NavSys->GetNumRunningBuildTasks();
				const int32 Remaining = NavSys->GetNumRemainingBuildTasks(); 
				if (Running || Remaining)
				{
					DebugLabels.Add(FDebugText(FString::Printf(TEXT("Tile jobs running/remaining: %6d / %6d"), Running, Remaining)));	
				}

				DebugLabels.Add(FDebugText(TEXT(""))); // empty line
			}

#if RECAST_INTERNAL_DEBUG_DATA			
			// Tile build time statistics
			if (bGatherTileBuildTimes || NavMeshGeometry.bGatherTileBuildTimesHeatMap)
			{
				DebugLabels.Add(FDebugText(FString::Printf(TEXT("Tile count: %i"), DebugDataMap->Num())));
				DebugLabels.Add(FDebugText(FString::Printf(TEXT("Avg tile build time: %0.2f ms"), AverageBuildTime*1000.)));
				DebugLabels.Add(FDebugText(FString::Printf(TEXT("   Avg comp layers time: %0.2f ms"), AverageCompLayersBuildTime*1000.)));
				DebugLabels.Add(FDebugText(FString::Printf(TEXT("   Avg nav layers time: %0.2f ms"), AverageNavLayersBuildTime*1000.)));
				DebugLabels.Add(FDebugText(FString::Printf(TEXT("   Avg link build time: %0.2f ms"), AverageLinkBuildTime*1000.)));
				DebugLabels.Add(FDebugText(FString::Printf(TEXT("Min: %0.2f ms  Max: %0.2f ms"), NavMeshGeometry.MinTileBuildTime*1000., NavMeshGeometry.MaxTileBuildTime*1000.)));
				DebugLabels.Add(FDebugText(FString::Printf(TEXT("Total: %0.3f s"), TotalTileBuildTime)));
				DebugLabels.Add(FDebugText(FString::Printf(TEXT("   Total link build time: %.4f s"), TotalBuildLinkTime)));

				DebugLabels.Add(FDebugText(TEXT(""))); // empty line
				const double TileAreaM2 = FMath::Square(NavMesh->TileSizeUU) / 10000.;
				const double TotalAreaM2 = DebugDataMap->Num() * TileAreaM2;
				const double TimePer100M2Ms = (TotalTileBuildTime / (TotalAreaM2/100.)) * 1000.;
				DebugLabels.Add(FDebugText(FString::Printf(TEXT("Time per 100m2: %0.2f ms"), TimePer100M2Ms)));

				const double TimePerSqKmS = TotalTileBuildTime / (TotalAreaM2/1000000.);
				DebugLabels.Add(FDebugText(FString::Printf(TEXT("Time per km2: %0.3f s"), TimePerSqKmS)));

				DebugLabels.Add(FDebugText(TEXT(""))); // empty line
			}
#endif // RECAST_INTERNAL_DEBUG_DATA		
		}
		
		const FNavDataConfig& NavConfig = NavMesh->GetConfig();
		TArray<FColor> NavMeshColors;
		NavMeshColors.AddDefaulted(RECAST_MAX_AREAS);

		for (uint8 Idx = 0; Idx < RECAST_MAX_AREAS; Idx++)
		{
			NavMeshColors[Idx] = NavMesh->GetAreaIDColor(Idx);
		}
		NavMeshColors[RECAST_DEFAULT_AREA] = NavConfig.Color.DWColor() > 0 ? NavConfig.Color : NavMeshRenderColor_RecastMesh;

		// Just a little trick to make sure navmeshes with different sized are not drawn with same offset.
		// When DrawOffset is 0, don't add any offset (usually used to debug navmesh height).
		if(NavMesh->DrawOffset != 0.f)
		{
			NavMeshDrawOffset.Z += NavMesh->GetConfig().AgentRadius / 10.f;
		}

		NavMesh->BeginBatchQuery();
		if (TileSet.Num() > 0)
		{
			bool bDone = false;
			for (int32 Idx = 0; Idx < TileSet.Num() && !bDone; Idx++)
			{
				bDone = NavMesh->GetDebugGeometryForTile(NavMeshGeometry, TileSet[Idx]);
			}
		}
		else
		{
			NavMesh->GetDebugGeometryForTile(NavMeshGeometry, FNavTileRef());
		}

		const TArray<FVector>& MeshVerts = NavMeshGeometry.MeshVerts;

		// @fixme, this is going to double up on lots of interior lines
		const bool bGatherTriEdges = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::TriangleEdges);
		if (bGatherTriEdges)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_GatherDebugDrawing_GatherTriEdges);
			
			for (int32 AreaIdx = 0; AreaIdx < RECAST_MAX_AREAS; ++AreaIdx)
			{
				const TArray<int32>& MeshIndices = NavMeshGeometry.AreaIndices[AreaIdx];
				for (int32 Idx = 0; Idx < MeshIndices.Num(); Idx += 3)
				{
					ThickLineItems.Add(FDebugRenderSceneProxy::FDebugLine(MeshVerts[MeshIndices[Idx + 0]] + NavMeshDrawOffset, MeshVerts[MeshIndices[Idx + 1]] + NavMeshDrawOffset, NavMeshRenderColor_Recast_TriangleEdges, DefaultEdges_LineThickness));
					ThickLineItems.Add(FDebugRenderSceneProxy::FDebugLine(MeshVerts[MeshIndices[Idx + 1]] + NavMeshDrawOffset, MeshVerts[MeshIndices[Idx + 2]] + NavMeshDrawOffset, NavMeshRenderColor_Recast_TriangleEdges, DefaultEdges_LineThickness));
					ThickLineItems.Add(FDebugRenderSceneProxy::FDebugLine(MeshVerts[MeshIndices[Idx + 2]] + NavMeshDrawOffset, MeshVerts[MeshIndices[Idx + 0]] + NavMeshDrawOffset, NavMeshRenderColor_Recast_TriangleEdges, DefaultEdges_LineThickness));
				}
			}
		}

		// make lines for tile edges
		const bool bGatherPolyEdges = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::PolyEdges);
		if (bGatherPolyEdges)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_GatherDebugDrawing_GatherPolyEdges);
			
			const TArray<FVector>& TileEdgeVerts = NavMeshGeometry.PolyEdges;
			for (int32 Idx = 0; Idx < TileEdgeVerts.Num(); Idx += 2)
			{
				TileEdgeLines.Add(FDebugRenderSceneProxy::FDebugLine(TileEdgeVerts[Idx] + NavMeshDrawOffset, TileEdgeVerts[Idx + 1] + NavMeshDrawOffset, NavMeshRenderColor_Recast_TileEdges));
			}
		}

		// make lines for navmesh edges
		const bool bGatherBoundaryEdges = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::BoundaryEdges);
		if (bGatherBoundaryEdges)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_GatherDebugDrawing_GatherBoundaryEdges);
			
			const FColor EdgesColor = FNavMeshRenderingHelpers::DarkenColor(NavMeshColors[RECAST_DEFAULT_AREA]);
			const TArray<FVector>& NavMeshEdgeVerts = NavMeshGeometry.NavMeshEdges;
			for (int32 Idx = 0; Idx < NavMeshEdgeVerts.Num(); Idx += 2)
			{
				NavMeshEdgeLines.Add(FDebugRenderSceneProxy::FDebugLine(NavMeshEdgeVerts[Idx] + NavMeshDrawOffset, NavMeshEdgeVerts[Idx + 1] + NavMeshDrawOffset, EdgesColor));
			}
		}

		// offset all navigation-link positions
		const bool bGatherNavLinks = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::NavLinks);
		const bool bGatherFailedNavLinks = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::FailedNavLinks);

#if WITH_NAVMESH_CLUSTER_LINKS
		const bool bGatherClusters = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::Clusters);
		if (bGatherClusters == false)
#endif // WITH_NAVMESH_CLUSTER_LINKS
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_GatherDebugDrawing_OffMeshLinks);
			
			for (int32 OffMeshLineIndex = 0; OffMeshLineIndex < NavMeshGeometry.OffMeshLinks.Num(); ++OffMeshLineIndex)
			{
				FRecastDebugGeometry::FOffMeshLink& Link = NavMeshGeometry.OffMeshLinks[OffMeshLineIndex];
				const bool bLinkValid = (Link.ValidEnds & FRecastDebugGeometry::OMLE_Left) && (Link.ValidEnds & FRecastDebugGeometry::OMLE_Right);

				if (bGatherFailedNavLinks || (bGatherNavLinks && bLinkValid))
				{
					const FVector V0 = Link.Left + NavMeshDrawOffset;
					const FVector V1 = Link.Right + NavMeshDrawOffset;
					const FColor LinkColor = ((Link.Direction && Link.ValidEnds) || (Link.ValidEnds & FRecastDebugGeometry::OMLE_Left)) ? FNavMeshRenderingHelpers::SemiDarkenColor(NavMeshColors[Link.AreaID]) : NavMeshRenderColor_OffMeshConnectionInvalid;

					FNavMeshRenderingHelpers::CacheLink(NavLinkLines, V0, V1, LinkColor, Link.Direction, Link.bIsGenerated);

					// if the connection as a whole is valid check if there are any of ends is invalid
					if (LinkColor != NavMeshRenderColor_OffMeshConnectionInvalid)
					{
						if (Link.Direction && (Link.ValidEnds & FRecastDebugGeometry::OMLE_Left) == 0)
						{
							// left end invalid - mark it
							FNavMeshRenderingHelpers::DrawWireCylinder(NavLinkLines, V0, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), NavMeshRenderColor_OffMeshConnectionInvalid, Link.Radius, NavMesh->GetAgentMaxStepHeight(ENavigationDataResolution::Default), 16, 0, DefaultEdges_LineThickness);
						}
						if ((Link.ValidEnds & FRecastDebugGeometry::OMLE_Right) == 0)
						{
							FNavMeshRenderingHelpers::DrawWireCylinder(NavLinkLines, V1, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1), NavMeshRenderColor_OffMeshConnectionInvalid, Link.Radius, NavMesh->GetAgentMaxStepHeight(ENavigationDataResolution::Default), 16, 0, DefaultEdges_LineThickness);
						}
					}
				}
			}
			
			if (NavMeshGeometry.bMarkForbiddenPolys)
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_GatherDebugDrawing_MarkForbiddenPolys);
				
				for (int32 OffMeshLineIndex = 0; OffMeshLineIndex < NavMeshGeometry.ForbiddenLinks.Num(); ++OffMeshLineIndex)
				{
					FRecastDebugGeometry::FOffMeshLink& Link = NavMeshGeometry.ForbiddenLinks[OffMeshLineIndex];
					
					const FVector V0 = Link.Left + NavMeshDrawOffset;
					const FVector V1 = Link.Right + NavMeshDrawOffset;
					const FColor LinkColor = NavMeshRenderColor_PolyForbidden;

					FNavMeshRenderingHelpers::CacheLink(NavLinkLines, V0, V1, LinkColor, Link.Direction, Link.bIsGenerated);
				}
			}
		}

		const bool bGatherTileLabels = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::TileLabels);
		const bool bGatherTileBounds = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::TileBounds);
		const bool bGatherTileResolutions = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::TileResolutions);
		const bool bGatherPolygonLabels = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::PolygonLabels);
		const bool bGatherPolygonCost = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::PolygonCost);
		const bool bGatherPolygonFlags = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::PolygonFlags);
		const bool bGatherPolyAreaIDs = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::PolygonAreaIDs);

		if (bGatherTileLabels || bGatherTileBounds || bGatherTileResolutions || bGatherPolygonLabels || bGatherPolygonCost || bGatherPolygonFlags || bGatherPolyAreaIDs || bGatherTileBuildTimes)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_GatherDebugDrawing_TileIterations);

			TArray<FNavTileRef> UseTileRefs;
			if (TileSet.Num() > 0)
			{
				UseTileRefs = TileSet;
			}
			else
			{
				NavMesh->GetAllNavMeshTiles(UseTileRefs);
			}

			TMap<FIntPoint, FVector> TileBuildTimeLabelLocations;
			TSet<int32> TilesAreaIDs;
			
			// calculate appropriate points for displaying debug labels
			DebugLabels.Reserve(UseTileRefs.Num());
			for (int32 TileSetIdx = 0; TileSetIdx < UseTileRefs.Num(); TileSetIdx++)
			{
				const FNavTileRef TileRef = UseTileRefs[TileSetIdx];
				int32 X, Y, Layer;
				if (NavMesh->GetNavMeshTileXY(TileRef, X, Y, Layer))
				{
					const FBox TileBoundingBox = NavMesh->GetNavMeshTileBounds(TileRef);
					FVector TileLabelLocation = TileBoundingBox.GetCenter();
					TileLabelLocation.Z = TileBoundingBox.Max.Z;

					FNavLocation NavLocation(TileLabelLocation);
					if (!NavMesh->ProjectPoint(TileLabelLocation, NavLocation, FVector(NavMesh->TileSizeUU / 100, NavMesh->TileSizeUU / 100, TileBoundingBox.Max.Z - TileBoundingBox.Min.Z)))
					{
						NavMesh->ProjectPoint(TileLabelLocation, NavLocation, FVector(NavMesh->TileSizeUU / 2, NavMesh->TileSizeUU / 2, TileBoundingBox.Max.Z - TileBoundingBox.Min.Z));
					}

					if (bGatherTileLabels)
					{
						DebugLabels.Add(FDebugText(NavLocation.Location + NavMeshDrawOffset, FString::Printf(TEXT("(%d,%d:%d)"), X, Y, Layer)));
					}

					if (bGatherTileBuildTimes)
					{
						// Just keep the highest layer location
						const FIntPoint Coord(X, Y);
						FVector* Location = TileBuildTimeLabelLocations.Find(Coord);
						if (!Location)
						{
							TileBuildTimeLabelLocations.Add(Coord, NavLocation.Location);
						}
						else if(NavLocation.Location.Z > Location->Z)
						{
							*Location = NavLocation;
						}
					}	
					
					if (bGatherPolygonLabels || bGatherPolygonCost || bGatherPolygonFlags || bGatherPolyAreaIDs)
					{
						TArray<FNavPoly> Polys;
						NavMesh->GetPolysInTile(TileRef, Polys);

						float DefaultCosts[RECAST_MAX_AREAS];
						float FixedCosts[RECAST_MAX_AREAS];

						if (bGatherPolygonCost)
						{
							NavMesh->GetDefaultQueryFilter()->GetAllAreaCosts(DefaultCosts, FixedCosts, RECAST_MAX_AREAS);
						}

						for (const FNavPoly& Poly : Polys)
						{
							TStringBuilder<100> StringBuilder;

							if (bGatherPolygonLabels)
							{
								uint32 NavPolyIndex = 0;
								uint32 NavTileIndex = 0;
								NavMesh->GetPolyTileIndex(Poly.Ref, NavPolyIndex, NavTileIndex);

								if (StringBuilder.Len() > 0)
								{
									StringBuilder.Append(TEXT("\n"));
								}
								StringBuilder.Appendf(TEXT("Index Tile/Poly: [%d : %d]"), NavTileIndex, NavPolyIndex);
							}

							if (bGatherPolygonCost)
							{
								const uint32 AreaID = NavMesh->GetPolyAreaID(Poly.Ref);

								if (StringBuilder.Len() > 0)
								{
									StringBuilder.Append(TEXT("\n"));
								}
								StringBuilder.Appendf(TEXT("Cost Default/Fixed: [%.3f : %.3f]"), DefaultCosts[AreaID], FixedCosts[AreaID]);
							}

							if (bGatherPolygonFlags)
							{
								uint16 PolyFlags = 0;
								uint16 AreaFlags = 0;
								NavMesh->GetPolyFlags(Poly.Ref, PolyFlags, AreaFlags);

								if (StringBuilder.Len() > 0)
								{
									StringBuilder.Append(TEXT("\n"));
								}
								StringBuilder.Appendf(TEXT("Flags Poly/Area: [0x%X : 0x%X]"), PolyFlags, AreaFlags);
							}

							if (bGatherPolyAreaIDs)
							{
								const uint32 PolyAreaID = NavMesh->GetPolyAreaID(Poly.Ref);
								TilesAreaIDs.Add(PolyAreaID);

								if (StringBuilder.Len() > 0)
								{
									StringBuilder.Append(TEXT("\n"));
								}
								StringBuilder.Appendf(TEXT("AreaID: %d"), PolyAreaID);
							}

							DebugLabels.Add(FDebugText(Poly.Center + NavMeshDrawOffset, StringBuilder.ToString()));
						}
					}

					if (bGatherTileBounds)
					{
						const FBox TileBox = NavMesh->GetNavMeshTileBounds(TileRef);
						const FVector::FReal DrawZ = (TileBox.Min.Z + TileBox.Max.Z) * 0.5;
						const FVector LL(TileBox.Min.X, TileBox.Min.Y, DrawZ);
						const FVector UR(TileBox.Max.X, TileBox.Max.Y, DrawZ);
						const FVector UL(LL.X, UR.Y, DrawZ);
						const FVector LR(UR.X, LL.Y, DrawZ);

						ThickLineItems.Add(FDebugRenderSceneProxy::FDebugLine(LL, UL, NavMeshRenderColor_TileBounds, DefaultEdges_LineThickness));
						ThickLineItems.Add(FDebugRenderSceneProxy::FDebugLine(UL, UR, NavMeshRenderColor_TileBounds, DefaultEdges_LineThickness));
						ThickLineItems.Add(FDebugRenderSceneProxy::FDebugLine(UR, LR, NavMeshRenderColor_TileBounds, DefaultEdges_LineThickness));
						ThickLineItems.Add(FDebugRenderSceneProxy::FDebugLine(LR, LL, NavMeshRenderColor_TileBounds, DefaultEdges_LineThickness));
					}

					if (bGatherTileResolutions)
					{
						const FBox TileBox = NavMesh->GetNavMeshTileBounds(TileRef);
						const FVector::FReal DrawZ = TileBox.Max.Z + NavMeshDrawOffset.Z;
						constexpr FVector::FReal InsideOffset = 10.f;
						const FVector LowerLeft(TileBox.Min.X + InsideOffset, TileBox.Min.Y + InsideOffset, DrawZ);
						const FVector UpperRight(TileBox.Max.X - InsideOffset, TileBox.Max.Y - InsideOffset, DrawZ);
						const FVector UpperLeft(LowerLeft.X, UpperRight.Y, DrawZ);
						const FVector LowerRight(UpperRight.X, LowerLeft.Y, DrawZ);

						FColor TileBoundsColor = FColor::Silver;
						ENavigationDataResolution Resolution = ENavigationDataResolution::Invalid;
						
						if (NavMesh->GetNavmeshTileResolution(TileRef, Resolution))
						{
							switch (Resolution)
							{
							case ENavigationDataResolution::Low:
								TileBoundsColor = FColor::Blue;
								break;
							
							case ENavigationDataResolution::Default:
								TileBoundsColor = FColor::Green;
								break;

							case ENavigationDataResolution::High:
								TileBoundsColor = FColor::Orange;
								break;
							
							default:
								// Unset
								break;
							}
						}
						
						ThickLineItems.Add(FDebugRenderSceneProxy::FDebugLine(LowerLeft, UpperLeft, TileBoundsColor, TileResolution_LineThickness));
						ThickLineItems.Add(FDebugRenderSceneProxy::FDebugLine(UpperLeft, UpperRight, TileBoundsColor, TileResolution_LineThickness));
						ThickLineItems.Add(FDebugRenderSceneProxy::FDebugLine(UpperRight, LowerRight, TileBoundsColor, TileResolution_LineThickness));
						ThickLineItems.Add(FDebugRenderSceneProxy::FDebugLine(LowerRight, LowerLeft, TileBoundsColor, TileResolution_LineThickness));
					}
				}
			}

#if RECAST_INTERNAL_DEBUG_DATA
			if (bGatherTileBuildTimes)
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_GatherDebugDrawing_TileBuildTimes);
				for(const TPair<FIntPoint, FVector>& Pair : TileBuildTimeLabelLocations)
				{
					if (const FRecastInternalDebugData* DebugData = DebugDataMap->Find(Pair.Key))
					{
						constexpr double SecToMs = 1000.;
						const double BuildTimeMs = DebugData->BuildTime * SecToMs;
						const double CompressedLayerTimeMs = DebugData->BuildCompressedLayerTime * SecToMs;
						const double NavigationDataTimeMs = DebugData->BuildNavigationDataTime * SecToMs;
						const ENavigationDataResolution Resolution = (ENavigationDataResolution)DebugData->Resolution;

						const double Range = NavMeshGeometry.MaxTileBuildTime - NavMeshGeometry.MinTileBuildTime;
						int32 Rank = 0;
						if (Range != 0.)
						{
							const double RankCalc = FRecastDebugGeometry::BuildTimeBucketsCount * ((DebugData->BuildTime - NavMeshGeometry.MinTileBuildTime) / Range);
							Rank = static_cast<int32>(FMath::Clamp<double>(RankCalc, 0., FRecastDebugGeometry::BuildTimeBucketsCount-1));
						}

						const FString ResolutionText = StaticEnum<ENavigationDataResolution>()->GetNameStringByValue((int64)Resolution);
						DebugLabels.Add(FDebugText(Pair.Value + NavMeshDrawOffset, FString::Printf(TEXT("%s res\n %.2f ms\n%.2f comp + %.2f nav\n%i tri"),
							*ResolutionText, BuildTimeMs, CompressedLayerTimeMs, NavigationDataTimeMs,
							DebugData->TriangleCount)));
					}
				}
			}
#endif // RECAST_INTERNAL_DEBUG_DATA

			if (bGatherPolyAreaIDs && !TilesAreaIDs.IsEmpty())
			{
				auto SortHandle = [](const int32 A, const int32 B)
				{
					return A < B;
				};
				TilesAreaIDs.StableSort(SortHandle);
				DebugLabels.Add(FDebugText(TEXT("NavAreaClasses:")));
				for (const int32 AreaID : TilesAreaIDs)
				{
					const UClass* AreaClass = NavMesh->GetAreaClass(AreaID);
					DebugLabels.Add(FDebugText(FString::Printf(TEXT("  %d: %s"), AreaID, *GetNameSafe(AreaClass))));
				}
			}
		}

#if RECAST_INTERNAL_DEBUG_DATA
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_GatherDebugDrawing_DisplayInternalData);

			// Display internal debug data
			for (const FIntPoint& point : NavMeshGeometry.TilesToDisplayInternalData)
			{
				if (NavMesh->GetDebugDataMap())
				{
					const FRecastInternalDebugData& DebugData = NavMesh->GetDebugDataMap()->FindRef(point);
					AddMeshForInternalData(DebugData);
				}
			}
		}
#endif // RECAST_INTERNAL_DEBUG_DATA

		NavMesh->FinishBatchQuery();

		// Draw Mesh
		const bool bGatherFilledPolys = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::FilledPolys);
		if (bGatherFilledPolys)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_GatherDebugDrawing_GatherFilledPolys);
			
#if WITH_NAVMESH_CLUSTER_LINKS
			if (bGatherClusters)
			{
				for (int32 Idx = 0; Idx < NavMeshGeometry.Clusters.Num(); ++Idx)
				{
					const TArray<int32>& MeshIndices = NavMeshGeometry.Clusters[Idx].MeshIndices;
					if (MeshIndices.Num() == 0)
					{
						continue;
					}

					FDebugMeshData DebugMeshData;
					DebugMeshData.ClusterColor = FNavMeshRenderingHelpers::GetClusterColor(Idx);
					for (int32 VertIdx = 0; VertIdx < MeshVerts.Num(); ++VertIdx)
					{
						FNavMeshRenderingHelpers::AddVertex(DebugMeshData, MeshVerts[VertIdx] + NavMeshDrawOffset, DebugMeshData.ClusterColor);
					}
					for (int32 TriIdx = 0; TriIdx < MeshIndices.Num(); TriIdx += 3)
					{
						FNavMeshRenderingHelpers::AddTriangleIndices(DebugMeshData, MeshIndices[TriIdx], MeshIndices[TriIdx + 1], MeshIndices[TriIdx + 2]);
					}

					MeshBuilders.Add(DebugMeshData);
				}
			}
			else
#endif // WITH_NAVMESH_CLUSTER_LINKS
			{
				for (int32 AreaType = 0; AreaType < RECAST_MAX_AREAS; ++AreaType)
				{
					FNavMeshRenderingHelpers::AddCluster(MeshBuilders, NavMeshGeometry.AreaIndices[AreaType], NavMeshGeometry.MeshVerts, NavMeshColors[AreaType], NavMeshDrawOffset);
				}
				FNavMeshRenderingHelpers::AddCluster(MeshBuilders, NavMeshGeometry.ForbiddenIndices, NavMeshGeometry.MeshVerts, NavMeshRenderColor_PolyForbidden, NavMeshDrawOffset);
			}
		}

#if RECAST_INTERNAL_DEBUG_DATA		
		if (NavMeshGeometry.bGatherTileBuildTimesHeatMap)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_GatherDebugDrawing_GatherTileBuildHeatMap);
			
			for (int32 Index = 0; Index < FRecastDebugGeometry::BuildTimeBucketsCount; ++Index)
			{
				FNavMeshRenderingHelpers::AddCluster(MeshBuilders, NavMeshGeometry.TileBuildTimesIndices[Index], NavMeshGeometry.MeshVerts, TileBuildTimeColors[Index], NavMeshDrawOffset);
			}
		}
#endif // RECAST_INTERNAL_DEBUG_DATA

		const bool bGatherOctree = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::NavOctree);
		const bool bGatherOctreeDetails = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::NavOctreeDetails);
		const bool bGatherPathCollidingGeometry = FNavMeshRenderingHelpers::HasFlag(NavDetailFlags, ENavMeshDetailFlags::PathCollidingGeometry);
		if (bGatherOctree || bGatherPathCollidingGeometry)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_GatherDebugDrawing_Octree);
			
			const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(NavMesh->GetWorld());
			const FNavigationOctree* NavOctree = NavSys ? NavSys->GetNavOctree() : nullptr;
			if (NavOctree)
			{
				TArray<FVector> CollidingVerts;
				TArray<uint32> CollidingIndices;
				int32 NumElements = 0;

				FBox CurrentNodeBoundsBox;
				NavOctree->FindElementsWithPredicate([bGatherOctree, bGatherOctreeDetails, bGatherPathCollidingGeometry, this, &CurrentNodeBoundsBox, NavOctree, &NumElements](FNavigationOctree::FNodeIndex /*ParentNodeIndex*/, FNavigationOctree::FNodeIndex NodeIndex, const FBoxCenterAndExtent& NodeBounds)
				{
					if (bGatherOctree)
					{
						OctreeBounds.Add(NodeBounds);

						if (bGatherOctreeDetails)
						{
							const int32 NumElementsInNode = NavOctree->GetElementsForNode(NodeIndex).Num();
							NumElements += NumElementsInNode;
							DebugLabels.Emplace(NodeBounds.Center, FString::Printf(TEXT("%d elements"), NumElementsInNode));
						}
					}

					if (bGatherPathCollidingGeometry)
					{
						CurrentNodeBoundsBox = NodeBounds.GetBox();
					}
					return true;
				},
				[bGatherOctree, bGatherOctreeDetails, bGatherPathCollidingGeometry, this, NavMesh, &CollidingVerts, &CollidingIndices, &CurrentNodeBoundsBox](FNavigationOctree::FNodeIndex /*ParentNodeIndex*/, const FNavigationOctreeElement& Element)
				{
					if (bGatherOctree && bGatherOctreeDetails)
					{
						OctreeBounds.Add(FBoxCenterAndExtent(Element.Bounds));
					}

					if (bGatherPathCollidingGeometry)
					{
						if (Element.ShouldUseGeometry(NavMesh->GetConfig()) && Element.Data->CollisionData.Num())
						{
							const FRecastGeometryCache CachedGeometry(Element.Data->CollisionData.GetData());
							TArray<FTransform> InstanceTransforms;
							Element.Data->NavDataPerInstanceTransformDelegate.ExecuteIfBound(CurrentNodeBoundsBox, InstanceTransforms);

							if (InstanceTransforms.Num() == 0)
							{
								FNavMeshRenderingHelpers::AddRecastGeometry(CollidingVerts, CollidingIndices, CachedGeometry.Verts, CachedGeometry.Header.NumVerts, CachedGeometry.Indices, CachedGeometry.Header.NumFaces);
							}
							else
							{
								for (const auto& Transform : InstanceTransforms)
								{
									FNavMeshRenderingHelpers::AddRecastGeometry(CollidingVerts, CollidingIndices, CachedGeometry.Verts, CachedGeometry.Header.NumVerts, CachedGeometry.Indices, CachedGeometry.Header.NumFaces, Transform);
								}
							}
						}
					}
				});

				if (bGatherOctreeDetails)
				{
					DebugLabels.Emplace(FString::Printf(TEXT("Total: %d elements"), NumElements));
				}

				if (CollidingVerts.Num())
				{
					FDebugMeshData DebugMeshData;
					for (int32 VertIdx = 0; VertIdx < CollidingVerts.Num(); ++VertIdx)
					{
						FNavMeshRenderingHelpers::AddVertex(DebugMeshData, CollidingVerts[VertIdx], NavMeshRenderColor_PathCollidingGeom);
					}
					DebugMeshData.Indices = CollidingIndices;
					DebugMeshData.ClusterColor = NavMeshRenderColor_PathCollidingGeom;
					MeshBuilders.Add(DebugMeshData);
				}
			}
		}

		if (NavMeshGeometry.BuiltMeshIndices.Num() > 0)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_GatherDebugDrawing_BuiltMeshIndices);
			
			FDebugMeshData DebugMeshData;
			for (int32 VertIdx = 0; VertIdx < MeshVerts.Num(); ++VertIdx)
			{
				FNavMeshRenderingHelpers::AddVertex(DebugMeshData, MeshVerts[VertIdx] + NavMeshDrawOffset, NavMeshRenderColor_RecastTileBeingRebuilt);
			}
			DebugMeshData.Indices.Append(NavMeshGeometry.BuiltMeshIndices);
			DebugMeshData.ClusterColor = NavMeshRenderColor_RecastTileBeingRebuilt;
			MeshBuilders.Add(DebugMeshData);
		}

#if WITH_NAVMESH_CLUSTER_LINKS
		if (bGatherClusters)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_GatherDebugDrawing_GatherClusters);
			
			for (int32 Idx = 0; Idx < NavMeshGeometry.ClusterLinks.Num(); Idx++)
			{
				const FRecastDebugGeometry::FClusterLink& CLink = NavMeshGeometry.ClusterLinks[Idx];
				const FVector V0 = CLink.FromCluster + NavMeshDrawOffset;
				const FVector V1 = CLink.ToCluster + NavMeshDrawOffset + FVector(0, 0, 20.0f);

				FNavMeshRenderingHelpers::CacheArc(ClusterLinkLines, V0, V1, 0.4f, 4, FColor::Black, ClusterLinkLines_LineThickness);
				const FVector VOffset(0, 0, FVector::Dist(V0, V1) * 1.333f);
				FNavMeshRenderingHelpers::CacheArrowHead(ClusterLinkLines, V1, V0 + VOffset, 30.f, FColor::Black, ClusterLinkLines_LineThickness);
			}
		}
#endif // WITH_NAVMESH_CLUSTER_LINKS

#if WITH_NAVMESH_SEGMENT_LINKS
		// cache segment links
		if (bGatherNavLinks)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_GatherDebugDrawing_NavLinks);
			
			for (int32 AreaIdx = 0; AreaIdx < RECAST_MAX_AREAS; AreaIdx++)
			{
				const TArray<int32>& Indices = NavMeshGeometry.OffMeshSegmentAreas[AreaIdx];
				FNavMeshSceneProxyData::FDebugMeshData DebugMeshData;
				int32 VertBase = 0;

				for (int32 Idx = 0; Idx < Indices.Num(); Idx++)
				{
					FRecastDebugGeometry::FOffMeshSegment& SegInfo = NavMeshGeometry.OffMeshSegments[Indices[Idx]];
					const FVector A0 = SegInfo.LeftStart + NavMeshDrawOffset;
					const FVector A1 = SegInfo.LeftEnd + NavMeshDrawOffset;
					const FVector B0 = SegInfo.RightStart + NavMeshDrawOffset;
					const FVector B1 = SegInfo.RightEnd + NavMeshDrawOffset;
					const FVector Edge0 = B0 - A0;
					const FVector Edge1 = B1 - A1;
					const FVector::FReal Len0 = Edge0.Size();
					const FVector::FReal Len1 = Edge1.Size();
					const FColor SegColor = FNavMeshRenderingHelpers::DarkenColor(NavMeshColors[SegInfo.AreaID]);
					const FColor ColA = (SegInfo.ValidEnds & FRecastDebugGeometry::OMLE_Left) ? FColor::White : FColor::Black;
					const FColor ColB = (SegInfo.ValidEnds & FRecastDebugGeometry::OMLE_Right) ? FColor::White : FColor::Black;

					constexpr int32 NumArcPoints = 8;
					constexpr FVector::FReal ArcPtsScale = 1. / NumArcPoints;

					FVector Prev0 = FNavMeshRenderingHelpers::EvalArc(A0, Edge0, Len0*0.25f, 0);
					FVector Prev1 = FNavMeshRenderingHelpers::EvalArc(A1, Edge1, Len1*0.25f, 0);
					FNavMeshRenderingHelpers::AddVertex(DebugMeshData, Prev0, ColA);
					FNavMeshRenderingHelpers::AddVertex(DebugMeshData, Prev1, ColA);
					for (int32 ArcIdx = 1; ArcIdx <= NumArcPoints; ArcIdx++)
					{
						const FVector::FReal u = ArcIdx * ArcPtsScale;
						FVector Pt0 = FNavMeshRenderingHelpers::EvalArc(A0, Edge0, Len0*0.25f, u);
						FVector Pt1 = FNavMeshRenderingHelpers::EvalArc(A1, Edge1, Len1*0.25f, u);

						FNavMeshRenderingHelpers::AddVertex(DebugMeshData, Pt0, (ArcIdx == NumArcPoints) ? ColB : FColor::White);
						FNavMeshRenderingHelpers::AddVertex(DebugMeshData, Pt1, (ArcIdx == NumArcPoints) ? ColB : FColor::White);

						FNavMeshRenderingHelpers::AddTriangleIndices(DebugMeshData, VertBase + 0, VertBase + 2, VertBase + 1);
						FNavMeshRenderingHelpers::AddTriangleIndices(DebugMeshData, VertBase + 2, VertBase + 3, VertBase + 1);
						FNavMeshRenderingHelpers::AddTriangleIndices(DebugMeshData, VertBase + 0, VertBase + 1, VertBase + 2);
						FNavMeshRenderingHelpers::AddTriangleIndices(DebugMeshData, VertBase + 2, VertBase + 1, VertBase + 3);

						VertBase += 2;
						Prev0 = Pt0;
						Prev1 = Pt1;
					}
					VertBase += 2;

					DebugMeshData.ClusterColor = SegColor;
				}

				if (DebugMeshData.Indices.Num())
				{
					MeshBuilders.Add(DebugMeshData);
				}
			}
		}
#endif // WITH_NAVMESH_SEGMENT_LINKS
	}
}


#if RECAST_INTERNAL_DEBUG_DATA

namespace FNavMeshRenderingHelpers
{
	struct FUniqueColor
	{
		FUniqueColor() : Color(0), Count(0) {}
		FUniqueColor(const FColor InColor, const uint32 InCount) : Color(InColor), Count(InCount) {}
		FColor Color;
		uint32 Count;
		bool operator==(const FUniqueColor& Other) const
		{
			return Color == Other.Color;

		}
	};
	FORCEINLINE uint32 GetTypeHash(const FUniqueColor& UniqueColor)
	{
		return UniqueColor.Color.DWColor();
	}
}

void FNavMeshSceneProxyData::AddMeshForInternalData(const FRecastInternalDebugData& InInternalData)
{
	if (InInternalData.TriangleIndices.Num() > 0)
	{
		const TArray<uint32>& Indices = InInternalData.TriangleIndices;
		const TArray<FVector>& Vertices = InInternalData.TriangleVertices;
		const TArray<FColor>& Colors = InInternalData.TriangleColors;

		if (ensure(Vertices.Num() == Colors.Num()))
		{
			// Split the mesh into different colored pieces (because we cannot use vertex colors).
			TSet<FNavMeshRenderingHelpers::FUniqueColor> UniqueColors;
			FColor PrevColor = Colors[Indices[0]];
			FSetElementId ColorIdx = UniqueColors.Add(FNavMeshRenderingHelpers::FUniqueColor(PrevColor, 1));
			for (int32 i = 3; i < Indices.Num(); i += 3)
			{
				const FColor Color = Colors[Indices[i]];	// Use the first color as representative of the triangle color.
				if (Color != PrevColor)
				{
					ColorIdx = UniqueColors.Add(FNavMeshRenderingHelpers::FUniqueColor(Color, 1));
					PrevColor = Color;
				}
				else
				{
					UniqueColors[ColorIdx].Count++;
				}
			}

			// Add triangles
			for (const FNavMeshRenderingHelpers::FUniqueColor& CurrentColor : UniqueColors)
			{
				const uint32 VertexCount = CurrentColor.Count * 3;
				FDebugMeshData& MeshData = MeshBuilders.AddDefaulted_GetRef();

				MeshData.Indices.Reserve(VertexCount);
				MeshData.Vertices.Reserve(VertexCount);
				for (int32 i = 0; i < Indices.Num(); i += 3)
				{
					const FColor Color = Colors[Indices[i]];
					if (Color == CurrentColor.Color)
					{
						const int32 VertexBase = MeshData.Vertices.Num();
						FNavMeshRenderingHelpers::AddVertex(MeshData, Vertices[Indices[i]] + NavMeshDrawOffset, Color);
						FNavMeshRenderingHelpers::AddVertex(MeshData, Vertices[Indices[i + 1]] + NavMeshDrawOffset, Color);
						FNavMeshRenderingHelpers::AddVertex(MeshData, Vertices[Indices[i + 2]] + NavMeshDrawOffset, Color);
						MeshData.Indices.Add(VertexBase);
						MeshData.Indices.Add(VertexBase + 1);
						MeshData.Indices.Add(VertexBase + 2);
					}
				}
				MeshData.ClusterColor = CurrentColor.Color;
			}
		}
	}

	if (InInternalData.LineVertices.Num() > 0)
	{
		const TArray<FVector>& Vertices = InInternalData.LineVertices;
		const TArray<FColor>& Colors = InInternalData.LineColors;
		if (ensure(Vertices.Num() == Colors.Num()))
		{
			for (int32 i = 0; i < Vertices.Num(); i += 2)
			{
				AuxLines.Emplace(Vertices[i] + NavMeshDrawOffset, Vertices[i + 1] + NavMeshDrawOffset, Colors[i], 0.0f);
			}
		}
	}

	if (InInternalData.PointVertices.Num() > 0)
	{
		const TArray<FVector>& Vertices = InInternalData.PointVertices;
		const TArray<FColor>& Colors = InInternalData.PointColors;
		if (ensure(Vertices.Num() == Colors.Num()))
		{
			for (int32 i = 0; i < Vertices.Num(); i++)
			{
				AuxPoints.Emplace(Vertices[i] + NavMeshDrawOffset, Colors[i], 5.0f);
			}
		}
	}

	if (InInternalData.LabelVertices.Num() > 0)
	{
		const TArray<FVector>& Vertices = InInternalData.LabelVertices;
		const TArray<FString>& Labels = InInternalData.Labels;
		for (int32 i = 0; i < Vertices.Num(); i++)
		{
			DebugLabels.Emplace(Vertices[i] + NavMeshDrawOffset, Labels[i]);
		}
	}
}
#endif //RECAST_INTERNAL_DEBUG_DATA

#endif

//////////////////////////////////////////////////////////////////////////
// FNavMeshSceneProxy

SIZE_T FNavMeshSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

FNavMeshSceneProxy::FNavMeshSceneProxy(const UPrimitiveComponent* InComponent, FNavMeshSceneProxyData* InProxyData, bool ForceToRender)
	: FDebugRenderSceneProxy(InComponent)
	, VertexFactory(GetScene().GetFeatureLevel(), "FNavMeshSceneProxy")
	, bForceRendering(ForceToRender)
{
	DrawType = EDrawType::SolidAndWireMeshes;
	ViewFlagName = TEXT("Navigation");

	if (InProxyData)
	{
		ProxyData = *InProxyData;
		Boxes.Append(InProxyData->AuxBoxes);
	}

	RenderingComponent = MakeWeakObjectPtr(const_cast<UNavMeshRenderingComponent*>(Cast<UNavMeshRenderingComponent>(InComponent)));
	bSkipDistanceCheck = GIsEditor && (GEngine->GetDebugLocalPlayer() == nullptr);
	bUseThickLines = GIsEditor;

	const int32 NumberOfMeshes = ProxyData.MeshBuilders.Num();
	if (!NumberOfMeshes)
	{
		return;
	}

	MeshColors.Reserve(NumberOfMeshes + 1);	// we add one more proxy after the loop
	MeshBatchElements.Reserve(NumberOfMeshes);
	const FMaterialRenderProxy* ParentMaterial = GEngine->DebugMeshMaterial->GetRenderProxy();

	TArray<FDynamicMeshVertex> Vertices;
	for (int32 Index = 0; Index < NumberOfMeshes; ++Index)
	{
		const auto& CurrentMeshBuilder = ProxyData.MeshBuilders[Index];

		FMeshBatchElement Element;
		Element.FirstIndex = IndexBuffer.Indices.Num();
		Element.NumPrimitives = FMath::FloorToInt((float)CurrentMeshBuilder.Indices.Num() / 3);
		Element.MinVertexIndex = Vertices.Num();
		Element.MaxVertexIndex = Element.MinVertexIndex + CurrentMeshBuilder.Vertices.Num() - 1;
		Element.IndexBuffer = &IndexBuffer;
		MeshBatchElements.Add(Element);

		MeshColors.Add(MakeUnique<FColoredMaterialRenderProxy>(ParentMaterial, CurrentMeshBuilder.ClusterColor));

		const int32 VertexIndexOffset = Vertices.Num();
		Vertices.Append(CurrentMeshBuilder.Vertices);
		if (VertexIndexOffset == 0)
		{
			IndexBuffer.Indices.Append(CurrentMeshBuilder.Indices);
		}
		else
		{
			IndexBuffer.Indices.Reserve(IndexBuffer.Indices.Num() + CurrentMeshBuilder.Indices.Num());
			for (const auto VertIndex : CurrentMeshBuilder.Indices)
			{
				IndexBuffer.Indices.Add(VertIndex + VertexIndexOffset);
			}
		}
	}

	MeshColors.Add(MakeUnique<FColoredMaterialRenderProxy>(ParentMaterial, NavMeshRenderColor_PathCollidingGeom));

	if (Vertices.Num())
	{
		VertexBuffers.InitFromDynamicVertex(&VertexFactory, Vertices);
	}
	if (IndexBuffer.Indices.Num())
	{
		BeginInitResource(&IndexBuffer);
	}
}

FNavMeshSceneProxy::~FNavMeshSceneProxy()
{
	VertexBuffers.PositionVertexBuffer.ReleaseResource();
	VertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
	VertexBuffers.ColorVertexBuffer.ReleaseResource();
	IndexBuffer.ReleaseResource();
	VertexFactory.ReleaseResource();
}

void FNavMeshSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastRenderingSceneProxy_GetDynamicMeshElements);

	FDebugRenderSceneProxy::GetDynamicMeshElements(Views, ViewFamily, VisibilityMap, Collector);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			const FSceneView* View = Views[ViewIndex];
			const bool bVisible = !!View->Family->EngineShowFlags.Navigation || bForceRendering;
			if (!bVisible)
			{
				continue;
			}
			FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);

			for (int32 Index = 0; Index < ProxyData.OctreeBounds.Num(); ++Index)
			{
				const FBoxCenterAndExtent& ProxyBounds = ProxyData.OctreeBounds[Index];
				FNavMeshRenderingHelpers::DrawDebugBox(PDI, ProxyBounds.Center, ProxyBounds.Extent, FColor::White);
			}

			// Draw Mesh
			if (MeshBatchElements.Num())
			{
				for (int32 Index = 0; Index < MeshBatchElements.Num(); ++Index)
				{
					if (MeshBatchElements[Index].NumPrimitives == 0)
					{
						continue;
					}

					FMeshBatch& Mesh = Collector.AllocateMesh();
					FMeshBatchElement& BatchElement = Mesh.Elements[0];
					BatchElement = MeshBatchElements[Index];

					FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
					DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), FMatrix::Identity, FMatrix::Identity, GetBounds(), GetLocalBounds(), false, false, AlwaysHasVelocity());
					BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

					Mesh.bWireframe = false;
					Mesh.VertexFactory = &VertexFactory;
					Mesh.MaterialRenderProxy = MeshColors[Index].Get();
					Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
					Mesh.Type = PT_TriangleList;
					Mesh.DepthPriorityGroup = SDPG_World;
					Mesh.bCanApplyViewModeOverrides = false;
					Collector.AddMesh(ViewIndex, Mesh);
				}
			}

			int32 Num = ProxyData.NavMeshEdgeLines.Num();
			PDI->AddReserveLines(SDPG_World, Num, false, false);
			PDI->AddReserveLines(SDPG_Foreground, Num, false, true);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const FDebugLine &Line = ProxyData.NavMeshEdgeLines[Index];
				if (FNavMeshRenderingHelpers::LineInView(Line.Start, Line.End, View))
				{
					if (FNavMeshRenderingHelpers::LineInCorrectDistance(Line.Start, Line.End, View))
					{
						PDI->DrawLine(Line.Start, Line.End, Line.Color, SDPG_World, NavMeshEdges_LineThickness, 0, true);
					}
					else if (bUseThickLines)
					{
						PDI->DrawLine(Line.Start, Line.End, Line.Color, SDPG_Foreground, DefaultEdges_LineThickness, 0, true);
					}
				}
			}

			Num = ProxyData.ClusterLinkLines.Num();
			PDI->AddReserveLines(SDPG_World, Num, false, false);
			PDI->AddReserveLines(SDPG_Foreground, Num, false, true);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const FDebugLine &Line = ProxyData.ClusterLinkLines[Index];
				if (FNavMeshRenderingHelpers::LineInView(Line.Start, Line.End, View))
				{
					if (FNavMeshRenderingHelpers::LineInCorrectDistance(Line.Start, Line.End, View))
					{
						PDI->DrawLine(Line.Start, Line.End, Line.Color, SDPG_World, ClusterLinkLines_LineThickness, 0, true);
					}
					else if (bUseThickLines)
					{
						PDI->DrawLine(Line.Start, Line.End, Line.Color, SDPG_Foreground, DefaultEdges_LineThickness, 0, true);
					}
				}
			}

			Num = ProxyData.TileEdgeLines.Num();
			PDI->AddReserveLines(SDPG_World, Num, false, false);
			PDI->AddReserveLines(SDPG_Foreground, Num, false, true);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const FDebugLine &Line = ProxyData.TileEdgeLines[Index];
				if (FNavMeshRenderingHelpers::LineInView(Line.Start, Line.End, View))
				{
					if (FNavMeshRenderingHelpers::LineInCorrectDistance(Line.Start, Line.End, View))
					{
						PDI->DrawLine(Line.Start, Line.End, Line.Color, SDPG_World, PolyEdges_LineThickness, 0, true);
					}
					else if (bUseThickLines)
					{
						PDI->DrawLine(Line.Start, Line.End, Line.Color, SDPG_Foreground, DefaultEdges_LineThickness, 0, true);
					}
				}
			}

			Num = ProxyData.NavLinkLines.Num();
			PDI->AddReserveLines(SDPG_World, Num, false, false);
			PDI->AddReserveLines(SDPG_Foreground, Num, false, true);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const FDebugLine &Line = ProxyData.NavLinkLines[Index];
				if (FNavMeshRenderingHelpers::LineInView(Line.Start, Line.End, View))
				{
					if (FNavMeshRenderingHelpers::LineInCorrectDistance(Line.Start, Line.End, View))
					{
						PDI->DrawLine(Line.Start, Line.End, Line.Color, SDPG_World, Line.Thickness, 0, true);
					}
					else if (bUseThickLines)
					{
						PDI->DrawLine(Line.Start, Line.End, Line.Color, SDPG_Foreground, DefaultEdges_LineThickness, 0, true);
					}
				}
			}

			Num = ProxyData.AuxLines.Num();
			PDI->AddReserveLines(SDPG_World, Num, false, false);
			PDI->AddReserveLines(SDPG_Foreground, Num, false, true);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const auto& Line = ProxyData.AuxLines[Index];
				if (FNavMeshRenderingHelpers::LineInView(Line.Start, Line.End, View))
				{
					PDI->DrawLine(Line.Start, Line.End, Line.Color, SDPG_World, Line.Thickness, 0, true);
				}
			}

			Num = ProxyData.AuxPoints.Num();
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const auto& Point = ProxyData.AuxPoints[Index];
				if (FNavMeshRenderingHelpers::PointInView(Point.Position, View))
				{
					PDI->DrawPoint(Point.Position, Point.Color, Point.Size, SDPG_World);
				}
			}

			Num = ProxyData.ThickLineItems.Num();
			PDI->AddReserveLines(SDPG_Foreground, Num, false, true);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const auto &Line = ProxyData.ThickLineItems[Index];
				if (FNavMeshRenderingHelpers::LineInView(Line.Start, Line.End, View))
				{
					if (FNavMeshRenderingHelpers::LineInCorrectDistance(Line.Start, Line.End, View))
					{
						PDI->DrawLine(Line.Start, Line.End, Line.Color, SDPG_World, Line.Thickness, 0, true);
					}
					else if (bUseThickLines)
					{
						PDI->DrawLine(Line.Start, Line.End, Line.Color, SDPG_Foreground, DefaultEdges_LineThickness, 0, true);
					}
				}
			}
		}
	}
}

#if WITH_RECAST && UE_ENABLE_DEBUG_DRAWING
void FNavMeshDebugDrawDelegateHelper::DrawDebugLabels(UCanvas* Canvas, APlayerController*)
{
	if (!Canvas)
	{
		return;
	}

	const bool bVisible = (Canvas->SceneView && !!Canvas->SceneView->Family->EngineShowFlags.Navigation) || bForceRendering;
	if (!bVisible || bNeedsNewData || DebugLabels.Num() == 0)
	{
		return;
	}

	const FColor OldDrawColor = Canvas->DrawColor;
	Canvas->SetDrawColor(FColor::White);
	const FSceneView* View = Canvas->SceneView;
	const UFont* Font = GEngine->GetSmallFont();
	const FNavMeshSceneProxyData::FDebugText* DebugText = DebugLabels.GetData();
	float ScreenY = 70.f;
	for (int32 Idx = 0; Idx < DebugLabels.Num(); ++Idx, ++DebugText)
	{
		if (DebugText->Location == FNavigationSystem::InvalidLocation)
		{
			constexpr float ScreenX = 10.f;
			Canvas->DrawText(Font, DebugText->Text, ScreenX, ScreenY);
			if (DebugText->Text.IsEmpty())
			{
				ScreenY += Font->GetMaxCharHeight();
			}
			else
			{
				ScreenY += static_cast<float>(Font->GetStringHeightSize(*DebugText->Text));
			}
		}
		else
		{
			if (FNavMeshRenderingHelpers::PointInView(DebugText->Location, View))
			{
				const FVector ScreenLoc = Canvas->Project(DebugText->Location);
				Canvas->DrawText(Font, DebugText->Text
				, FloatCastChecked<float>(ScreenLoc.X, UE::LWC::DefaultFloatPrecision)
				, FloatCastChecked<float>(ScreenLoc.Y, UE::LWC::DefaultFloatPrecision));
			}
		}
	}

	Canvas->SetDrawColor(OldDrawColor);
}
#endif

FPrimitiveViewRelevance FNavMeshSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	const bool bVisible = !!View->Family->EngineShowFlags.Navigation || bForceRendering;
	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = bVisible && IsShown(View);
	Result.bDynamicRelevance = true;
	// ideally the TranslucencyRelevance should be filled out by the material, here we do it conservative
	Result.bSeparateTranslucency = Result.bNormalTranslucency = bVisible && IsShown(View);
	return Result;
}

uint32 FNavMeshSceneProxy::GetAllocatedSizeInternal() const
{
	return IntCastChecked<uint32>(
		FDebugRenderSceneProxy::GetAllocatedSize() +
		ProxyData.GetAllocatedSize() +
		IndexBuffer.Indices.GetAllocatedSize() +
		VertexBuffers.PositionVertexBuffer.GetNumVertices() * VertexBuffers.PositionVertexBuffer.GetStride() +
		VertexBuffers.StaticMeshVertexBuffer.GetResourceSize() +
		VertexBuffers.ColorVertexBuffer.GetNumVertices() * VertexBuffers.ColorVertexBuffer.GetStride() +
		MeshColors.GetAllocatedSize() + MeshColors.Num() * sizeof(FColoredMaterialRenderProxy) +
		MeshBatchElements.GetAllocatedSize());
}

//////////////////////////////////////////////////////////////////////////
// NavMeshRenderingComponent

#if WITH_EDITOR
namespace
{
	bool AreAnyViewportsRelevant(const UWorld* World)
	{
		FWorldContext* WorldContext = GEngine->GetWorldContextFromWorld(World);
		if (WorldContext && WorldContext->GameViewport)
		{
			return true;
		}
		
		for (FEditorViewportClient* CurrentViewport : GEditor->GetAllViewportClients())
		{
			if (CurrentViewport && CurrentViewport->IsVisible())
			{
				return true;
			}
		}

		return false;
	}
}
#endif

UNavMeshRenderingComponent::UNavMeshRenderingComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	bIsEditorOnly = true;
	bSelectable = false;
	bCollectNavigationData = false;
	bForceUpdate = false;
}

bool UNavMeshRenderingComponent::IsNavigationShowFlagSet(const UWorld* World)
{
	bool bShowNavigation = false;

	FWorldContext* WorldContext = GEngine->GetWorldContextFromWorld(World);

#if WITH_EDITOR
	if (GEditor && WorldContext && WorldContext->WorldType != EWorldType::Game)
	{
		bShowNavigation = WorldContext->GameViewport != nullptr && WorldContext->GameViewport->EngineShowFlags.Navigation;
		if (bShowNavigation == false)
		{
			// we have to check all viewports because we can't to distinguish between SIE and PIE at this point.
			for (FEditorViewportClient* CurrentViewport : GEditor->GetAllViewportClients())
			{
				if (CurrentViewport && CurrentViewport->EngineShowFlags.Navigation)
				{
					bShowNavigation = true;
					break;
				}
			}
		}
	}
	else
#endif //WITH_EDITOR
	{
		bShowNavigation = WorldContext && WorldContext->GameViewport && WorldContext->GameViewport->EngineShowFlags.Navigation;
	}

	return bShowNavigation;
}

void UNavMeshRenderingComponent::TimerFunction()
{
	const UWorld* World = GetWorld();
#if WITH_EDITOR
	if (GEditor && (AreAnyViewportsRelevant(World) == false))
	{
		// unable to tell if the flag is on or not
		return;
	}
#endif // WITH_EDITOR

	const bool bShowNavigation = bForceUpdate || IsNavigationShowFlagSet(World);

	if (bShowNavigation != !!bCollectNavigationData && bShowNavigation == true)
	{
		bForceUpdate = false;
		bCollectNavigationData = bShowNavigation;
		MarkRenderStateDirty();
	}
}

void UNavMeshRenderingComponent::OnRegister()
{
	Super::OnRegister();

#if WITH_RECAST && UE_ENABLE_DEBUG_DRAWING
	// it's a kind of HACK but there is no event or other information that show flag was changed by user => we have to check it periodically
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->GetTimerManager()->SetTimer(TimerHandle, FTimerDelegate::CreateUObject(this, &UNavMeshRenderingComponent::TimerFunction), 1, true);
	}
	else
#endif //WITH_EDITOR
	{
		GetWorld()->GetTimerManager().SetTimer(TimerHandle, FTimerDelegate::CreateUObject(this, &UNavMeshRenderingComponent::TimerFunction), 1, true);
	}
#endif //WITH_RECAST && UE_ENABLE_DEBUG_DRAWING
}

void UNavMeshRenderingComponent::OnUnregister()
{
#if WITH_RECAST && UE_ENABLE_DEBUG_DRAWING
	// it's a kind of HACK but there is no event or other information that show flag was changed by user => we have to check it periodically
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->GetTimerManager()->ClearTimer(TimerHandle);
	}
	else
#endif //WITH_EDITOR
	{
		GetWorld()->GetTimerManager().ClearTimer(TimerHandle);
	}
#endif //WITH_RECAST && UE_ENABLE_DEBUG_DRAWING
	Super::OnUnregister();
}

void UNavMeshRenderingComponent::GatherData(const ARecastNavMesh& NavMesh, FNavMeshSceneProxyData& OutProxyData) const
{
#if WITH_RECAST
	const int32 DetailFlags = FNavMeshRenderingHelpers::GetDetailFlags(&NavMesh);
	const TArray<FNavTileRef> EmptyTileSet;
	OutProxyData.GatherData(&NavMesh, DetailFlags, EmptyTileSet);
#endif // WITH_RECAST
}

#if UE_ENABLE_DEBUG_DRAWING
FDebugRenderSceneProxy* UNavMeshRenderingComponent::CreateDebugSceneProxy()
{
#if WITH_RECAST
	FNavMeshSceneProxy* NavMeshSceneProxy = nullptr;

	const bool bShowNavigation = IsNavigationShowFlagSet(GetWorld());

	bCollectNavigationData = bShowNavigation;

	if (bCollectNavigationData && IsVisible())
	{
		const ARecastNavMesh* NavMesh = Cast<ARecastNavMesh>(GetOwner());
		if (NavMesh && NavMesh->IsDrawingEnabled())
		{
			FNavMeshSceneProxyData ProxyData;
			GatherData(*NavMesh, ProxyData);

			NavMeshSceneProxy = new FNavMeshSceneProxy(this, &ProxyData);
			NavMeshDebugDrawDelegateManager.SetupFromProxy(NavMeshSceneProxy);
		}
		else
		{
			NavMeshDebugDrawDelegateManager.Reset();
		}
	}

	return NavMeshSceneProxy;
#else
	return nullptr;
#endif // WITH_RECAST
}
#endif

FBoxSphereBounds UNavMeshRenderingComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBox BoundingBox(ForceInit);
#if WITH_RECAST
	ARecastNavMesh* NavMesh = Cast<ARecastNavMesh>(GetOwner());
	if (NavMesh)
	{
		BoundingBox = NavMesh->GetNavMeshBounds();
		if (NavMesh->bDrawOctree)
		{
			const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
			const FNavigationOctree* NavOctree = NavSys ? NavSys->GetNavOctree() : nullptr;
			if (NavOctree)
			{
				//this is only iterating over the root node
				BoundingBox += NavOctree->GetRootBounds().GetBox();
			}
		}
	}
#endif
	return FBoxSphereBounds(BoundingBox);
}

