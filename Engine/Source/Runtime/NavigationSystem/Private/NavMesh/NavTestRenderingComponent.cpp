// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavMesh/NavTestRenderingComponent.h"
#include "NavigationTestingActor.h"
#include "NavMesh/RecastNavMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavTestRenderingComponent)

#if UE_ENABLE_DEBUG_DRAWING
#include "Debug/DebugDrawService.h"
#include "DynamicMeshBuilder.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "MeshElementCollector.h"
#include "PrimitiveDrawingUtils.h"

static constexpr FColor NavMeshRenderColor_OpenSet(255,128,0,255);
static constexpr FColor NavMeshRenderColor_ClosedSet(255,196,0,255);
static constexpr uint8 NavMeshRenderAlpha_Modified = 255;
static constexpr uint8 NavMeshRenderAlpha_NonModified = 64;

SIZE_T FNavTestSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

FNavTestSceneProxy::FNavTestSceneProxy(const UNavTestRenderingComponent* InComponent)
	: FDebugRenderSceneProxy(InComponent)
	, NavMeshDrawOffset(0,0,10)
	, NavTestActor(nullptr)
{
	ViewFlagName = TEXT("Navigation");

	if (InComponent == nullptr)
	{
		return;
	}

	NavTestActor = Cast<ANavigationTestingActor>(InComponent->GetOwner());
	if (NavTestActor == nullptr)
	{
		return;
	}

	NavMeshDrawOffset.Z += NavTestActor->NavAgentProps.AgentRadius / 10.f;
	bShowNodePool = NavTestActor->bShowNodePool;
	bShowBestPath = NavTestActor->bShowBestPath;
	bShowDiff = NavTestActor->bShowDiffWithPreviousStep;

	ClosestWallLocation = NavTestActor->bDrawDistanceToWall ? NavTestActor->ClosestWallLocation : FNavigationSystem::InvalidLocation;

	GatherPathPoints();
	GatherPathStep();
}

void FNavTestSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	FRHICommandList& RHICmdList = Collector.GetRHICommandList();

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			const FSceneView* View = Views[ViewIndex];
			FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);

			if (NavTestActor)
			{
				//DrawArc(PDI, Link.Left, Link.Right, 0.4f, NavMeshColors[Link.AreaID], SDPG_World, 3.5f);
				//const FVector VOffset(0,0,FVector::Dist(Link.Left, Link.Right)*1.333f);
				//DrawArrowHead(PDI, Link.Right, Link.Left+VOffset, 30.f, NavMeshColors[Link.AreaID], SDPG_World, 3.5f);

				//@todo - the rendering thread should never read from UObjects directly!  These are race conditions, the properties should be mirrored on the proxy
				const FVector ActorLocation = NavTestActor->GetActorLocation();
				const FVector ProjectedLocation = NavTestActor->bProjectedLocationValid ? (NavTestActor->ProjectedLocation + (FVector)NavMeshDrawOffset) : (ActorLocation - FVector(0, 0, NavTestActor->QueryingExtent.Z));
				const FColor ProjectedColor = (NavTestActor->bProjectedLocationValid ? FColor::Green : FColor::Red).WithAlpha(120);
				const FColor ClosestWallColor = FColorList::Orange;
				const FVector BoxExtent(20, 20, 20);

				const FMaterialRenderProxy* const ColoredMeshInstance = &Collector.AllocateOneFrameResource<FColoredMaterialRenderProxy>(GEngine->DebugMeshMaterial->GetRenderProxy(), ProjectedColor);
				//DrawBox(PDI, FTransform(ProjectedLocation).ToMatrixNoScale(),BoxExtent, ColoredMeshInstance, SDPG_World);
				if (NavTestActor->bProjectedLocationValid)
				{
					GetSphereMesh(ProjectedLocation, BoxExtent, 10, 7, ColoredMeshInstance, SDPG_World, false, ViewIndex, Collector);
				}

				//DrawWireBox(PDI, FBox(ProjectedLocation-BoxExtent, ProjectedLocation+BoxExtent), ProjectedColor, false);
				DrawWireBox(PDI, FBox(ActorLocation - BoxExtent, ActorLocation + BoxExtent), FColor::White, false);
				const FVector LineEnd = NavTestActor->bProjectedLocationValid ? ProjectedLocation - (ProjectedLocation - ActorLocation).GetSafeNormal()*BoxExtent.X : ProjectedLocation;
				PDI->DrawLine(LineEnd, ActorLocation, ProjectedColor, SDPG_World, 2.5);
				DrawArrowHead(PDI, LineEnd, ActorLocation, 20.f, ProjectedColor, SDPG_World, 2.5f);

				// draw query extent
				DrawWireBox(PDI, FBox(ActorLocation - NavTestActor->QueryingExtent, ActorLocation + NavTestActor->QueryingExtent), FColor::Blue, false);

				if (FNavigationSystem::IsValidLocation(ClosestWallLocation))
				{
					PDI->DrawLine(ClosestWallLocation, ActorLocation, ClosestWallColor, SDPG_World, 2.5);
				}

				if (NavTestActor->bDrawIfNavDataIsReadyInRadius)
				{
					constexpr double HalfHeight = 1000;
					constexpr int32 NumSides = 32;
					DrawWireCylinder(PDI, ActorLocation, FVector(1, 0, 0), FVector(0, 1, 0), FVector(0, 0, 1),
						NavTestActor->bNavDataIsReadyInRadius ? FColor::Green : FColor::Red, 
						NavTestActor->RadiusUsedToValidateNavData, HalfHeight,  
						NumSides, SDPG_World);
				}

				if (NavTestActor->bDrawIfNavDataIsReadyToQueryTargetActor && NavTestActor->QueryTargetActor)
				{
					const FVector QueryTargetLocation = NavTestActor->QueryTargetActor->GetActorLocation();
					float Distance;
					FVector Forward, Right, Up;
					(QueryTargetLocation-ActorLocation).ToDirectionAndLength(Up, Distance);
					Up.FindBestAxisVectors(Forward, Right);

					constexpr int32 NumSides = 32;
					DrawWireCapsule(PDI, ActorLocation + 0.5f * Distance * Up, Forward, Right, Up,
						NavTestActor->bNavDataIsReadyToQueryTargetActor ? FColor::Green : FColor::Red, 
						NavTestActor->RadiusUsedToValidateNavData, 0.5f*Distance + NavTestActor->RadiusUsedToValidateNavData,  
						NumSides, SDPG_World);
				}

				if (NavTestActor->bDrawRaycastToQueryTargetActor && NavTestActor->QueryTargetActor)
				{
					const FVector QueryTargetLocation = NavTestActor->QueryTargetActor->GetActorLocation();
					PDI->DrawLine(ActorLocation, QueryTargetLocation, NavTestActor->bRaycastToQueryTargetActorResult ? FColor::Red : NavTestActor->bRaycastToQueryTargetEndsInCorridor ? FColor::Green : FColor::Orange, SDPG_World, 2.5);

					if (FNavigationSystem::IsValidLocation(NavTestActor->RaycastHitLocation))
					{
						DrawWireSphere(PDI, NavTestActor->RaycastHitLocation, FColor::Red, 25.f, 6, SDPG_World);
					}
				}
			}

			// draw path
			if (!bShowBestPath || !NodeDebug.Num())
			{
				for (int32 PointIndex = 1; PointIndex < PathPoints.Num(); PointIndex++)
				{
					PDI->DrawLine(PathPoints[PointIndex-1], PathPoints[PointIndex], FLinearColor::Red, SDPG_World, 2.0f, 0.0f);
					DrawArrowHead(PDI, PathPoints[PointIndex], PathPoints[PointIndex - 1], 25.f, FLinearColor::Red, SDPG_World, 2.0f);
				}
			}

			// draw path debug data
			if (bShowNodePool)
			{
				if (ClosedSetIndices.Num())
				{
					const FColoredMaterialRenderProxy *MeshColorInstance = &Collector.AllocateOneFrameResource<FColoredMaterialRenderProxy>(GEngine->DebugMeshMaterial->GetRenderProxy(), NavMeshRenderColor_ClosedSet);						
					FDynamicMeshBuilder	MeshBuilder(View->GetFeatureLevel());
					MeshBuilder.AddVertices(ClosedSetVerts);
					MeshBuilder.AddTriangles(ClosedSetIndices);
					MeshBuilder.GetMesh(FMatrix::Identity, MeshColorInstance, IntCastChecked<uint8>((int32)GetDepthPriorityGroup(View)), false, false, ViewIndex, Collector);
				}

				if (OpenSetIndices.Num())
				{
					const FColoredMaterialRenderProxy *MeshColorInstance = &Collector.AllocateOneFrameResource<FColoredMaterialRenderProxy>(GEngine->DebugMeshMaterial->GetRenderProxy(), NavMeshRenderColor_OpenSet);						
					FDynamicMeshBuilder	MeshBuilder(View->GetFeatureLevel());
					MeshBuilder.AddVertices(OpenSetVerts);
					MeshBuilder.AddTriangles(OpenSetIndices);
					MeshBuilder.GetMesh(FMatrix::Identity, MeshColorInstance, IntCastChecked<uint8>((int32)GetDepthPriorityGroup(View)), false, false, ViewIndex, Collector);
				}
			}

			for (TSet<FNodeDebugData>::TConstIterator It(NodeDebug); It; ++It)
			{
				const FNodeDebugData& NodeData = *It;

				FColor LineColor(FColor::Blue);
				if (bShowBestPath && NodeData.bBestPath)
				{
					LineColor = FColor::Red;
				}

				if (bShowDiff)
				{
					LineColor.A = NodeData.bModified ? NavMeshRenderAlpha_Modified : NavMeshRenderAlpha_NonModified;
				}

				FVector ParentPos(NodeData.ParentId.IsValidId() ? NodeDebug[NodeData.ParentId].Position : NodeData.Position);

				if (bShowDiff && !NodeData.bModified)
				{
					PDI->DrawLine(NodeData.Position, ParentPos, LineColor, SDPG_World);
				}
				else
				{
					PDI->DrawLine(NodeData.Position, ParentPos, LineColor, SDPG_World, 2.0f, 0.0, true);
				}

				if (NodeData.bOffMeshLink)
				{
					DrawWireBox(PDI, FBox::BuildAABB(NodeData.Position, FVector(10.0f)), LineColor, SDPG_World);
				}

				if (bShowDiff && NodeData.bModified)
				{
					PDI->DrawLine(NodeData.Position + FVector(0,0,10), NodeData.Position + FVector(0,0,100), FColor::Green, SDPG_World);
				}
			}
		}
	}
}

void FNavTestSceneProxy::GatherPathPoints()
{
	if (NavTestActor && NavTestActor->LastPath.IsValid())
	{
		for (int32 PointIndex = 0; PointIndex < NavTestActor->LastPath->GetPathPoints().Num(); PointIndex++)
		{
			PathPoints.Add(NavTestActor->LastPath->GetPathPoints()[PointIndex].Location);
			PathPointFlags.Add(FString::Printf(TEXT("%d-%d"), PointIndex, FNavMeshNodeFlags(NavTestActor->LastPath->GetPathPoints()[PointIndex].Flags).AreaFlags));
		}
	}
}

void FNavTestSceneProxy::GatherPathStep()
{
	OpenSetVerts.Reset();
	ClosedSetVerts.Reset();
	OpenSetIndices.Reset();
	ClosedSetIndices.Reset();
	NodeDebug.Empty(NodeDebug.Num());
	BestNodeId = FSetElementId();

#if WITH_EDITORONLY_DATA && WITH_RECAST
	// DebugSteps are only available for: WITH_EDITORONLY_DATA && WITH_RECAST
	if (NavTestActor && NavTestActor->DebugSteps.Num() && NavTestActor->ShowStepIndex >= 0)
	{
		const int32 ShowIdx = FMath::Min(NavTestActor->ShowStepIndex, NavTestActor->DebugSteps.Num() - 1);
		const FRecastDebugPathfindingData& DebugStep = NavTestActor->DebugSteps[ShowIdx];
		int32 BaseOpen = 0;
		int32 BaseClosed = 0;

		for (TSet<FRecastDebugPathfindingNode>::TConstIterator It(DebugStep.Nodes); It; ++It)
		{
			const FRecastDebugPathfindingNode& DebugNode = *It;
			if (DebugNode.bOpenSet)
			{
				for (int32 iv = 0; iv < DebugNode.Verts.Num(); iv++)
				{
					OpenSetVerts.Add(DebugNode.Verts[iv] + NavMeshDrawOffset);
				}

				for (int32 iv = 2; iv < DebugNode.Verts.Num(); iv++)
				{
					OpenSetIndices.Add(BaseOpen + 0);
					OpenSetIndices.Add(BaseOpen + iv - 1);
					OpenSetIndices.Add(BaseOpen + iv);
				}

				BaseOpen += DebugNode.Verts.Num();
			}
			else
			{
				for (int32 iv = 0; iv < DebugNode.Verts.Num(); iv++)
				{
					ClosedSetVerts.Add(DebugNode.Verts[iv] + NavMeshDrawOffset);
				}

				for (int32 iv = 2; iv < DebugNode.Verts.Num(); iv++)
				{
					ClosedSetIndices.Add(BaseClosed + 0);
					ClosedSetIndices.Add(BaseClosed + iv - 1);
					ClosedSetIndices.Add(BaseClosed + iv);
				}

				BaseClosed += DebugNode.Verts.Num();
			}

			FNodeDebugData NewNodeData;

			FVector::FReal DisplayedCost = TNumericLimits<FVector::FReal>::Max();
			switch (NavTestActor->CostDisplayMode)
			{
			case ENavCostDisplay::TotalCost:
				DisplayedCost = DebugNode.TotalCost;
				break;
			case ENavCostDisplay::RealCostOnly:
				DisplayedCost = DebugNode.Cost;
				break;
			case ENavCostDisplay::HeuristicOnly:
				DisplayedCost = DebugNode.GetHeuristicCost();
				break;
			default:
				break;
			}

			NewNodeData.Desc = FString::Printf(TEXT("%.2f%s"), DisplayedCost, DebugNode.bOffMeshLink ? TEXT(" [link]") : TEXT(""));

			NewNodeData.Position = DebugNode.NodePos;
			NewNodeData.PolyRef = DebugNode.PolyRef;
			NewNodeData.bClosedSet = !DebugNode.bOpenSet;
			NewNodeData.bBestPath = (It.GetId() == DebugStep.BestNode);
			NewNodeData.bModified = DebugNode.bModified;
			NewNodeData.bOffMeshLink = DebugNode.bOffMeshLink;

			const FSetElementId NewId = NodeDebug.Add(NewNodeData);
			if (NewNodeData.bBestPath)
			{
				BestNodeId = NewId;
			}
		}

		FRecastDebugPathfindingNode ThisNode;
		FNodeDebugData ParentDebugNode;

		for (TSet<FNodeDebugData>::TIterator It(NodeDebug); It; ++It)
		{
			FNodeDebugData& MyDebugNode = *It;
				
			ThisNode.PolyRef = MyDebugNode.PolyRef;
			const FRecastDebugPathfindingNode* MyNode = DebugStep.Nodes.Find(ThisNode);

			if (MyNode)
			{
				ParentDebugNode.PolyRef = MyNode->ParentRef;
				MyDebugNode.ParentId = NodeDebug.FindId(ParentDebugNode);
			}
		}

		FSetElementId BestPathId = BestNodeId;
		while (BestPathId.IsValidId())
		{
			FNodeDebugData& MyDebugNode = NodeDebug[BestPathId];

			MyDebugNode.bBestPath = true;
			BestPathId = MyDebugNode.ParentId;
		}
	}
#endif // WITH_EDITORONLY_DATA && WITH_RECAST
}

FPrimitiveViewRelevance FNavTestSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = IsShown(View);
	Result.bDynamicRelevance = true;
	// ideally the TranslucencyRelevance should be filled out by the material, here we do it conservative
	Result.bSeparateTranslucency = Result.bNormalTranslucency = IsShown(View) && GIsEditor;
	return Result;
}

uint32 FNavTestSceneProxy::GetAllocatedSizeInternal() const
{
	SIZE_T InternalAllocSize = 0;
	for (TSet<FNodeDebugData>::TConstIterator It(NodeDebug); It; ++It)
	{
		InternalAllocSize += (*It).Desc.GetAllocatedSize();
	}

	return IntCastChecked<uint32>(FDebugRenderSceneProxy::GetAllocatedSize() + PathPoints.GetAllocatedSize()
		+ PathPointFlags.GetAllocatedSize()
		+ OpenSetVerts.GetAllocatedSize() + OpenSetIndices.GetAllocatedSize()
		+ ClosedSetVerts.GetAllocatedSize() + ClosedSetIndices.GetAllocatedSize()
		+ NodeDebug.GetAllocatedSize() + InternalAllocSize);

}

void FNavTestDebugDrawDelegateHelper::SetupFromProxy(const FNavTestSceneProxy* InSceneProxy)
{
	PathPoints.Reset();
	PathPoints.Append(InSceneProxy->PathPoints);
	PathPointFlags.Reset();
	PathPointFlags.Append(InSceneProxy->PathPointFlags);
	NodeDebug.Reset();
	NodeDebug.Append(InSceneProxy->NodeDebug);
	NavTestActor = InSceneProxy->NavTestActor;
	BestNodeId = InSceneProxy->BestNodeId;
	bShowBestPath = InSceneProxy->bShowBestPath;
	bShowDiff = InSceneProxy->bShowDiff;
}

void FNavTestDebugDrawDelegateHelper::DrawDebugLabels(UCanvas* Canvas, APlayerController*)
{
	if (NavTestActor == nullptr)
	{
		return;
	}

	const FColor OldDrawColor = Canvas->DrawColor;
	Canvas->SetDrawColor(FColor::White);
	const FSceneView* View = Canvas->SceneView;

	if (NodeDebug.Num())
	{
		const UFont* RenderFont = GEngine->GetSmallFont();
		for (TSet<FNavTestSceneProxy::FNodeDebugData>::TConstIterator It(NodeDebug); It; ++It)
		{
			const FNavTestSceneProxy::FNodeDebugData& NodeData = *It;

			if (FNavTestSceneProxy::LocationInView(NodeData.Position, View))
			{
				FColor MyColor = NodeData.bClosedSet ? FColor(64, 64, 64) : FColor::White;
				if (!bShowBestPath && It.GetId() == BestNodeId)
				{
					MyColor = FColor::Red;
				}
				if (bShowDiff)
				{
					MyColor.A = NodeData.bModified ? NavMeshRenderAlpha_Modified : NavMeshRenderAlpha_NonModified;
				}

				Canvas->SetDrawColor(MyColor);

				const FVector3f ScreenLoc(Canvas->Project(NodeData.Position) + FVector(NavTestActor->TextCanvasOffset, 0.f));
				Canvas->DrawText(RenderFont, NodeData.Desc, ScreenLoc.X, ScreenLoc.Y);
			}
		}
	}
	else
	{
		for (int32 PointIndex = 0; PointIndex < PathPoints.Num(); ++PointIndex)
		{
			if (FNavTestSceneProxy::LocationInView(PathPoints[PointIndex], View))
			{
				const FVector3f PathPointLoc(Canvas->Project(PathPoints[PointIndex]));
				const UFont* RenderFont = GEngine->GetSmallFont();
				Canvas->DrawText(RenderFont, PathPointFlags[PointIndex], PathPointLoc.X, PathPointLoc.Y);

			}
		}
	}

	Canvas->SetDrawColor(OldDrawColor);
}
#endif // UE_ENABLE_DEBUG_DRAWING

#if UE_ENABLE_DEBUG_DRAWING
FDebugRenderSceneProxy* UNavTestRenderingComponent::CreateDebugSceneProxy()
{
	FNavTestSceneProxy* NewSceneProxy = new FNavTestSceneProxy(this);
	NavTestDebugDrawDelegateHelper.SetupFromProxy(NewSceneProxy);
	return NewSceneProxy;
}
#endif

FBoxSphereBounds UNavTestRenderingComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBox BoundingBox(ForceInit);

	ANavigationTestingActor* TestActor = Cast<ANavigationTestingActor>(GetOwner());
	if (TestActor)
	{
		BoundingBox = TestActor->GetComponentsBoundingBox(true);
	
		if (TestActor->LastPath.IsValid())
		{
			for (int32 PointIndex = 0; PointIndex < TestActor->LastPath->GetPathPoints().Num(); PointIndex++)
			{
				BoundingBox += TestActor->LastPath->GetPathPoints()[PointIndex].Location;
			}
		}
#if WITH_EDITORONLY_DATA && WITH_RECAST
		// DebugSteps are only available for: WITH_EDITORONLY_DATA && WITH_RECAST
		if (TestActor->DebugSteps.Num() && TestActor->ShowStepIndex >= 0)
		{
			const int32 ShowIdx = FMath::Min(TestActor->ShowStepIndex, TestActor->DebugSteps.Num() - 1);
			const FRecastDebugPathfindingData& DebugStep = TestActor->DebugSteps[ShowIdx];
			for (TSet<FRecastDebugPathfindingNode>::TConstIterator It(DebugStep.Nodes); It; ++It)
			{
				const FRecastDebugPathfindingNode& DebugNode = *It;
				for (int32 iv = 0; iv < DebugNode.Verts.Num(); iv++)
				{
					BoundingBox += (FVector)DebugNode.Verts[iv];
				}
			}
		}
#endif // WITH_EDITORONLY_DATA && WITH_RECAST
	}

	return FBoxSphereBounds(BoundingBox);
}

