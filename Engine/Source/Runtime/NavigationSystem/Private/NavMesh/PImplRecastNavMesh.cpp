// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavMesh/PImplRecastNavMesh.h"
#include "NavigationSystem.h"
#include "Misc/TransactionCommon.h"

#if WITH_RECAST

#include "NavMesh/RecastHelpers.h"
#include "NavMesh/RecastVersion.h"
#include "Detour/DetourNode.h"
#include "Detour/DetourNavMesh.h"
#include "Recast/RecastAlloc.h"
#include "DetourTileCache/DetourTileCacheBuilder.h"
#include "NavAreas/NavArea.h"
#include "NavMesh/RecastNavMeshGenerator.h"
#include "NavMesh/RecastQueryFilter.h"
#include "NavLinkCustomInterface.h"
#include "VisualLogger/VisualLogger.h"

#include "Misc/LargeWorldCoordinates.h"
#include "DebugUtils/DebugDrawLargeWorldCoordinates.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

//----------------------------------------------------------------------//
// bunch of compile-time checks to assure types used by Recast and our
// mid-layer are the same size
//----------------------------------------------------------------------//
static_assert(sizeof(NavNodeRef) == sizeof(dtPolyRef), "NavNodeRef and dtPolyRef should be the same size.");
static_assert(RECAST_MAX_AREAS <= DT_MAX_AREAS, "Number of allowed areas cannot exceed DT_MAX_AREAS.");
static_assert(RECAST_STRAIGHTPATH_OFFMESH_CONNECTION == DT_STRAIGHTPATH_OFFMESH_CONNECTION, "Path flags values differ.");
static_assert(RECAST_UNWALKABLE_POLY_COST == DT_UNWALKABLE_POLY_COST, "Unwalkable poly cost differ.");
static_assert(std::is_same_v<FVector::FReal, dtReal>, "FReal and dtReal must be the same type!");
static_assert(std::is_same_v<FVector::FReal, rcReal>, "FReal and rcReal must be the same type!");
static_assert(std::is_same_v<FVector::FReal, duReal>, "FReal and duReal must be the same type!");

/// Helper for accessing navigation query from different threads
#define INITIALIZE_NAVQUERY_SIMPLE(NavQueryVariable, NumNodes)	\
	dtNavMeshQuery NavQueryVariable##Private;	\
	dtNavMeshQuery& NavQueryVariable = IsInGameThread() ? SharedNavQuery : NavQueryVariable##Private; \
	NavQueryVariable.init(DetourNavMesh, NumNodes);

#define INITIALIZE_NAVQUERY(NavQueryVariable, NumNodes, LinkFilter)	\
	dtNavMeshQuery NavQueryVariable##Private;	\
	dtNavMeshQuery& NavQueryVariable = IsInGameThread() ? SharedNavQuery : NavQueryVariable##Private; \
	NavQueryVariable.init(DetourNavMesh, NumNodes, &LinkFilter);

static void* DetourMalloc(int Size, dtAllocHint Hint)
{
	LLM_SCOPE(ELLMTag::NavigationRecast);
	void* Result = FMemory::Malloc(uint32(Size));
#if STATS
	const SIZE_T ActualSize = FMemory::GetAllocSize(Result);

	switch (Hint)
	{
	case DT_ALLOC_TEMP:
		INC_MEMORY_STAT_BY(STAT_Navigation_DetourTEMP, ActualSize);
		break;

	case DT_ALLOC_PERM_AVOIDANCE:				
		INC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_AVOIDANCE, ActualSize);
		break;

	case DT_ALLOC_PERM_CROWD:					
		INC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_CROWD, ActualSize);
		break;

	case DT_ALLOC_PERM_LOOKUP:
		INC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_LOOKUP, ActualSize);
		break;

	case DT_ALLOC_PERM_NAVQUERY:
		INC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_NAVQUERY, ActualSize);
		break;

	case DT_ALLOC_PERM_NAVMESH:
		INC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_NAVMESH, ActualSize);
		break;

	case DT_ALLOC_PERM_NODE_POOL:
		INC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_NODE_POOL, ActualSize);
		break;

	case DT_ALLOC_PERM_PATH_CORRIDOR:
		INC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_PATH_CORRIDOR, ActualSize);
		break;

	case DT_ALLOC_PERM_PATH_QUEUE:
		INC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_PATH_QUEUE, ActualSize);
		break;

	case DT_ALLOC_PERM_PROXIMITY_GRID:
		INC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_PROXY_GRID, ActualSize);
		break;

	case DT_ALLOC_PERM_TILE_DATA:
		INC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_TILE_DATA, ActualSize);
		break;

	case DT_ALLOC_PERM_TILE_DYNLINK:
		INC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_TILE_DYNLINK, ActualSize);
		break;

	case DT_ALLOC_PERM_TILE_DYNLINK_CLUSTER:
		INC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_TILE_DYNLINK_CLUSTER, ActualSize);
		break;

	case DT_ALLOC_PERM_TILES:
		INC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_TILES, ActualSize);
		break;

	case DT_ALLOC_PERM_TILE_LINK_BUILDER:
		INC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_TILE_LINK_BUILDER, ActualSize);
		break;
		
	default:
		ensureMsgf(false, TEXT("Unsupported allocation hint %d"), Hint);
		break;
	}

	INC_DWORD_STAT_BY(STAT_NavigationMemory, ActualSize);
	INC_MEMORY_STAT_BY(STAT_Navigation_RecastMemory, ActualSize);
#endif // STATS
	return Result;
}

static void* RecastMalloc(int Size, rcAllocHint)
{
	LLM_SCOPE(ELLMTag::NavigationRecast);
	void* Result = FMemory::Malloc(uint32(Size));
#if STATS
	const SIZE_T ActualSize = FMemory::GetAllocSize(Result);
	INC_DWORD_STAT_BY(STAT_NavigationMemory, ActualSize);
	INC_MEMORY_STAT_BY(STAT_Navigation_RecastMemory, ActualSize);
#endif // STATS
	return Result;
}

static void DetourFree(void* Original, dtAllocHint Hint)
{
#if STATS
	const SIZE_T Size = FMemory::GetAllocSize(Original);

	switch (Hint)
	{
	case DT_ALLOC_TEMP:
		DEC_MEMORY_STAT_BY(STAT_Navigation_DetourTEMP, Size);
		break;

	case DT_ALLOC_PERM_AVOIDANCE:
		DEC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_AVOIDANCE, Size);
		break;

	case DT_ALLOC_PERM_CROWD:
		DEC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_CROWD, Size);
		break;

	case DT_ALLOC_PERM_LOOKUP:
		DEC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_LOOKUP, Size);
		break;

	case DT_ALLOC_PERM_NAVQUERY:
		DEC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_NAVQUERY, Size);
		break;

	case DT_ALLOC_PERM_NAVMESH:
		DEC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_NAVMESH, Size);
		break;

	case DT_ALLOC_PERM_NODE_POOL:
		DEC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_NODE_POOL, Size);
		break;

	case DT_ALLOC_PERM_PATH_CORRIDOR:
		DEC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_PATH_CORRIDOR, Size);
		break;

	case DT_ALLOC_PERM_PATH_QUEUE:
		DEC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_PATH_QUEUE, Size);
		break;

	case DT_ALLOC_PERM_PROXIMITY_GRID:
		DEC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_PROXY_GRID, Size);
		break;
	
	case DT_ALLOC_PERM_TILE_DATA:
		DEC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_TILE_DATA, Size);
		break;

	case DT_ALLOC_PERM_TILE_DYNLINK:
		DEC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_TILE_DYNLINK, Size);
		break;

	case DT_ALLOC_PERM_TILE_DYNLINK_CLUSTER:
		DEC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_TILE_DYNLINK_CLUSTER, Size);
		break;

	case DT_ALLOC_PERM_TILES:
		DEC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_TILES, Size);
		break;

	case DT_ALLOC_PERM_TILE_LINK_BUILDER:
		DEC_MEMORY_STAT_BY(STAT_Navigation_DetourPERM_TILE_LINK_BUILDER, Size);
		break;
		
	default:
		ensureMsgf(false, TEXT("Unsupported allocation hint %d"), Hint);
		break;
	}

	DEC_DWORD_STAT_BY(STAT_NavigationMemory, Size);
	DEC_MEMORY_STAT_BY(STAT_Navigation_RecastMemory, Size);
#endif // STATS

	FMemory::Free(Original);
}

static void RecastFree(void* Original)
{
#if STATS
	const SIZE_T Size = FMemory::GetAllocSize(Original);
	DEC_DWORD_STAT_BY(STAT_NavigationMemory, Size);
	DEC_MEMORY_STAT_BY(STAT_Navigation_RecastMemory, Size);
#endif // STATS
	FMemory::Free(Original);
}

static void DetourStatsPostAddTile(const dtMeshTile& TileAdded)
{
	FDetourTileLayout TileLayout(TileAdded);

	INC_MEMORY_STAT_BY(STAT_DetourTileMemory, TileLayout.TileSize);
	INC_MEMORY_STAT_BY(STAT_DetourTileMeshHeaderMemory, TileLayout.HeaderSize);
	INC_MEMORY_STAT_BY(STAT_DetourTileNavVertsMemory, TileLayout.VertsSize);
	INC_MEMORY_STAT_BY(STAT_DetourTileNavPolysMemory, TileLayout.PolysSize);
	INC_MEMORY_STAT_BY(STAT_DetourTileLinksMemory, TileLayout.LinksSize);
	INC_MEMORY_STAT_BY(STAT_DetourTileDetailMeshesMemory, TileLayout.DetailMeshesSize);
	INC_MEMORY_STAT_BY(STAT_DetourTileDetailVertsMemory, TileLayout.DetailVertsSize);
	INC_MEMORY_STAT_BY(STAT_DetourTileDetailTrisMemory, TileLayout.DetailTrisSize);
	INC_MEMORY_STAT_BY(STAT_DetourTileBVTreeMemory, TileLayout.BvTreeSize);
	INC_MEMORY_STAT_BY(STAT_DetourTileOffMeshConsMemory, TileLayout.OffMeshConsSize);
	INC_MEMORY_STAT_BY(STAT_DetourTileOffMeshSegsMemory, TileLayout.OffMeshSegsSize);
	INC_MEMORY_STAT_BY(STAT_DetourTileClustersMemory, TileLayout.ClustersSize);
	INC_MEMORY_STAT_BY(STAT_DetourTilePolyClustersMemory, TileLayout.PolyClustersSize);
}

static void DetourStatsPreRemoveTile(const dtMeshTile& TileRemoving)
{
	FDetourTileLayout TileLayout(TileRemoving);

	DEC_MEMORY_STAT_BY(STAT_DetourTileMemory, TileLayout.TileSize);
	DEC_MEMORY_STAT_BY(STAT_DetourTileMeshHeaderMemory, TileLayout.HeaderSize);
	DEC_MEMORY_STAT_BY(STAT_DetourTileNavVertsMemory, TileLayout.VertsSize);
	DEC_MEMORY_STAT_BY(STAT_DetourTileNavPolysMemory, TileLayout.PolysSize);
	DEC_MEMORY_STAT_BY(STAT_DetourTileLinksMemory, TileLayout.LinksSize);
	DEC_MEMORY_STAT_BY(STAT_DetourTileDetailMeshesMemory, TileLayout.DetailMeshesSize);
	DEC_MEMORY_STAT_BY(STAT_DetourTileDetailVertsMemory, TileLayout.DetailVertsSize);
	DEC_MEMORY_STAT_BY(STAT_DetourTileDetailTrisMemory, TileLayout.DetailTrisSize);
	DEC_MEMORY_STAT_BY(STAT_DetourTileBVTreeMemory, TileLayout.BvTreeSize);
	DEC_MEMORY_STAT_BY(STAT_DetourTileOffMeshConsMemory, TileLayout.OffMeshConsSize);
	DEC_MEMORY_STAT_BY(STAT_DetourTileOffMeshSegsMemory, TileLayout.OffMeshSegsSize);
	DEC_MEMORY_STAT_BY(STAT_DetourTileClustersMemory, TileLayout.ClustersSize);
	DEC_MEMORY_STAT_BY(STAT_DetourTilePolyClustersMemory, TileLayout.PolyClustersSize);
}

struct FRecastInitialSetup
{
	FRecastInitialSetup()
	{
		dtAllocSetCustom(DetourMalloc, DetourFree);
		rcAllocSetCustom(RecastMalloc, RecastFree);

		dtStatsSetCustom(DetourStatsPostAddTile, DetourStatsPreRemoveTile);
	}
};
static FRecastInitialSetup RecastSetup;



/****************************
 * helpers
 ****************************/

static void Unr2RecastVector(FVector const& V, FVector::FReal* R)
{
	// @todo: speed this up with axis swaps instead of a full transform?
	FVector const RecastV = Unreal2RecastPoint(V);
	R[0] = RecastV.X;
	R[1] = RecastV.Y;
	R[2] = RecastV.Z;
}

static void Unr2RecastSizeVector(FVector const& V, FVector::FReal* R)
{
	// @todo: speed this up with axis swaps instead of a full transform?
	FVector const RecastVAbs = Unreal2RecastPoint(V).GetAbs();
	R[0] = RecastVAbs.X;
	R[1] = RecastVAbs.Y;
	R[2] = RecastVAbs.Z;
}

static FVector Recast2UnrVector(FVector::FReal const* R)
{
	return Recast2UnrealPoint(R);
}

ENavigationQueryResult::Type DTStatusToNavQueryResult(dtStatus Status)
{
	// @todo look at possible dtStatus values (in DetourStatus.h), there's more data that can be retrieved from it

	// Partial paths are treated by Recast as Success while we treat as fail
	return dtStatusSucceed(Status) ? (dtStatusDetail(Status, DT_PARTIAL_RESULT) ? ENavigationQueryResult::Fail : ENavigationQueryResult::Success)
		: (dtStatusDetail(Status, DT_INVALID_PARAM) ? ENavigationQueryResult::Error : ENavigationQueryResult::Fail);
}

//----------------------------------------------------------------------//
// FRecastQueryFilter();
//----------------------------------------------------------------------//

FRecastQueryFilter::FRecastQueryFilter(bool bIsVirtual)
	: dtQueryFilter(bIsVirtual)
{
	SetExcludedArea(RECAST_NULL_AREA);
}

INavigationQueryFilterInterface* FRecastQueryFilter::CreateCopy() const 
{
	return new FRecastQueryFilter(*this);
}

void FRecastQueryFilter::SetIsVirtual(bool bIsVirtual)
{
	isVirtual = bIsVirtual;
}

void FRecastQueryFilter::Reset()
{
	// resetting just the cost data, we don't want to override the vf table like we did before (UE-95704)
	new(&data)dtQueryFilterData();
	SetExcludedArea(RECAST_NULL_AREA);
}

void FRecastQueryFilter::SetAreaCost(uint8 AreaType, float Cost)
{
	setAreaCost(AreaType, Cost);
}

void FRecastQueryFilter::SetFixedAreaEnteringCost(uint8 AreaType, float Cost) 
{
#if WITH_FIXED_AREA_ENTERING_COST
	setAreaFixedCost(AreaType, Cost);
#endif // WITH_FIXED_AREA_ENTERING_COST
}

void FRecastQueryFilter::SetExcludedArea(uint8 AreaType)
{
	setAreaCost(AreaType, DT_UNWALKABLE_POLY_COST);
}

void FRecastQueryFilter::SetAllAreaCosts(const float* CostArray, const int32 Count) 
{
	// @todo could get away with memcopying to m_areaCost, but it's private and would require a little hack
	// need to consider if it's wort a try (not sure there'll be any perf gain)
	if (Count > RECAST_MAX_AREAS)
	{
		UE_LOGF(LogNavigation, Warning, "FRecastQueryFilter: Trying to set cost to more areas than allowed! Discarding redundant values.");
	}

	const int32 ElementsCount = FPlatformMath::Min(Count, RECAST_MAX_AREAS);
	for (int32 i = 0; i < ElementsCount; ++i)
	{
		setAreaCost(i, CostArray[i]);
	}
}

void FRecastQueryFilter::GetAllAreaCosts(float* CostArray, float* FixedCostArray, const int32 Count) const
{
	const FVector::FReal* DetourCosts = getAllAreaCosts();
	const FVector::FReal* DetourFixedCosts = getAllFixedAreaCosts();
	const int32 NumItems = FMath::Min(Count, RECAST_MAX_AREAS);

	for (int i = 0; i < NumItems; ++i)
	{
		CostArray[i] = UE_REAL_TO_FLOAT_CLAMPED(DetourCosts[i]);
		FixedCostArray[i] = UE_REAL_TO_FLOAT_CLAMPED(DetourFixedCosts[i]);
	}
}

void FRecastQueryFilter::SetBacktrackingEnabled(const bool bBacktracking)
{
	setIsBacktracking(bBacktracking);
}

void FRecastQueryFilter::SetShouldIgnoreClosedNodes(const bool bIgnoreClosed)
{
	setShouldIgnoreClosedNodes(bIgnoreClosed);
}

bool FRecastQueryFilter::IsBacktrackingEnabled() const
{
	return getIsBacktracking();
}

float FRecastQueryFilter::GetHeuristicScale() const
{
	return UE_REAL_TO_FLOAT(getHeuristicScale());
}

bool FRecastQueryFilter::IsEqual(const INavigationQueryFilterInterface* Other) const
{
	// @NOTE: not type safe, should be changed when another filter type is introduced
	return FMemory::Memcmp(this, Other, sizeof(FRecastQueryFilter)) == 0; //-V598
}

void FRecastQueryFilter::SetIncludeFlags(uint16 Flags)
{
	setIncludeFlags(Flags);
}

uint16 FRecastQueryFilter::GetIncludeFlags() const
{
	return getIncludeFlags();
}

void FRecastQueryFilter::SetExcludeFlags(uint16 Flags)
{
	setExcludeFlags(Flags);
}

uint16 FRecastQueryFilter::GetExcludeFlags() const
{
	return getExcludeFlags();
}

bool FRecastSpeciaLinkFilter::isLinkAllowed(const uint64 UserId) const
{
	const INavLinkCustomInterface* CustomLink = NavSys ? NavSys->GetCustomLink(FNavLinkId(UserId)) : nullptr;
	return (CustomLink != NULL) && CustomLink->IsLinkPathfindingAllowed(CachedOwnerOb);
}

void FRecastSpeciaLinkFilter::initialize()
{
	CachedOwnerOb = SearchOwner.Get();
}

//----------------------------------------------------------------------//
// FPImplRecastNavMesh
//----------------------------------------------------------------------//

FPImplRecastNavMesh::FPImplRecastNavMesh(ARecastNavMesh* Owner)
	: NavMeshOwner(Owner)
	, DetourNavMesh(NULL)
{
	check(Owner && "Owner must never be NULL");

	INC_DWORD_STAT_BY( STAT_NavigationMemory
		, Owner->HasAnyFlags(RF_ClassDefaultObject) == false ? sizeof(*this) : 0 );
};

FPImplRecastNavMesh::~FPImplRecastNavMesh()
{
	ReleaseDetourNavMesh();

	DEC_DWORD_STAT_BY( STAT_NavigationMemory, sizeof(*this) );
};

void FPImplRecastNavMesh::ReleaseDetourNavMesh()
{
	// release navmesh only if we own it
	if (DetourNavMesh != nullptr)
	{
		dtFreeNavMesh(DetourNavMesh);
	}
	DetourNavMesh = nullptr;
	
	CompressedTileCacheLayers.Empty();

#if RECAST_INTERNAL_DEBUG_DATA
	DebugDataMap.Empty();
#endif
}

/**
 * Serialization.
 * @param Ar - The archive with which to serialize.
 * @returns true if serialization was successful.
 */
void FPImplRecastNavMesh::Serialize( FArchive& Ar, int32 NavMeshVersion )
{
	//@todo: How to handle loading nav meshes saved w/ recast when recast isn't present????

	if (!Ar.IsLoading() && DetourNavMesh == NULL)
	{
		// nothing to write
		return;
	}
	
	// All we really need to do is read/write the data blob for each tile

	if (Ar.IsLoading())
	{
		// allocate the navmesh object
		ReleaseDetourNavMesh();
		DetourNavMesh = dtAllocNavMesh();

		if (DetourNavMesh == NULL)
		{
			UE_VLOG(NavMeshOwner, LogNavigation, Error, TEXT("Failed to allocate Recast navmesh"));
		}
	}

	int32 NumTiles = 0;
	TArray<FNavTileRef> TilesToSave;

	if (Ar.IsSaving() && !Ar.IsTransacting() && !UE::Transaction::DiffUtil::IsGeneratingDiffableObject(Ar)) // Do not save tiles during transactions (i.e. undo/redo)
	{
		TilesToSave.Reserve(DetourNavMesh->getMaxTiles());
		
		const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<const UNavigationSystemV1>(NavMeshOwner->GetWorld());

		if (NavMeshOwner->bIsWorldPartitioned)
		{
			// Ignore (leave TilesToSave empty so no tiles are saved).
			// Navmesh data are stored in ANavigationDataChunkActor.
			UE_LOGF(LogNavigation, VeryVerbose, "%ls Ar.IsSaving() no tiles are being saved because bIsWorldPartitioned=true in %ls.", ANSI_TO_TCHAR(__FUNCTION__), *NavMeshOwner->GetFullName());
		}
		else
		{
			// Need to keep the check !IsRunningCommandlet() for the case where maps are cooked and saved from UCookCommandlet.
			// In that flow the nav bounds are not set (no bounds means no tiles to save and the navmesh would be saved without tiles).
			// This flow would benefit to be revisited since navmesh serialization should not be different whether it was run or not by a commandlet.
			// Fixes missing navmesh regression (UE-103604).
			if (NavMeshOwner->SupportsStreaming() && NavSys && !IsRunningCommandlet())
			{
				// We save only tiles that belongs to this level
				GetNavMeshTilesIn(NavMeshOwner->GetNavigableBoundsInLevel(NavMeshOwner->GetLevel()), TilesToSave);
			}
			else
			{
				// Otherwise all valid tiles
				dtNavMesh const* ConstNavMesh = DetourNavMesh;
				for (int i = 0; i < ConstNavMesh->getMaxTiles(); ++i)
				{
					const dtMeshTile* Tile = ConstNavMesh->getTile(i);
					if (Tile != NULL && Tile->header != NULL && Tile->dataSize > 0)
					{
						FNavTileRef TileRef(ConstNavMesh->getTileRef(Tile));
						TilesToSave.Add(TileRef);
					}
				}
			}
		}
		
		NumTiles = TilesToSave.Num();
		UE_LOGF(LogNavigation, VeryVerbose, "%s Ar.IsSaving() %i tiles in %ls.", __FUNCTION__, NumTiles, *NavMeshOwner->GetFullName());
	}

	Ar << NumTiles;

	dtNavMeshParams Params = *DetourNavMesh->getParams();
	Ar << Params.orig[0] << Params.orig[1] << Params.orig[2];
	Ar << Params.tileWidth;				///< The width of each tile. (Along the x-axis.)
	Ar << Params.tileHeight;			///< The height of each tile. (Along the z-axis.)
	Ar << Params.maxTiles;				///< The maximum number of tiles the navigation mesh can contain.
	UE_LOGF(LogNavigation, VeryVerbose, "%s maxTiles: %i", __FUNCTION__, Params.maxTiles);
	Ar << Params.maxPolys;

	if (NavMeshOwner->NavMeshVersion >= NAVMESHVER_TILE_RESOLUTIONS)
	{
		Ar << Params.walkableHeight;
		Ar << Params.walkableRadius;
		Ar << Params.walkableClimb;
		Ar << Params.resolutionParams[(uint8)ENavigationDataResolution::Low].bvQuantFactor;
		Ar << Params.resolutionParams[(uint8)ENavigationDataResolution::Default].bvQuantFactor;
		Ar << Params.resolutionParams[(uint8)ENavigationDataResolution::High].bvQuantFactor;
	}
	else if (NavMeshOwner->NavMeshVersion >= NAVMESHVER_OPTIM_FIX_SERIALIZE_PARAMS)
	{
		// Load previous version navmesh data into new struct
		Ar << Params.walkableHeight;
		Ar << Params.walkableRadius;
		Ar << Params.walkableClimb;
		Ar << Params.resolutionParams[(uint8)ENavigationDataResolution::Default].bvQuantFactor;
		if (Ar.IsLoading())
		{
			const dtReal DefaultQuantFactor = Params.resolutionParams[(uint8)ENavigationDataResolution::Default].bvQuantFactor; 
			Params.resolutionParams[(uint8)ENavigationDataResolution::Low].bvQuantFactor = DefaultQuantFactor;
			Params.resolutionParams[(uint8)ENavigationDataResolution::High].bvQuantFactor = DefaultQuantFactor;
		}
	}
	else
	{
		Params.walkableHeight = NavMeshOwner->AgentHeight;
		Params.walkableRadius = NavMeshOwner->AgentRadius;
		Params.walkableClimb = NavMeshOwner->GetAgentMaxStepHeight(ENavigationDataResolution::Default);
		const float DefaultQuantFactor =  1.f / NavMeshOwner->GetCellSize(ENavigationDataResolution::Default);
		for(uint8 Index = 0; Index < (uint8)ENavigationDataResolution::MAX; Index++)
		{
			Params.resolutionParams[Index].bvQuantFactor = DefaultQuantFactor;	
		}
	}

	if (Ar.IsLoading())
	{
		// at this point we can tell whether navmesh being loaded is in line
		// ARecastNavMesh's params. If not, just skip it.
		// assumes tiles are rectangular
		
		float DefaultCellSize = NavMeshOwner->GetCellSize(ENavigationDataResolution::Default);

		// Detect mismatch with quantization factor
		bool bQuantizationFactorMismatch = false;
		if (NavMeshOwner->NavMeshVersion >= NAVMESHVER_TILE_RESOLUTIONS)
		{
			for (uint8 Index = 0; Index < (uint8)ENavigationDataResolution::MAX; Index++)
			{
				const double expectedQuantFactor =  (NavMeshOwner->NavMeshResolutionParams[Index].CellSize != 0.f) ? (1.f / NavMeshOwner->NavMeshResolutionParams[Index].CellSize) : 0.f;
				if (!FMath::IsNearlyEqual(Params.resolutionParams[(uint8)Index].bvQuantFactor, expectedQuantFactor))
				{
					UE_LOGF(LogNavigation, Warning, "%s Navmesh %ls will be loaded empty because it has been built with different navmesh settings than the current settings. "
						"If possible, this will trigger a navmesh rebuild. "
						"The mismatch was detected on stored quantization factor %f (expected: %f) for CellSize %f (resolution: %i).",
						__FUNCTION__,
						*NavMeshOwner->GetName(),
						Params.resolutionParams[(uint8)Index].bvQuantFactor,
						expectedQuantFactor,
						NavMeshOwner->NavMeshResolutionParams[Index].CellSize,
						Index);
				
					bQuantizationFactorMismatch = true;
				}
			}
		}

#if WITH_EDITORONLY_DATA
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (NavMeshVersion < NAVMESHVER_TILE_RESOLUTIONS)
		{
			// For backward compatibility, read original CellSize value.
			// In ARecastNavMesh::PostLoad(), cell sizes for the different resolutions are set to the old CellSize value but it occurs later (PostLoad).
			// This why we explicitly need to read CellSize for older versions here.
			DefaultCellSize = NavMeshOwner->CellSize;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

		const FVector::FReal ActorsTileSize = static_cast<FVector::FReal>(FMath::TruncToInt(NavMeshOwner->TileSizeUU / DefaultCellSize)) * DefaultCellSize;
		const bool bTileSizeMismatch = ActorsTileSize != Params.tileWidth;
		if (bTileSizeMismatch)
		{
			UE_LOGF(LogNavigation, Warning, "%hs Navmesh %ls will be loaded empty because it has been built with different navmesh settings than the current settings. "
				"If possible, this will trigger a navmesh rebuild, else you should rebuild it manually."
				"The mismatch was detected on tile size %f (expected: %f) for CellSize %f, NavmeshOwnerTileSize %f.",
				__FUNCTION__,
				*NavMeshOwner->GetName(),
				Params.tileWidth,
				ActorsTileSize,
				DefaultCellSize,
				NavMeshOwner->TileSizeUU);
		}

		if (bTileSizeMismatch || bQuantizationFactorMismatch)
		{
			// just move archive position
			ReleaseDetourNavMesh();

			for (int i = 0; i < NumTiles; ++i)
			{
				dtTileRef TileRef = MAX_uint64;
				int32 TileDataSize = 0;
				Ar << TileRef << TileDataSize;

				if (TileRef == MAX_uint64 || TileDataSize == 0)
				{
					continue;
				}

				unsigned char* TileData = NULL;
				TileDataSize = 0;
				SerializeRecastMeshTile(Ar, NavMeshVersion, TileData, TileDataSize);
				if (TileData != NULL)
				{
					dtMeshHeader* const TileHeader = (dtMeshHeader*)TileData;
					dtFree(TileHeader, DT_ALLOC_PERM_TILE_DATA);

					unsigned char* ComressedTileData = NULL;
					int32 CompressedTileDataSize = 0;
					SerializeCompressedTileCacheData(Ar, NavMeshVersion, ComressedTileData, CompressedTileDataSize);
					dtFree(ComressedTileData, DT_ALLOC_PERM_TILE_DATA);
				}
			}
		}
		else
		{
			// regular loading
			dtStatus Status = DetourNavMesh->init(&Params);
			if (dtStatusFailed(Status))
			{
				UE_VLOG(NavMeshOwner, LogNavigation, Error, TEXT("Failed to initialize NavMesh"));
			}
			
			NavMeshOwner->bHasNoTileData = (NumTiles == 0);	

			// Defer off-mesh point link snapping until after all tiles are loaded
			// and area cost order is applied. Without this, links using
			// bSnapToCheapestArea snap by distance instead of cost because
			// m_areaCostOrder hasn't been initialized yet.
			DetourNavMesh->setDeferOffMeshLinking(true);

#if WITH_NAVMESH_SEGMENT_LINKS			
			// Store the tileRefs when iterating over the tile list.
			// These will be processed again to add the segment link connections.
			TArray<dtTileRef> AddedTiles;
			AddedTiles.Reserve(NumTiles);
#endif // WITH_NAVMESH_SEGMENT_LINKS			

			for (int i = 0; i < NumTiles; ++i)
			{
				dtTileRef TileRef = MAX_uint64;
				int32 TileDataSize = 0;
				Ar << TileRef << TileDataSize;

				if (TileRef == MAX_uint64 || TileDataSize == 0)
				{
					continue;
				}
				
				unsigned char* TileData = NULL;
				TileDataSize = 0;
				SerializeRecastMeshTile(Ar, NavMeshVersion, TileData, TileDataSize);

				if (TileData != NULL)
				{
#if WITH_NAVMESH_SEGMENT_LINKS					
					AddedTiles.Add(TileRef);
#endif

					dtMeshHeader* const TileHeader = (dtMeshHeader*)TileData;
					Status = DetourNavMesh->addTile(TileData, TileDataSize, DT_TILE_FREE_DATA, TileRef, NULL);
					if (dtStatusDetail(Status, DT_OUT_OF_MEMORY))
					{
						UE_LOGF(LogNavigation, Warning, 
							"%s Failed to add tile (%d,%d:%d), %d tile limit reached in %ls. Increase AverageLayersPerTile or, if using FixedTilePoolSize, increase the TilePoolSize or use bigger tiles.",
							__FUNCTION__, TileHeader->x, TileHeader->y, TileHeader->layer, DetourNavMesh->getMaxTiles(), *NavMeshOwner->GetFullName());
					}

					// Serialize compressed tile cache layer
					uint8* ComressedTileData = nullptr;
					int32 CompressedTileDataSize = 0;
					SerializeCompressedTileCacheData(Ar, NavMeshVersion, ComressedTileData, CompressedTileDataSize);
					
					if (CompressedTileDataSize > 0)
					{
						AddTileCacheLayer(TileHeader->x, TileHeader->y, TileHeader->layer,
							FNavMeshTileData(ComressedTileData, CompressedTileDataSize, TileHeader->layer, Recast2UnrealBox(TileHeader->bmin, TileHeader->bmax)));
					}
				}
			}

#if WITH_NAVMESH_SEGMENT_LINKS
			// Create segment link connections now that all the tiles have been loaded.
			for (int i = 0; i < AddedTiles.Num(); ++i)
			{
				ProcessSegmentLinksForTile(FNavTileRef(AddedTiles[i]));
			}
#endif // WITH_NAVMESH_SEGMENT_LINKS
		}
	}
	else if (Ar.IsSaving())
	{
		const bool bSupportsRuntimeGeneration = NavMeshOwner->SupportsRuntimeGeneration();
		dtNavMesh const* ConstNavMesh = DetourNavMesh;
		
		for (FNavTileRef TileRefToSave : TilesToSave)
		{
			const dtMeshTile* Tile = ConstNavMesh->getTileByRef(static_cast<dtTileRef>(TileRefToSave));
			dtTileRef TileRef = ConstNavMesh->getTileRef(Tile);
			int32 TileDataSize = Tile->dataSize;
			Ar << TileRef << TileDataSize;

			unsigned char* TileData = Tile->data;
			SerializeRecastMeshTile(Ar, NavMeshVersion, TileData, TileDataSize);

			// Serialize compressed tile cache layer only if navmesh requires it
			{
				FNavMeshTileData TileCacheLayer;
				uint8* CompressedData = nullptr;
				int32 CompressedDataSize = 0;
				if (bSupportsRuntimeGeneration)
				{
					TileCacheLayer = GetTileCacheLayer(Tile->header->x, Tile->header->y, Tile->header->layer);
					CompressedData = TileCacheLayer.GetDataSafe();
					CompressedDataSize = TileCacheLayer.DataSize;
				}
				
				SerializeCompressedTileCacheData(Ar, NavMeshVersion, CompressedData, CompressedDataSize);
			}
		}
	}
}

void FPImplRecastNavMesh::SerializeRecastMeshTile(FArchive& Ar, int32 NavMeshVersion, unsigned char*& TileData, int32& TileDataSize)
{
	// The strategy here is to serialize the data blob that is passed into addTile()
	// @see dtCreateNavMeshData() for details on how this data is laid out

	FDetourTileSizeInfo SizeInfo;

	if (Ar.IsSaving())
	{
		// fill in data to write
		dtMeshHeader* const H = (dtMeshHeader*)TileData;
		SizeInfo.VertCount = H->vertCount;
		SizeInfo.PolyCount = H->polyCount;
		SizeInfo.MaxLinkCount = H->maxLinkCount;
		SizeInfo.DetailMeshCount = H->detailMeshCount;
		SizeInfo.DetailVertCount = H->detailVertCount;
		SizeInfo.DetailTriCount = H->detailTriCount;
		SizeInfo.BvNodeCount = H->bvNodeCount;
		SizeInfo.OffMeshConCount = H->offMeshConCount;
#if WITH_NAVMESH_SEGMENT_LINKS
		SizeInfo.OffMeshSegConCount = H->offMeshSegConCount;
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if WITH_NAVMESH_CLUSTER_LINKS
		SizeInfo.ClusterCount = H->clusterCount;
#endif // WITH_NAVMESH_CLUSTER_LINKS
	}

	Ar << SizeInfo.VertCount << SizeInfo.PolyCount << SizeInfo.MaxLinkCount ;
	Ar << SizeInfo.DetailMeshCount << SizeInfo.DetailVertCount << SizeInfo.DetailTriCount;
	Ar << SizeInfo.BvNodeCount << SizeInfo.OffMeshConCount << SizeInfo.OffMeshSegConCount;
	Ar << SizeInfo.ClusterCount;
	SizeInfo.OffMeshBase = SizeInfo.DetailMeshCount;
	const int32 polyClusterCount = SizeInfo.OffMeshBase;

	// calc sizes for our data so we know how much to allocate and where to read/write stuff
	// note this may not match the on-disk size or the in-memory size on the machine that generated that data

	// In-memory tile layout always uses latest vertex format (dtTileVert).
	FDetourTileLayout TileLayout(SizeInfo);

	if (Ar.IsLoading())
	{
		check(TileData == NULL);

		TileDataSize = TileLayout.TileSize;

		TileData = (unsigned char*)dtAlloc(sizeof(unsigned char)*TileDataSize, DT_ALLOC_PERM_TILE_DATA);
		if (!TileData)
		{
			UE_LOGF(LogNavigation, Error, "Failed to alloc navmesh tile");
		}
		else
		{
			FMemory::Memset(TileData, 0, TileDataSize);
		}
	}
	else if (Ar.IsSaving())
	{
		// TileData and TileDataSize should already be set, verify
		check(TileData != NULL);
		check(TileLayout.TileSize == TileDataSize);
	}

	if (TileData != NULL)
	{
		// sort out where various data types do/will live
		unsigned char* d = TileData;
		dtMeshHeader* Header = (dtMeshHeader*)d; d += TileLayout.HeaderSize;
		dtTileVert* NavVerts = (dtTileVert*)d; d += TileLayout.VertsSize;
		dtPoly* NavPolys = (dtPoly*)d; d += TileLayout.PolysSize;
		d += TileLayout.LinksSize;			// @fixme, are links autogenerated on addTile?
		dtPolyDetail* DetailMeshes = (dtPolyDetail*)d; d += TileLayout.DetailMeshesSize;
		dtTileVert* DetailVerts = (dtTileVert*)d; d += TileLayout.DetailVertsSize;
		unsigned char* DetailTris = (unsigned char*)d; d += TileLayout.DetailTrisSize;
		dtBVNode* BVTree = (dtBVNode*)d; d += TileLayout.BvTreeSize;
		dtOffMeshConnection* OffMeshCons = (dtOffMeshConnection*)d; d += TileLayout.OffMeshConsSize;

#if WITH_NAVMESH_SEGMENT_LINKS
		dtOffMeshSegmentConnection* OffMeshSegs = (dtOffMeshSegmentConnection*)d; d += TileLayout.OffMeshSegsSize;
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if WITH_NAVMESH_CLUSTER_LINKS
		dtCluster* Clusters = (dtCluster*)d; d += TileLayout.ClustersSize;
		unsigned short* PolyClusters = (unsigned short*)d; d += TileLayout.PolyClustersSize;
#endif // WITH_NAVMESH_CLUSTER_LINKS

		check(d==(TileData + TileDataSize));

		// now serialize the data in the blob!

		// header
		Ar << Header->version << Header->x << Header->y;
		Ar << Header->layer << Header->polyCount << Header->vertCount;
		Ar << Header->maxLinkCount << Header->detailMeshCount << Header->detailVertCount << Header->detailTriCount;
		Ar << Header->bvNodeCount << Header->offMeshConCount<< Header->offMeshBase;
		
		if (NavMeshVersion >= NAVMESHVER_TILE_RESOLUTIONS)
		{
			Ar << Header->resolution;
		}
		
		Ar << Header->bmin[0] << Header->bmin[1] << Header->bmin[2];
		Ar << Header->bmax[0] << Header->bmax[1] << Header->bmax[2];
#if WITH_NAVMESH_CLUSTER_LINKS
		Ar << Header->clusterCount;
#else
		unsigned short DummyClusterCount = 0;
		Ar << DummyClusterCount;
#endif // WITH_NAVMESH_CLUSTER_LINKS

#if WITH_NAVMESH_SEGMENT_LINKS
		Ar << Header->offMeshSegConCount << Header->offMeshSegPolyBase << Header->offMeshSegVertBase;
#else
		unsigned short DummySegmentInt = 0;
		Ar << DummySegmentInt << DummySegmentInt << DummySegmentInt;
#endif // WITH_NAVMESH_SEGMENT_LINKS

		// mesh and offmesh connection vertices
		if (NavMeshVersion < NAVMESHVER_TILEVERT_ENCODING)
		{
			// Old format is in world-space (FVector::FReal * 3 per vertex).
			if (Ar.IsLoading())
			{
				// Load old format and convert to tile-local dtTileVert.
				for (int32 VertIdx = 0; VertIdx < SizeInfo.VertCount; ++VertIdx)
				{
					FVector::FReal wx, wy, wz;
					Ar << wx;
					Ar << wy;
					Ar << wz;
					const dtReal worldPos[3] = {(dtReal)wx, (dtReal)wy, (dtReal)wz};
					NavVerts[VertIdx].setFromWorld(Header->bmin, worldPos);
				}

				// Update navigation tile data version to latest so it becomes readable.
				Header->version = DT_NAVMESH_VERSION;
			}
			else
			{
				// Save in old format: convert from tile-local dtTileVert to world-space.
				for (int32 VertIdx = 0; VertIdx < SizeInfo.VertCount; ++VertIdx)
				{
					dtReal worldPos[3];
					NavVerts[VertIdx].toWorld(Header->bmin, worldPos);
					FVector::FReal wx = (FVector::FReal)worldPos[0];
					FVector::FReal wy = (FVector::FReal)worldPos[1];
					FVector::FReal wz = (FVector::FReal)worldPos[2];
					Ar << wx;
					Ar << wy;
					Ar << wz;
				}
			}
		}
		else
		{
			// New format: tile-local floats stored directly in dtTileVert.
			for (int32 VertIdx = 0; VertIdx < SizeInfo.VertCount; ++VertIdx)
			{
				Ar << NavVerts[VertIdx].x;
				Ar << NavVerts[VertIdx].y;
				Ar << NavVerts[VertIdx].z;
			}
		}

		// mesh and off-mesh connection polys
		for (int32 PolyIdx=0; PolyIdx < SizeInfo.PolyCount; ++PolyIdx)
		{
			dtPoly& P = NavPolys[PolyIdx];
			Ar << P.firstLink;

			for (uint32 VertIdx=0; VertIdx < DT_VERTS_PER_POLYGON; ++VertIdx)
			{
				Ar << P.verts[VertIdx];
			}
			for (uint32 NeiIdx=0; NeiIdx < DT_VERTS_PER_POLYGON; ++NeiIdx)
			{
				Ar << P.neis[NeiIdx];
			}
			Ar << P.flags << P.vertCount << P.areaAndtype;
		}

		// serialize detail meshes
		for (int32 MeshIdx=0; MeshIdx < SizeInfo.DetailMeshCount; ++MeshIdx)
		{
			dtPolyDetail& DM = DetailMeshes[MeshIdx];
			Ar << DM.vertBase << DM.triBase << DM.vertCount << DM.triCount;
		}

		// serialize detail verts
		if (NavMeshVersion < NAVMESHVER_TILEVERT_ENCODING)
		{
			// Old format is in world-space (FVector::FReal * 3 per vertex).
			if (Ar.IsLoading())
			{
				// Load old format and convert to tile-local dtTileVert.
				for (int32 VertIdx = 0; VertIdx < SizeInfo.DetailVertCount; ++VertIdx)
				{
					FVector::FReal wx, wy, wz;
					Ar << wx;
					Ar << wy;
					Ar << wz;
					const dtReal worldPos[3] = {(dtReal)wx, (dtReal)wy, (dtReal)wz};
					DetailVerts[VertIdx].setFromWorld(Header->bmin, worldPos);
				}
			}
			else
			{
				// Save in old format: convert from tile-local dtTileVert to world-space.
				for (int32 VertIdx = 0; VertIdx < SizeInfo.DetailVertCount; ++VertIdx)
				{
					dtReal worldPos[3];
					DetailVerts[VertIdx].toWorld(Header->bmin, worldPos);
					FVector::FReal wx = (FVector::FReal)worldPos[0];
					FVector::FReal wy = (FVector::FReal)worldPos[1];
					FVector::FReal wz = (FVector::FReal)worldPos[2];
					Ar << wx;
					Ar << wy;
					Ar << wz;
				}
			}
		}
		else
		{
			// New format: tile-local floats stored directly in dtTileVert.
			for (int32 VertIdx = 0; VertIdx < SizeInfo.DetailVertCount; ++VertIdx)
			{
				Ar << DetailVerts[VertIdx].x;
				Ar << DetailVerts[VertIdx].y;
				Ar << DetailVerts[VertIdx].z;
			}
		}

		// serialize detail tris (4 one-byte indices per tri)
		{
			unsigned char* V = DetailTris;
			for (int32 TriIdx=0; TriIdx < SizeInfo.DetailTriCount; ++TriIdx)
			{
				Ar << *V; V++;
				Ar << *V; V++;
				Ar << *V; V++;
				Ar << *V; V++;
			}
		}

		// serialize BV tree
		for (int32 NodeIdx=0; NodeIdx < SizeInfo.BvNodeCount; ++NodeIdx)
		{
			dtBVNode& Node = BVTree[NodeIdx];
			Ar << Node.bmin[0] << Node.bmin[1] << Node.bmin[2];
			Ar << Node.bmax[0] << Node.bmax[1] << Node.bmax[2];
			Ar << Node.i;
		}

		// serialize off-mesh connections
		for (int32 ConnIdx=0; ConnIdx < SizeInfo.OffMeshConCount; ++ConnIdx)
		{
			dtOffMeshConnection& Conn = OffMeshCons[ConnIdx];
			Ar << Conn.pos[0] << Conn.pos[1] << Conn.pos[2] << Conn.pos[3] << Conn.pos[4] << Conn.pos[5];
			Ar << Conn.rad << Conn.poly << Conn.flags << Conn.side;

			if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::NavigationLinkID32To64)
			{
				uint32 Id;

				Ar << Id;
				Conn.userId = Id;
			}
			else
			{
				Ar << Conn.userId;
			}
		}

		if (NavMeshVersion >= NAVMESHVER_OFFMESH_HEIGHT_BUG)
		{
			for (int32 ConnIdx = 0; ConnIdx < SizeInfo.OffMeshConCount; ++ConnIdx)
			{
				dtOffMeshConnection& Conn = OffMeshCons[ConnIdx];
				Ar << Conn.height;
			}
		}

		for (int32 SegIdx=0; SegIdx < SizeInfo.OffMeshSegConCount; ++SegIdx)
		{
#if WITH_NAVMESH_SEGMENT_LINKS
			dtOffMeshSegmentConnection& Seg = OffMeshSegs[SegIdx];
			Ar << Seg.startA[0] << Seg.startA[1] << Seg.startA[2];
			Ar << Seg.startB[0] << Seg.startB[1] << Seg.startB[2];
			Ar << Seg.endA[0] << Seg.endA[1] << Seg.endA[2];
			Ar << Seg.endB[0] << Seg.endB[1] << Seg.endB[2];
			Ar << Seg.rad << Seg.firstPoly << Seg.npolys << Seg.flags << Seg.userId;
#else
			FVector::FReal DummySegmentConReal;
			unsigned int DummySegmentConInt;
			unsigned short DummySegmentConShort;
			unsigned char DummySegmentConChar;
			Ar << DummySegmentConReal << DummySegmentConReal << DummySegmentConReal; // real startA[3];	///< Start point of segment A
			Ar << DummySegmentConReal << DummySegmentConReal << DummySegmentConReal; // real endA[3];	///< End point of segment A
			Ar << DummySegmentConReal << DummySegmentConReal << DummySegmentConReal; // real startB[3];	///< Start point of segment B
			Ar << DummySegmentConReal << DummySegmentConReal << DummySegmentConReal; // real endB[3];	///< End point of segment B
			Ar << DummySegmentConReal << DummySegmentConShort << DummySegmentConChar << DummySegmentConChar << DummySegmentConInt; // real rad, short firstPoly, char npolys, char flags, int userId
#endif // WITH_NAVMESH_SEGMENT_LINKS
		}

		// serialize clusters
		for (int32 CIdx = 0; CIdx < SizeInfo.ClusterCount; ++CIdx)
		{
#if WITH_NAVMESH_CLUSTER_LINKS
			dtCluster& Cluster = Clusters[CIdx];
			Ar << Cluster.center[0] << Cluster.center[1] << Cluster.center[2];
#else
			FVector::FReal DummyCluster[3];
			Ar << DummyCluster[0] << DummyCluster[1] << DummyCluster[2];
#endif // WITH_NAVMESH_CLUSTER_LINKS
		}

		// serialize poly clusters map
		{
#if WITH_NAVMESH_CLUSTER_LINKS
			unsigned short* C = PolyClusters;
			for (int32 PolyClusterIdx = 0; PolyClusterIdx < polyClusterCount; ++PolyClusterIdx)
			{
				Ar << *C; C++;
			}
#else
			unsigned short DummyPolyCluster = 0;
			for (int32 PolyClusterIdx = 0; PolyClusterIdx < polyClusterCount; ++PolyClusterIdx)
			{
				Ar << DummyPolyCluster;
			}
#endif // WITH_NAVMESH_CLUSTER_LINKS
		}
	}
}

void FPImplRecastNavMesh::SerializeCompressedTileCacheData(FArchive& Ar, int32 NavMeshVersion, unsigned char*& CompressedData, int32& CompressedDataSize)
{
	constexpr int32 EmptyDataValue = -1;

	// Note when saving the CompressedDataSize is either 0 or it must be big enough to include the size of the uncompressed dtTileCacheLayerHeader.
	checkf((Ar.IsSaving() == false) || CompressedDataSize == 0 || CompressedDataSize >= dtAlign(sizeof(dtTileCacheLayerHeader)), TEXT("When saving CompressedDataSize must either be zero or large enough to hold dtTileCacheLayerHeader!"));
	checkf((Ar.IsSaving() == false) || CompressedDataSize == 0 || CompressedData != nullptr, TEXT("When saving CompressedDataSize must either be zero or CompressedData must be != nullptr"));

	if (Ar.IsLoading())
	{
		// Initialize to 0 if we are loading as this is calculated and used during processing.
		CompressedDataSize = 0;
	}
	
	// There are 3 cases that need to be serialized here, no header no compresseed data, header only no compressed data or header and compressed data.
	// CompressedDataSizeNoHeader == NoHeaderValue, indicates we have no header and no compressed data.
	// CompressedDataSizeNoHeader == 0, indicates we have a header only no compressed data.
	// CompressedDataSizeNoHeader > 0, indicates we have a header and compressed data.
	int32 CompressedDataSizeNoHeader = 0;
	if (Ar.IsSaving())
	{
		// Handle invalid CompressedDataSize ( i.e. CompressedDataSize > 0 and CompressedDataSize < dtAlign(sizeof(dtTileCacheLayerHeader))
		// as well as valid CompressedDataSize == 0, to make this function at least as robust as it was.
		if (CompressedDataSize < dtAlign(sizeof(dtTileCacheLayerHeader)))
		{
			CompressedDataSizeNoHeader = EmptyDataValue;
		}
		else
		{
			CompressedDataSizeNoHeader = CompressedDataSize - dtAlign(sizeof(dtTileCacheLayerHeader));
		}
	}

	Ar << CompressedDataSizeNoHeader;

	const bool bHasHeader = CompressedDataSizeNoHeader >= 0;

	if (!bHasHeader)
	{
		return;
	}

	if (Ar.IsLoading())
	{
		CompressedDataSize = CompressedDataSizeNoHeader + dtAlign(sizeof(dtTileCacheLayerHeader));
		CompressedData = (unsigned char*)dtAlloc(sizeof(unsigned char)*CompressedDataSize, DT_ALLOC_PERM_TILE_DATA);
		if (!CompressedData)
		{
			UE_LOGF(LogNavigation, Error, "Failed to alloc tile compressed data");
		}
		else
		{
			FMemory::Memset(CompressedData, 0, CompressedDataSize);
		}
	}

	check(CompressedData != nullptr);

	// Serialize dtTileCacheLayerHeader by hand so we can account for the FReals always being serialized as doubles
	dtTileCacheLayerHeader* Header = (dtTileCacheLayerHeader*)CompressedData;
	Ar << Header->version;
	Ar << Header->tx;
	Ar << Header->ty;
	Ar << Header->tlayer;
	for (int i = 0; i < 3; ++i)
	{
		Ar << Header->bmin[i];
		Ar << Header->bmax[i];
	}
	Ar << Header->hmin;	// @todo: remove
	Ar << Header->hmax;
	Ar << Header->width;
	Ar << Header->height;
	Ar << Header->minx;
	Ar << Header->maxx;
	Ar << Header->miny;
	Ar << Header->maxy;

	if (CompressedDataSizeNoHeader > 0)
	{
		// @todo this does not appear to be accounting for potential endian differences!
		Ar.Serialize(CompressedData + dtAlign(sizeof(dtTileCacheLayerHeader)), CompressedDataSizeNoHeader);
	}
}

void FPImplRecastNavMesh::SetRecastMesh(dtNavMesh* NavMesh)
{
	if (NavMesh == DetourNavMesh)
	{
		return;
	}

	ReleaseDetourNavMesh();
	DetourNavMesh = NavMesh;

	if (NavMeshOwner)
	{
		NavMeshOwner->UpdateNavObject();
	}

	// reapply area sort order in new detour navmesh
	OnAreaCostChanged();
}

void FPImplRecastNavMesh::Raycast(const FVector& StartLoc, const FVector& EndLoc, const FNavigationQueryFilter& InQueryFilter, const UObject* Owner, 
	ARecastNavMesh::FRaycastResult& RaycastResult, NavNodeRef StartNode) const
{
	if (DetourNavMesh == NULL || NavMeshOwner == NULL)
	{
		return;
	}

	const dtQueryFilter* QueryFilter = ((const FRecastQueryFilter*)(InQueryFilter.GetImplementation()))->GetAsDetourQueryFilter();
	if (QueryFilter == NULL)
	{
		UE_VLOG(NavMeshOwner, LogNavigation, Warning, TEXT("FPImplRecastNavMesh::Raycast failing due to QueryFilter == NULL"));
		return;
	}

	FRecastSpeciaLinkFilter LinkFilter(FNavigationSystem::GetCurrent<UNavigationSystemV1>(NavMeshOwner->GetWorld()), Owner);
	INITIALIZE_NAVQUERY(NavQuery, InQueryFilter.GetMaxSearchNodes(), LinkFilter);

	const FVector NavExtent = NavMeshOwner->GetModifiedQueryExtent(NavMeshOwner->GetDefaultQueryExtent());
	const FVector::FReal Extent[3] = { NavExtent.X, NavExtent.Z, NavExtent.Y };

	const FVector RecastStart = Unreal2RecastPoint(StartLoc);
	const FVector RecastEnd = Unreal2RecastPoint(EndLoc);

	if (StartNode == INVALID_NAVNODEREF)
	{
		NavQuery.findNearestContainingPoly(&RecastStart.X, Extent, QueryFilter, &StartNode, NULL);
	}

	NavNodeRef EndNode = INVALID_NAVNODEREF;
	NavQuery.findNearestContainingPoly(&RecastEnd.X, Extent, QueryFilter, &EndNode, NULL);

	if (StartNode != INVALID_NAVNODEREF)
	{
		FVector::FReal RecastHitNormal[3];

		const dtStatus RaycastStatus = NavQuery.raycast(StartNode, &RecastStart.X, &RecastEnd.X
			, QueryFilter, &RaycastResult.HitTime, RecastHitNormal
			, RaycastResult.CorridorPolys, &RaycastResult.CorridorPolysCount, RaycastResult.GetMaxCorridorSize());

		RaycastResult.HitNormal = Recast2UnrVector(RecastHitNormal);
		RaycastResult.bIsRaycastEndInCorridor = dtStatusSucceed(RaycastStatus) && (RaycastResult.GetLastNodeRef() == EndNode);
	}
	else
	{
		RaycastResult.HitTime = 0.f;
		RaycastResult.HitNormal = (StartLoc - EndLoc).GetSafeNormal();
	}
}

// @TODONAV
ENavigationQueryResult::Type FPImplRecastNavMesh::FindPath(const FVector& StartLoc, const FVector& EndLoc, const FVector::FReal CostLimit, const bool bRequireNavigableEndLocation, FNavMeshPath& Path, const FNavigationQueryFilter& InQueryFilter, const UObject* Owner) const
{
	// temporarily disabling this check due to it causing too much "crashes"
	// @todo but it needs to be back at some point since it realy checks for a buggy setup
	//ensure(DetourNavMesh != NULL || NavMeshOwner->bRebuildAtRuntime == false);

	if (DetourNavMesh == NULL || NavMeshOwner == NULL)
	{
		return ENavigationQueryResult::Error;
	}

	const FRecastQueryFilter* FilterImplementation = (const FRecastQueryFilter*)(InQueryFilter.GetImplementation());
	if (FilterImplementation == NULL)
	{
		UE_VLOG(NavMeshOwner, LogNavigation, Error, TEXT("FPImplRecastNavMesh::FindPath failed due to passed filter having NULL implementation!"));
		return ENavigationQueryResult::Error;
	}

	const dtQueryFilter* QueryFilter = FilterImplementation->GetAsDetourQueryFilter();
	if (QueryFilter == NULL)
	{
		UE_VLOG(NavMeshOwner, LogNavigation, Warning, TEXT("FPImplRecastNavMesh::FindPath failing due to QueryFilter == NULL"));
		return ENavigationQueryResult::Error;
	}

	FRecastSpeciaLinkFilter LinkFilter(FNavigationSystem::GetCurrent<UNavigationSystemV1>(NavMeshOwner->GetWorld()), Owner);
	INITIALIZE_NAVQUERY(NavQuery, InQueryFilter.GetMaxSearchNodes(), LinkFilter);
	NavQuery.setRequireNavigableEndLocation(bRequireNavigableEndLocation);

	FVector RecastStartPos, RecastEndPos;
	NavNodeRef StartPolyID, EndPolyID;
	const bool bCanSearch = InitPathfinding(StartLoc, EndLoc, NavQuery, QueryFilter, RecastStartPos, StartPolyID, RecastEndPos, EndPolyID, &Path);
	if (!bCanSearch)
	{
		return ENavigationQueryResult::Error;
	}

	// get path corridor
	dtQueryResult PathResult;
	const dtStatus FindPathStatus = NavQuery.findPath(StartPolyID, EndPolyID, &RecastStartPos.X, &RecastEndPos.X, CostLimit, QueryFilter, PathResult, 0);

	return PostProcessPathInternal(FindPathStatus, Path, NavQuery, QueryFilter, StartPolyID, EndPolyID, RecastStartPos, RecastEndPos, PathResult);
}


ENavigationQueryResult::Type FPImplRecastNavMesh::PostProcessPathInternal(dtStatus FindPathStatus, FNavMeshPath& Path, 
	const dtNavMeshQuery& NavQuery, const dtQueryFilter* QueryFilter, 
	NavNodeRef StartPolyID, NavNodeRef EndPolyID, 
	const FVector& RecastStartPos, const FVector& RecastEndPos,
	dtQueryResult& PathResult) const
{
	// check for special case, where path has not been found, and starting polygon
	// was the one closest to the target
	if (PathResult.size() == 1 && dtStatusDetail(FindPathStatus, DT_PARTIAL_RESULT))
	{
		// in this case we find a point on starting polygon, that's closest to destination
		// and store it as path end
		FVector RecastHandPlacedPathEnd;
		NavQuery.closestPointOnPolyBoundary(StartPolyID, &RecastEndPos.X, &RecastHandPlacedPathEnd.X);

		new(Path.GetPathPoints()) FNavPathPoint(Recast2UnrVector(&RecastStartPos.X), StartPolyID);
		new(Path.GetPathPoints()) FNavPathPoint(Recast2UnrVector(&RecastHandPlacedPathEnd.X), StartPolyID);

		Path.PathCorridor.Add(PathResult.getRef(0));
		Path.PathCorridorCost.Add(CalcSegmentCostOnPoly(StartPolyID, QueryFilter, RecastHandPlacedPathEnd, RecastStartPos));
	}
	else
	{
		PostProcessPath(FindPathStatus, Path, NavQuery, QueryFilter,
			StartPolyID, EndPolyID, Recast2UnrVector(&RecastStartPos.X), Recast2UnrVector(&RecastEndPos.X), RecastStartPos, RecastEndPos,
			PathResult);
	}

	if (dtStatusDetail(FindPathStatus, DT_PARTIAL_RESULT))
	{
		Path.SetIsPartial(true);
		// this means path finding algorithm reached the limit of InQueryFilter.GetMaxSearchNodes()
		// nodes in A* node pool. This can mean resulting path is way off.
		Path.SetSearchReachedLimit(dtStatusDetail(FindPathStatus, DT_OUT_OF_NODES));
	}

#if ENABLE_VISUAL_LOG
	if (dtStatusDetail(FindPathStatus, DT_INVALID_CYCLE_PATH))
	{
		UE_VLOG(NavMeshOwner, LogNavigation, Error, TEXT("FPImplRecastNavMesh::FindPath resulted in a cyclic path!"));
		FVisualLogger::Get().ExecuteOnLastEntryForObject(NavMeshOwner, [&Path](FVisualLogEntry& LogEntry)
		{
			Path.DescribeSelfToVisLog(&LogEntry);
		});
	}
#endif // ENABLE_VISUAL_LOG

	Path.MarkReady();

	return DTStatusToNavQueryResult(FindPathStatus);
}

ENavigationQueryResult::Type FPImplRecastNavMesh::TestPath(const FVector& StartLoc, const FVector& EndLoc, const bool bRequireNavigableEndLocation, const FNavigationQueryFilter& InQueryFilter, const UObject* Owner, int32* NumVisitedNodes) const
{
	// Same check as in FPImplRecastNavMesh::FindPath (ex: this can occur when tileWidth of loading navmesh mismatch).
	if (DetourNavMesh == NULL || NavMeshOwner == NULL)
	{
		return ENavigationQueryResult::Error;
	}
	
	const dtQueryFilter* QueryFilter = ((const FRecastQueryFilter*)(InQueryFilter.GetImplementation()))->GetAsDetourQueryFilter();
	if (QueryFilter == NULL)
	{
		UE_VLOG(NavMeshOwner, LogNavigation, Warning, TEXT("FPImplRecastNavMesh::FindPath failing due to QueryFilter == NULL"));
		return ENavigationQueryResult::Error;
	}

	FRecastSpeciaLinkFilter LinkFilter(FNavigationSystem::GetCurrent<UNavigationSystemV1>(NavMeshOwner->GetWorld()), Owner);
	INITIALIZE_NAVQUERY(NavQuery, InQueryFilter.GetMaxSearchNodes(), LinkFilter);
	NavQuery.setRequireNavigableEndLocation(bRequireNavigableEndLocation);

	FVector RecastStartPos, RecastEndPos;
	NavNodeRef StartPolyID, EndPolyID;
	const bool bCanSearch = InitPathfinding(StartLoc, EndLoc, NavQuery, QueryFilter, RecastStartPos, StartPolyID, RecastEndPos, EndPolyID);
	if (!bCanSearch)
	{
		return ENavigationQueryResult::Error;
	}

	// get path corridor
	dtQueryResult PathResult;
	const FVector::FReal CostLimit = TNumericLimits<FVector::FReal>::Max();
	const dtStatus FindPathStatus = NavQuery.findPath(StartPolyID, EndPolyID,
		&RecastStartPos.X, &RecastEndPos.X, CostLimit, QueryFilter, PathResult, 0);

	if (NumVisitedNodes)
	{
		*NumVisitedNodes = NavQuery.getQueryNodes();
	}

	return DTStatusToNavQueryResult(FindPathStatus);
}

#if WITH_NAVMESH_CLUSTER_LINKS
ENavigationQueryResult::Type FPImplRecastNavMesh::TestClusterPath(const FVector& StartLoc, const FVector& EndLoc, int32* NumVisitedNodes) const
{
	FVector RecastStartPos, RecastEndPos;
	NavNodeRef StartPolyID, EndPolyID;
	const dtQueryFilter* ClusterFilter = ((const FRecastQueryFilter*)NavMeshOwner->GetDefaultQueryFilterImpl())->GetAsDetourQueryFilter();

	check(NavMeshOwner->DefaultMaxHierarchicalSearchNodes >= 0. && NavMeshOwner->DefaultMaxHierarchicalSearchNodes <= (float)TNumericLimits<int32>::Max());
	INITIALIZE_NAVQUERY_SIMPLE(ClusterQuery, static_cast<int32>(NavMeshOwner->DefaultMaxHierarchicalSearchNodes));
	ClusterQuery.setRequireNavigableEndLocation(true);

	const bool bCanSearch = InitPathfinding(StartLoc, EndLoc, ClusterQuery, ClusterFilter, RecastStartPos, StartPolyID, RecastEndPos, EndPolyID);
	if (!bCanSearch)
	{
		return ENavigationQueryResult::Error;
	}

	const dtStatus status = ClusterQuery.testClusterPath(StartPolyID, EndPolyID);
	if (NumVisitedNodes)
	{
		*NumVisitedNodes = ClusterQuery.getQueryNodes();
	}

	return DTStatusToNavQueryResult(status);
}
#endif // WITH_NAVMESH_CLUSTER_LINKS

bool FPImplRecastNavMesh::InitPathfinding(const FVector& UnrealStart, const FVector& UnrealEnd,
	const dtNavMeshQuery& Query, const dtQueryFilter* Filter,
	FVector& RecastStart, dtPolyRef& StartPoly,
	FVector& RecastEnd, dtPolyRef& EndPoly, FNavMeshPath* OutPath) const
{
	const FVector NavExtent = NavMeshOwner->GetModifiedQueryExtent(NavMeshOwner->GetDefaultQueryExtent());
	const FVector::FReal Extent[3] = { NavExtent.X, NavExtent.Z, NavExtent.Y };

	const FVector RecastStartToProject = Unreal2RecastPoint(UnrealStart);
	const FVector RecastEndToProject = Unreal2RecastPoint(UnrealEnd);

	StartPoly = INVALID_NAVNODEREF;
	Query.findNearestPoly(&RecastStartToProject.X, Extent, Filter, &StartPoly, &RecastStart.X);
	if (StartPoly == INVALID_NAVNODEREF)
	{
		UE_VLOG_UELOG(NavMeshOwner, LogNavigation, Log, TEXT("FPImplRecastNavMesh::InitPathfinding start point not on navmesh (%s)"), *UnrealStart.ToString());
		UE_VLOG_SEGMENT(NavMeshOwner, LogNavigation, Log, UnrealStart, UnrealEnd, FColor::Red, TEXT("Failed path"));
		UE_VLOG_LOCATION(NavMeshOwner, LogNavigation, Log, UnrealStart, 15, FColor::Red, TEXT("Start failed"));
		UE_VLOG_BOX(NavMeshOwner, LogNavigation, Log, FBox(UnrealStart - NavExtent, UnrealStart + NavExtent), FColor::Red, TEXT_EMPTY);

		if (OutPath)
		{
			OutPath->SetErrorStartLocationNonNavigable(true);
		}
		return false;
	}

	EndPoly = INVALID_NAVNODEREF;
	Query.findNearestPoly(&RecastEndToProject.X, Extent, Filter, &EndPoly, &RecastEnd.X);
	if (EndPoly == INVALID_NAVNODEREF)
	{
		if (Query.isRequiringNavigableEndLocation())
		{
			UE_VLOG_UELOG(NavMeshOwner, LogNavigation, Log, TEXT("FPImplRecastNavMesh::InitPathfinding end point not on navmesh (%s)"), *UnrealEnd.ToString());
			UE_VLOG_SEGMENT(NavMeshOwner, LogNavigation, Log, UnrealEnd, UnrealEnd, FColor::Red, TEXT("Failed path"));
			UE_VLOG_LOCATION(NavMeshOwner, LogNavigation, Log, UnrealEnd, 15, FColor::Red, TEXT("End failed"));
			UE_VLOG_BOX(NavMeshOwner, LogNavigation, Log, FBox(UnrealEnd - NavExtent, UnrealEnd + NavExtent), FColor::Red, TEXT_EMPTY);

			if (OutPath)
			{
				OutPath->SetErrorEndLocationNonNavigable(true);
			}
			return false;
		}

		// we will use RecastEndToProject as the estimated end location since we didn't find a poly. It will be used to compute the heuristic mainly
		dtVcopy(&RecastEnd.X, &RecastEndToProject.X);
	}

	return true;
}

FVector::FReal FPImplRecastNavMesh::CalcSegmentCostOnPoly(NavNodeRef PolyID, const dtQueryFilter* Filter, const FVector& StartLoc, const FVector& EndLoc) const
{
	uint8 AreaID = RECAST_DEFAULT_AREA;
	DetourNavMesh->getPolyArea(PolyID, &AreaID);

	const FVector::FReal AreaTravelCost = Filter->getAreaCost(AreaID);
	return AreaTravelCost * (EndLoc - StartLoc).Size();
}

void FPImplRecastNavMesh::PostProcessPath(dtStatus FindPathStatus, FNavMeshPath& Path,
	const dtNavMeshQuery& NavQuery, const dtQueryFilter* Filter,
	NavNodeRef StartPolyID, NavNodeRef EndPolyID,
	FVector StartLoc, FVector EndLoc,
	FVector RecastStartPos, FVector RecastEndPos,
	dtQueryResult& PathResult) const
{
	check(Filter);

	// note that for recast partial path is successful, while we treat it as failed, just marking it as partial
	if (dtStatusSucceed(FindPathStatus))
	{
		// check if navlink poly at end of path is allowed
		int32 PathSize = PathResult.size();
		if ((PathSize > 1) && NavMeshOwner && !NavMeshOwner->bAllowNavLinkAsPathEnd)
		{
			uint16 PolyFlags = 0;
			DetourNavMesh->getPolyFlags(PathResult.getRef(PathSize - 1), &PolyFlags);

			if (PolyFlags & ARecastNavMesh::GetNavLinkFlag())
			{
				PathSize--;
			}
		}

		Path.PathCorridorCost.AddUninitialized(PathSize);

		if (PathSize == 1)
		{
			// failsafe cost for single poly corridor
			Path.PathCorridorCost[0] = CalcSegmentCostOnPoly(StartPolyID, Filter, EndLoc, StartLoc);
		}
		else
		{
			for (int32 i = 0; i < PathSize; i++)
			{
				Path.PathCorridorCost[i] = PathResult.getCost(i);
			}
		}
		
		// copy over corridor poly data
		Path.PathCorridor.AddUninitialized(PathSize);
		NavNodeRef* DestCorridorPoly = Path.PathCorridor.GetData();
		for (int i = 0; i < PathSize; ++i, ++DestCorridorPoly)
		{
			*DestCorridorPoly = PathResult.getRef(i);
		}

		Path.OnPathCorridorUpdated();

		// if we're backtracking this is the time to reverse the path.
		if (Filter->getIsBacktracking())
		{
			// for a proper string-pulling of a backtracking path we need to
			// reverse the data right now.
			Path.Invert();
			Swap(StartPolyID, EndPolyID);
			Swap(StartLoc, EndLoc);
			Swap(RecastStartPos, RecastEndPos);
		}

#if STATS
		if (dtStatusDetail(FindPathStatus, DT_OUT_OF_NODES))
		{
			INC_DWORD_STAT(STAT_Navigation_OutOfNodesPath);
		}

		if (dtStatusDetail(FindPathStatus, DT_PARTIAL_RESULT))
		{
			INC_DWORD_STAT(STAT_Navigation_PartialPath);
		}
#endif

		if (Path.WantsStringPulling())
		{
			FVector UseEndLoc = EndLoc;
			
			// if path is partial (path corridor doesn't contain EndPolyID), find new RecastEndPos on last poly in corridor
			if (dtStatusDetail(FindPathStatus, DT_PARTIAL_RESULT))
			{
				NavNodeRef LastPolyID = Path.PathCorridor.Last();
				FVector::FReal NewEndPoint[3];

				const dtStatus NewEndPointStatus = NavQuery.closestPointOnPoly(LastPolyID, &RecastEndPos.X, NewEndPoint);
				if (dtStatusSucceed(NewEndPointStatus))
				{
					UseEndLoc = Recast2UnrealPoint(NewEndPoint);
				}
			}

			Path.PerformStringPulling(StartLoc, UseEndLoc);
		}
		else
		{
			// make sure at least beginning and end of path are added
			new(Path.GetPathPoints()) FNavPathPoint(StartLoc, StartPolyID);
			new(Path.GetPathPoints()) FNavPathPoint(EndLoc, EndPolyID);

			// collect all custom links Ids
			for (int32 Idx = 0; Idx < Path.PathCorridor.Num(); Idx++)
			{
				const dtOffMeshConnection* OffMeshCon = DetourNavMesh->getOffMeshConnectionByRef(Path.PathCorridor[Idx]);
				if (OffMeshCon)
				{
					Path.CustomNavLinkIds.Add(FNavLinkId(OffMeshCon->userId));
				}
			}
		}

		if (Path.WantsPathCorridor())
		{
			TArray<FNavigationPortalEdge> PathCorridorEdges;
			GetEdgesForPathCorridorImpl(&Path.PathCorridor, &PathCorridorEdges, NavQuery);
			Path.SetPathCorridorEdges(PathCorridorEdges);
		}
	}
}

bool FPImplRecastNavMesh::FindStraightPath(const FVector& StartLoc, const FVector& EndLoc, const TArray<NavNodeRef>& PathCorridor, TArray<FNavPathPoint>& PathPoints, TArray<FNavLinkId>* CustomLinks) const
{
	INITIALIZE_NAVQUERY_SIMPLE(NavQuery, RECAST_MAX_SEARCH_NODES);

	const FVector RecastStartPos = Unreal2RecastPoint(StartLoc);
	const FVector RecastEndPos = Unreal2RecastPoint(EndLoc);
	bool bResult = false;

	dtQueryResult StringPullResult;
	const dtStatus StringPullStatus = NavQuery.findStraightPath(&RecastStartPos.X, &RecastEndPos.X,
		PathCorridor.GetData(), PathCorridor.Num(), StringPullResult, DT_STRAIGHTPATH_AREA_CROSSINGS);

	PathPoints.Reset();
	if (dtStatusSucceed(StringPullStatus))
	{
		PathPoints.AddZeroed(StringPullResult.size());

		// convert to desired format
		FNavPathPoint* CurVert = PathPoints.GetData();

		for (int32 VertIdx = 0; VertIdx < StringPullResult.size(); ++VertIdx)
		{
			const FVector::FReal* CurRecastVert = StringPullResult.getPos(VertIdx);
			CurVert->Location = Recast2UnrVector(CurRecastVert);
			CurVert->NodeRef = StringPullResult.getRef(VertIdx);

			FNavMeshNodeFlags CurNodeFlags(0);
			CurNodeFlags.PathFlags = IntCastChecked<uint8>(StringPullResult.getFlag(VertIdx));

			uint8 AreaID = RECAST_DEFAULT_AREA;
			DetourNavMesh->getPolyArea(CurVert->NodeRef, &AreaID);
			CurNodeFlags.Area = AreaID;

			const UClass* AreaClass = NavMeshOwner->GetAreaClass(AreaID);
			const UNavArea* DefArea = AreaClass ? ((UClass*)AreaClass)->GetDefaultObject<UNavArea>() : NULL;
			CurNodeFlags.AreaFlags = DefArea ? DefArea->GetAreaFlags() : 0;

			CurVert->Flags = CurNodeFlags.Pack();

			// include smart link data
			// if there will be more "edge types" we change this implementation to something more generic
			if (CustomLinks && (CurNodeFlags.PathFlags & DT_STRAIGHTPATH_OFFMESH_CONNECTION))
			{
				const dtOffMeshConnection* OffMeshCon = DetourNavMesh->getOffMeshConnectionByRef(CurVert->NodeRef);
				if (OffMeshCon)
				{
					CurVert->CustomNavLinkId.SetId(OffMeshCon->userId);
					CustomLinks->Add(FNavLinkId(OffMeshCon->userId));
				}
			}

			CurVert++;
		}

		// findStraightPath returns 0 for polyId of last point for some reason, even though it knows the poly id.  We will fill that in correctly with the last poly id of the corridor.
		// @TODO shouldn't it be the same as EndPolyID? (nope, it could be partial path)
		PathPoints.Last().NodeRef = PathCorridor.Last();
		bResult = true;
	}

	return bResult;
}

static bool IsDebugNodeModified(const FRecastDebugPathfindingNode& NodeData, const FRecastDebugPathfindingData& PreviousStep)
{
	const FRecastDebugPathfindingNode* PrevNodeData = PreviousStep.Nodes.Find(NodeData);
	if (PrevNodeData)
	{
		const bool bModified = PrevNodeData->bOpenSet != NodeData.bOpenSet ||
			PrevNodeData->TotalCost != NodeData.TotalCost ||
			PrevNodeData->Cost != NodeData.Cost ||
			PrevNodeData->ParentRef != NodeData.ParentRef ||
			!PrevNodeData->NodePos.Equals(NodeData.NodePos, SMALL_NUMBER);

		return bModified;
	}

	return true;
}

static void StorePathfindingDebugData(const dtNavMeshQuery& NavQuery, const dtNavMesh* NavMesh, FRecastDebugPathfindingData& Data)
{
	const dtNodePool* NodePool = NavQuery.getNodePool();
	check(NodePool);

	const int32 NodeCount = NodePool->getNodeCount();
	if (NodeCount <= 0)
	{
		return;
	}
	
	// cache path lengths for all nodes in pool, indexed by poolIdx (idx + 1)
	TArray<FVector::FReal> NodePathLength;
	if (Data.Flags & ERecastDebugPathfindingFlags::PathLength)
	{
		NodePathLength.AddZeroed(NodeCount + 1);
	}

	Data.Nodes.Reserve(NodeCount);
	for (int32 Idx = 0; Idx < NodeCount; Idx++)
	{
		const int32 NodePoolIdx = Idx + 1;
		const dtNode* Node = NodePool->getNodeAtIdx(NodePoolIdx);
		check(Node);

		const dtNode* ParentNode = Node->pidx ? NodePool->getNodeAtIdx(Node->pidx) : nullptr;

		FRecastDebugPathfindingNode NodeInfo;
		NodeInfo.PolyRef = Node->id;
		NodeInfo.ParentRef = ParentNode ? ParentNode->id : 0;
		NodeInfo.Cost = Node->cost; 
		NodeInfo.TotalCost = Node->total;
		NodeInfo.Length = 0.;
		NodeInfo.bOpenSet = (Node->flags & DT_NODE_OPEN) != 0;
		NodeInfo.bModified = true;
		NodeInfo.NodePos = Recast2UnrealPoint(&Node->pos[0]);

		const dtPoly* NavPoly = 0;
		const dtMeshTile* NavTile = 0;
		NavMesh->getTileAndPolyByRef(Node->id, &NavTile, &NavPoly);

		NodeInfo.bOffMeshLink = NavPoly ? (NavPoly->getType() != DT_POLYTYPE_GROUND) : false;
		if (Data.Flags & ERecastDebugPathfindingFlags::Vertices)
		{
			check(NavPoly);

			NodeInfo.NumVerts = NavPoly->vertCount;
			for (int32 VertIdx = 0; VertIdx < NavPoly->vertCount; VertIdx++)
			{
				dtReal V[3];
				NavTile->verts[NavPoly->verts[VertIdx]].toWorld(NavTile->header->bmin, V);
				NodeInfo.Verts.Add((FVector3f)Recast2UnrealPoint(V));
			}
		}

		if ((Data.Flags & ERecastDebugPathfindingFlags::PathLength) && ParentNode)
		{
			const FVector ParentPos = Recast2UnrealPoint(&ParentNode->pos[0]);
			const FVector::FReal NodeLinkLen = FVector::Dist(NodeInfo.NodePos, ParentPos);

			// no point in validating, it would already crash on reading ParentNode (no validation in NodePool.getNodeAtIdx)
			const FVector::FReal ParentPathLength = NodePathLength[Node->pidx];

			const FVector::FReal LinkAndParentLength = NodeLinkLen + ParentPathLength;
			
			NodePathLength[NodePoolIdx] = LinkAndParentLength;

			NodeInfo.Length = LinkAndParentLength;
		}

		Data.Nodes.Add(NodeInfo);
	}

	if (Data.Flags & ERecastDebugPathfindingFlags::BestNode)
	{
		dtNode* BestNode = nullptr;
		FVector::FReal BestNodeCost = 0.0f;
		NavQuery.getCurrentBestResult(BestNode, BestNodeCost);

		if (BestNode)
		{
			const FRecastDebugPathfindingNode BestNodeKey(BestNode->id);
			Data.BestNode = Data.Nodes.FindId(BestNodeKey);
		}
	}
}

static void StorePathfindingDebugStep(const dtNavMeshQuery& NavQuery, const dtNavMesh* NavMesh, TArray<FRecastDebugPathfindingData>& Steps)
{
	const int StepIdx = Steps.AddZeroed(1);
	FRecastDebugPathfindingData& StepInfo = Steps[StepIdx];
	StepInfo.Flags = ERecastDebugPathfindingFlags::BestNode | ERecastDebugPathfindingFlags::Vertices;
	
	StorePathfindingDebugData(NavQuery, NavMesh, StepInfo);

	if (Steps.Num() > 1)
	{
		FRecastDebugPathfindingData& PrevStepInfo = Steps[StepIdx - 1];
		for (TSet<FRecastDebugPathfindingNode>::TIterator It(StepInfo.Nodes); It; ++It)
		{
			FRecastDebugPathfindingNode& NodeData = *It;
			NodeData.bModified = IsDebugNodeModified(NodeData, PrevStepInfo);
		}
	}
}

int32 FPImplRecastNavMesh::DebugPathfinding(const FVector& StartLoc, const FVector& EndLoc, const FVector::FReal CostLimit, const bool bRequireNavigableEndLocation, const FNavigationQueryFilter& Filter, const UObject* Owner, TArray<FRecastDebugPathfindingData>& Steps)
{
	int32 NumSteps = 0;

	const dtQueryFilter* QueryFilter = ((const FRecastQueryFilter*)(Filter.GetImplementation()))->GetAsDetourQueryFilter();
	if (QueryFilter == NULL)
	{
		UE_VLOG(NavMeshOwner, LogNavigation, Warning, TEXT("FPImplRecastNavMesh::DebugPathfinding failing due to QueryFilter == NULL"));
		return NumSteps;
	}

	FRecastSpeciaLinkFilter LinkFilter(FNavigationSystem::GetCurrent<UNavigationSystemV1>(NavMeshOwner->GetWorld()), Owner);
	INITIALIZE_NAVQUERY(NavQuery, Filter.GetMaxSearchNodes(), LinkFilter);
	NavQuery.setRequireNavigableEndLocation(bRequireNavigableEndLocation);

	FVector RecastStartPos, RecastEndPos;
	NavNodeRef StartPolyID, EndPolyID;
	const bool bCanSearch = InitPathfinding(StartLoc, EndLoc, NavQuery, QueryFilter, RecastStartPos, StartPolyID, RecastEndPos, EndPolyID);
	if (!bCanSearch)
	{
		return NumSteps;
	}

	dtStatus status = NavQuery.initSlicedFindPath(StartPolyID, EndPolyID, &RecastStartPos.X, &RecastEndPos.X, CostLimit, bRequireNavigableEndLocation, QueryFilter);
	while (dtStatusInProgress(status))
	{
		StorePathfindingDebugStep(NavQuery, DetourNavMesh, Steps);
		NumSteps++;

		status = NavQuery.updateSlicedFindPath(1, 0);
	}

	static const int32 MAX_TEMP_POLYS = 16;
	NavNodeRef TempPolys[MAX_TEMP_POLYS];
	int32 NumTempPolys;
	NavQuery.finalizeSlicedFindPath(TempPolys, &NumTempPolys, MAX_TEMP_POLYS);

	return NumSteps;
}

#if WITH_NAVMESH_CLUSTER_LINKS
NavNodeRef FPImplRecastNavMesh::GetClusterRefFromPolyRef(const NavNodeRef PolyRef) const
{
	if (DetourNavMesh)
	{
		const dtMeshTile* Tile = DetourNavMesh->getTileByRef(PolyRef);
		uint32 PolyIdx = DetourNavMesh->decodePolyIdPoly(PolyRef);
		if (Tile && Tile->polyClusters && PolyIdx < (uint32)Tile->header->offMeshBase)
		{
			return DetourNavMesh->getClusterRefBase(Tile) | Tile->polyClusters[PolyIdx];
		}
	}

	return 0;
}
#endif // WITH_NAVMESH_CLUSTER_LINKS

FNavLocation FPImplRecastNavMesh::GetRandomPoint(const FNavigationQueryFilter& Filter, const UObject* Owner) const
{
	FNavLocation OutLocation;
	if (DetourNavMesh == NULL)
	{
		return OutLocation;
	}

	FRecastSpeciaLinkFilter LinkFilter(FNavigationSystem::GetCurrent<UNavigationSystemV1>(NavMeshOwner->GetWorld()), Owner);
	INITIALIZE_NAVQUERY(NavQuery, Filter.GetMaxSearchNodes(), LinkFilter);

	// inits to "pass all"
	const dtQueryFilter* QueryFilter = ((const FRecastQueryFilter*)(Filter.GetImplementation()))->GetAsDetourQueryFilter();
	ensure(QueryFilter);
	if (QueryFilter)
	{
		dtPolyRef Poly;
		FVector::FReal RandPt[3];
		dtStatus Status = NavQuery.findRandomPoint(QueryFilter, FMath::FRand, &Poly, RandPt);
		if (dtStatusSucceed(Status))
		{
			// arrange output
			OutLocation.Location = Recast2UnrVector(RandPt);
			OutLocation.NodeRef = Poly;
		}
	}

	return OutLocation;
}

#if WITH_NAVMESH_CLUSTER_LINKS
bool FPImplRecastNavMesh::GetRandomPointInCluster(NavNodeRef ClusterRef, FNavLocation& OutLocation) const
{
	if (DetourNavMesh == NULL || ClusterRef == 0)
	{
		return false;
	}

	INITIALIZE_NAVQUERY_SIMPLE(NavQuery, RECAST_MAX_SEARCH_NODES);

	dtPolyRef Poly;
	FVector::FReal RandPt[3];
	dtStatus Status = NavQuery.findRandomPointInCluster(ClusterRef, FMath::FRand, &Poly, RandPt);

	if (dtStatusSucceed(Status))
	{
		OutLocation = FNavLocation(Recast2UnrVector(RandPt), Poly);
		return true;
	}

	return false;
}
#endif // WITH_NAVMESH_CLUSTER_LINKS

bool FPImplRecastNavMesh::FindMoveAlongSurface(const FNavLocation& StartLocation, const FVector& TargetPosition, FNavLocation& OutLocation, const FNavigationQueryFilter& Filter, const UObject* Owner) const
{
	// sanity check
	if (DetourNavMesh == NULL)
	{
		return false;
	}

	FRecastSpeciaLinkFilter LinkFilter(FNavigationSystem::GetCurrent<UNavigationSystemV1>(NavMeshOwner->GetWorld()), Owner);
	INITIALIZE_NAVQUERY(NavQuery, Filter.GetMaxSearchNodes(), LinkFilter);

	const dtQueryFilter* QueryFilter = ((const FRecastQueryFilter*)(Filter.GetImplementation()))->GetAsDetourQueryFilter();
	ensure(QueryFilter);
	if (!QueryFilter)
	{
		return false;
	}

	FVector RcStartPos = Unreal2RecastPoint(StartLocation.Location);
	FVector RcEndPos = Unreal2RecastPoint(TargetPosition);

	FVector::FReal Result[3];
	static const int MAX_VISITED = 16;
	dtPolyRef Visited[MAX_VISITED];
	int VisitedCount = 0;

	dtStatus status = NavQuery.moveAlongSurface(StartLocation.NodeRef, &RcStartPos.X, &RcEndPos.X, QueryFilter, Result, Visited, &VisitedCount, MAX_VISITED);
	if (dtStatusFailed(status))
	{
		return false;
	}
	dtPolyRef ResultPoly = Visited[VisitedCount - 1];

	// Adjust the position to stay on top of the navmesh.
	FVector::FReal h = RcStartPos.Y;
	NavQuery.getPolyHeight(ResultPoly, Result, &h);
	Result[1] = h;

	const FVector UnrealResult = Recast2UnrVector(Result);

	OutLocation = FNavLocation(UnrealResult, ResultPoly);

	return true;
}

bool FPImplRecastNavMesh::ProjectPointToNavMesh(const FVector& Point, FNavLocation& Result, const FVector& Extent, const FNavigationQueryFilter& Filter, const UObject* Owner) const
{
	// sanity check
	if (DetourNavMesh == NULL)
	{
		return false;
	}

	bool bSuccess = false;

	FRecastSpeciaLinkFilter LinkFilter(FNavigationSystem::GetCurrent<UNavigationSystemV1>(NavMeshOwner->GetWorld()), Owner);
	// using 0 as NumNodes since findNearestPoly2D, being the only dtNavMeshQuery
	// function we're using, is not utilizing m_nodePool
	INITIALIZE_NAVQUERY(NavQuery, /*NumNodes=*/0, LinkFilter);

	const dtQueryFilter* QueryFilter = ((const FRecastQueryFilter*)(Filter.GetImplementation()))->GetAsDetourQueryFilter();
	ensure(QueryFilter);
	if (QueryFilter)
	{
		FVector::FReal ClosestPoint[3];

		const FVector ModifiedExtent = NavMeshOwner->GetModifiedQueryExtent(Extent);
		FVector RcExtent = Unreal2RecastPoint(ModifiedExtent).GetAbs();
	
		FVector RcPoint = Unreal2RecastPoint(Point);
		dtPolyRef PolyRef;
		NavQuery.findNearestPoly2D(&RcPoint.X, &RcExtent.X, QueryFilter, &PolyRef, ClosestPoint);

		if( PolyRef > 0 )
		{
			// one last step required due to recast's BVTree imprecision
			const FVector& UnrealClosestPoint = Recast2UnrVector(ClosestPoint);			
			const FVector ClosestPointDelta = UnrealClosestPoint - Point;
			if (-ModifiedExtent.X <= ClosestPointDelta.X && ClosestPointDelta.X <= ModifiedExtent.X
				&& -ModifiedExtent.Y <= ClosestPointDelta.Y && ClosestPointDelta.Y <= ModifiedExtent.Y
				&& -ModifiedExtent.Z <= ClosestPointDelta.Z && ClosestPointDelta.Z <= ModifiedExtent.Z)
			{
				bSuccess = true;
				Result = FNavLocation(UnrealClosestPoint, PolyRef);
			}
			else
			{
				const UObject* LogOwner = Owner ? Owner : NavMeshOwner;
				UE_VLOG(LogOwner, LogNavigation, Error, TEXT("ProjectPointToNavMesh failed due to ClosestPoint being too far away from projected point."));
				UE_VLOG_LOCATION(LogOwner, LogNavigation, Error, Point, 30, FColor::Blue, TEXT("Requested point"));
				UE_VLOG_LOCATION(LogOwner, LogNavigation, Error, UnrealClosestPoint, 30, FColor::Red, TEXT("Projection result"));
				UE_VLOG_SEGMENT(LogOwner, LogNavigation, Error, Point, UnrealClosestPoint, FColor::Red, TEXT(""));
			}
		}
	}

	return (bSuccess);
}

bool FPImplRecastNavMesh::ProjectPointMulti(const FVector& Point, TArray<FNavLocation>& Result, const FVector& Extent,
	FVector::FReal MinZ, FVector::FReal MaxZ, const FNavigationQueryFilter& Filter, const UObject* Owner) const
{
	// sanity check
	if (DetourNavMesh == NULL)
	{
		return false;
	}

	bool bSuccess = false;

	FRecastSpeciaLinkFilter LinkFilter(FNavigationSystem::GetCurrent<UNavigationSystemV1>(NavMeshOwner->GetWorld()), Owner);
	INITIALIZE_NAVQUERY(NavQuery, Filter.GetMaxSearchNodes(), LinkFilter);

	const dtQueryFilter* QueryFilter = ((const FRecastQueryFilter*)(Filter.GetImplementation()))->GetAsDetourQueryFilter();
	ensure(QueryFilter);
	if (QueryFilter)
	{
		const FVector ModifiedExtent = NavMeshOwner->GetModifiedQueryExtent(Extent);
		const FVector AdjustedPoint(Point.X, Point.Y, (MaxZ + MinZ) * 0.5f);
		const FVector AdjustedExtent(ModifiedExtent.X, ModifiedExtent.Y, (MaxZ - MinZ) * 0.5f);

		const FVector RcPoint = Unreal2RecastPoint( AdjustedPoint );
		const FVector RcExtent = Unreal2RecastPoint( AdjustedExtent ).GetAbs();

		const int32 MaxHitPolys = 256;
		dtPolyRef HitPolys[MaxHitPolys];
		int32 NumHitPolys = 0;

		dtStatus status = NavQuery.queryPolygons(&RcPoint.X, &RcExtent.X, QueryFilter, HitPolys, &NumHitPolys, MaxHitPolys);
		if (dtStatusSucceed(status))
		{
			for (int32 i = 0; i < NumHitPolys; i++)
			{
				FVector::FReal ClosestPoint[3];
				
				status = NavQuery.projectedPointOnPoly(HitPolys[i], &RcPoint.X, ClosestPoint);
				if (dtStatusSucceed(status))
				{
					FVector::FReal ExactZ = 0.0f;
					status = NavQuery.getPolyHeight(HitPolys[i], ClosestPoint, &ExactZ);
					if (dtStatusSucceed(status))
					{
						FNavLocation HitLocation(Recast2UnrealPoint(ClosestPoint), HitPolys[i]);
						HitLocation.Location.Z = ExactZ;

						ensure((HitLocation.Location - AdjustedPoint).SizeSquared2D() < KINDA_SMALL_NUMBER);

						Result.Add(HitLocation);
						bSuccess = true;
					}
				}
			}
		}
	}

	return bSuccess;
}

NavNodeRef FPImplRecastNavMesh::FindNearestPoly(FVector const& Loc, FVector const& Extent, const FNavigationQueryFilter& Filter, const UObject* Owner) const
{
	// sanity check
	if (DetourNavMesh == NULL)
	{
		return INVALID_NAVNODEREF;
	}

	FRecastSpeciaLinkFilter LinkFilter(FNavigationSystem::GetCurrent<UNavigationSystemV1>(NavMeshOwner->GetWorld()), Owner);
	INITIALIZE_NAVQUERY(NavQuery, Filter.GetMaxSearchNodes(), LinkFilter);

	// inits to "pass all"
	const dtQueryFilter* QueryFilter = ((const FRecastQueryFilter*)(Filter.GetImplementation()))->GetAsDetourQueryFilter();
	ensure(QueryFilter);
	if (QueryFilter)
	{
		FVector::FReal  RecastLoc[3];
		Unr2RecastVector(Loc, RecastLoc);
		FVector::FReal  RecastExtent[3];
		Unr2RecastSizeVector(NavMeshOwner->GetModifiedQueryExtent(Extent), RecastExtent);

		NavNodeRef OutRef;
		dtStatus Status = NavQuery.findNearestPoly(RecastLoc, RecastExtent, QueryFilter, &OutRef, NULL);
		if (dtStatusSucceed(Status))
		{
			return OutRef;
		}
	}

	return INVALID_NAVNODEREF;
}

bool FPImplRecastNavMesh::FindPolysAroundCircle(const FVector& CenterPos, const NavNodeRef CenterNodeRef, const FVector::FReal Radius, const FNavigationQueryFilter& Filter, const UObject* Owner, TArray<NavNodeRef>* OutPolys, TArray<NavNodeRef>* OutPolysParent, TArray<float>* OutPolysCost, int* OutPolysCount) const
{
	// sanity check
	if (DetourNavMesh == NULL || NavMeshOwner == NULL || CenterNodeRef == INVALID_NAVNODEREF)
	{
		return false;
	}

	TArray<FVector::FReal> PolysCost;

	// limit max number of polys found by that function
	// if you need more, please scan manually using ARecastNavMesh::GetPolyNeighbors for A*/Dijkstra loop
	const int32 MaxSearchLimit = 4096;
	const int32 MaxSearchNodes = Filter.GetMaxSearchNodes();
	ensureMsgf(MaxSearchNodes > 0 && MaxSearchNodes <= MaxSearchLimit, TEXT("MaxSearchNodes:%d is not within range: 0..%d"), MaxSearchNodes, MaxSearchLimit);

	FRecastSpeciaLinkFilter LinkFilter(FNavigationSystem::GetCurrent<UNavigationSystemV1>(NavMeshOwner->GetWorld()), Owner);
	INITIALIZE_NAVQUERY(NavQuery, Filter.GetMaxSearchNodes(), LinkFilter);

	const dtQueryFilter* QueryFilter = ((const FRecastQueryFilter*)(Filter.GetImplementation()))->GetAsDetourQueryFilter();
	if (ensure(QueryFilter))
	{
		if (OutPolys)
		{
			OutPolys->Reset();
			OutPolys->AddUninitialized(MaxSearchNodes);
		}

		if (OutPolysParent)
		{
			OutPolysParent->Reset();
			OutPolysParent->AddUninitialized(MaxSearchNodes);
		}

		if (OutPolysCost)
		{
			PolysCost.Reset();
			PolysCost.AddUninitialized(MaxSearchNodes);
		}

		FVector::FReal RecastLoc[3];
		Unr2RecastVector(CenterPos, RecastLoc);
		const dtStatus Status = NavQuery.findPolysAroundCircle(CenterNodeRef, RecastLoc, Radius, QueryFilter, OutPolys ? OutPolys->GetData() : nullptr, OutPolysParent ? OutPolysParent->GetData() : nullptr, OutPolysCost ? PolysCost.GetData() : nullptr, OutPolysCount, MaxSearchNodes);

		if (OutPolysCost)
		{
			*OutPolysCost = UE::LWC::ConvertArrayTypeClampMax<float>(PolysCost);
		}

		if (dtStatusSucceed(Status))
		{
			return true;
		}
	}

	return false;
}

bool FPImplRecastNavMesh::GetPolysWithinPathingDistance(FVector const& StartLoc, const FVector::FReal PathingDistance,
	const FNavigationQueryFilter& Filter, const UObject* Owner,
	TArray<NavNodeRef>& FoundPolys, FRecastDebugPathfindingData* OutDebugData) const
{
	ensure(PathingDistance > 0. && "PathingDistance <= 0 doesn't make sense");
	
	// limit max number of polys found by that function
	// if you need more, please scan manually using ARecastNavMesh::GetPolyNeighbors for A*/Dijkstra loop
	const int32 MaxSearchLimit = 4096;
	const int32 MaxSearchNodes = Filter.GetMaxSearchNodes();
	ensureMsgf(MaxSearchNodes > 0 && MaxSearchNodes <= MaxSearchLimit, TEXT("MaxSearchNodes:%d is not within range: 0..%d"), MaxSearchNodes, MaxSearchLimit);

	// sanity check
	if (DetourNavMesh == nullptr || MaxSearchNodes <= 0 || MaxSearchNodes > MaxSearchLimit)
	{
		return false;
	}

	FRecastSpeciaLinkFilter LinkFilter(FNavigationSystem::GetCurrent<UNavigationSystemV1>(NavMeshOwner->GetWorld()), Owner);
	INITIALIZE_NAVQUERY(NavQuery, MaxSearchNodes, LinkFilter);

	const dtQueryFilter* QueryFilter = ((const FRecastQueryFilter*)(Filter.GetImplementation()))->GetAsDetourQueryFilter();
	ensure(QueryFilter);
	if (QueryFilter == nullptr)
	{
		return false;
	}

	// @todo this should be configurable in some kind of FindPathQuery structure
	const FVector NavExtent = NavMeshOwner->GetModifiedQueryExtent(NavMeshOwner->GetDefaultQueryExtent());
	const FVector::FReal Extent[3] = { NavExtent.X, NavExtent.Z, NavExtent.Y };

	FVector::FReal RecastStartPos[3];
	Unr2RecastVector(StartLoc, RecastStartPos);
	// @TODO add failure handling
	NavNodeRef StartPolyID = INVALID_NAVNODEREF;
	NavQuery.findNearestPoly(RecastStartPos, Extent, QueryFilter, &StartPolyID, NULL);

	FoundPolys.Reset(MaxSearchNodes);
	FoundPolys.AddUninitialized(MaxSearchNodes);
	int32 NumPolys = 0;

	NavQuery.findPolysInPathDistance(StartPolyID, RecastStartPos, PathingDistance, QueryFilter, FoundPolys.GetData(), &NumPolys, MaxSearchNodes);
	FoundPolys.RemoveAt(NumPolys, FoundPolys.Num() - NumPolys);

	if (OutDebugData)
	{
		StorePathfindingDebugData(NavQuery, DetourNavMesh, *OutDebugData);
	}

	return FoundPolys.Num() > 0;
}

void FPImplRecastNavMesh::UpdateNavigationLinkArea(FNavLinkId UserId, uint8 AreaType, uint16 PolyFlags) const
{
	if (DetourNavMesh)
	{
		DetourNavMesh->updateOffMeshConnectionByUserId(UserId.GetId(), AreaType, PolyFlags);
	}
}

#if WITH_NAVMESH_SEGMENT_LINKS
void FPImplRecastNavMesh::UpdateSegmentLinkArea(int32 UserId, uint8 AreaType, uint16 PolyFlags) const
{
	if (DetourNavMesh)
	{
		DetourNavMesh->updateOffMeshSegmentConnectionByUserId(UserId, AreaType, PolyFlags);
	}
}

void FPImplRecastNavMesh::ProcessSegmentLinksForTile(FNavTileRef TileRef) const
{
	uint32 NumSkippedNeighborTiles;
	ProcessSegmentLinksForTile(TileRef, 0 /*MaxSkippedNeigborTiles*/, nullptr /*OutSkippedNeigborTiles*/, NumSkippedNeighborTiles);
}

void FPImplRecastNavMesh::ProcessSegmentLinksForTile(FNavTileRef TileRef, uint32 MaxSkippedNeigborTiles, dtTileRef* OutSkippedNeigborTiles, uint32& OutNumSkippedNeigborTiles) const
{
	if (DetourNavMesh)
	{
		DetourNavMesh->processSegmentLinksForTile(static_cast<dtTileRef>(TileRef), MaxSkippedNeigborTiles, OutSkippedNeigborTiles, OutNumSkippedNeigborTiles);
	}
}
#endif // WITH_NAVMESH_SEGMENT_LINKS

bool FPImplRecastNavMesh::GetPolyCenter(NavNodeRef PolyID, FVector& OutCenter) const
{
	if (DetourNavMesh)
	{
		// get poly data from recast
		dtPoly const* Poly;
		dtMeshTile const* Tile;
		dtStatus Status = DetourNavMesh->getTileAndPolyByRef((dtPolyRef)PolyID, &Tile, &Poly);
		if (dtStatusSucceed(Status))
		{
			// average verts
			FVector::FReal Center[3] = {0,0,0};

			for (uint32 VertIdx=0; VertIdx < Poly->vertCount; ++VertIdx)
			{
				FVector::FReal V[3];
				Tile->verts[Poly->verts[VertIdx]].toWorld(Tile->header->bmin, V);
				Center[0] += V[0];
				Center[1] += V[1];
				Center[2] += V[2];
			}
			const FVector::FReal InvCount = 1.0f / Poly->vertCount;
			Center[0] *= InvCount;
			Center[1] *= InvCount;
			Center[2] *= InvCount;

			// convert output to UE coords
			OutCenter = Recast2UnrVector(Center);

			return true;
		}
	}

	return false;
}

bool FPImplRecastNavMesh::GetPolyVerts(NavNodeRef PolyID, TArray<FVector>& OutVerts) const
{
	if (DetourNavMesh)
	{
		// get poly data from recast
		dtPoly const* Poly;
		dtMeshTile const* Tile;
		dtStatus Status = DetourNavMesh->getTileAndPolyByRef((dtPolyRef)PolyID, &Tile, &Poly);
		if (dtStatusSucceed(Status))
		{
			// flush and pre-size output array
			OutVerts.Reset(Poly->vertCount);

			// convert to UE coords and copy verts into output array 
			for (uint32 VertIdx=0; VertIdx < Poly->vertCount; ++VertIdx)
			{
				FVector::FReal V[3];
				Tile->verts[Poly->verts[VertIdx]].toWorld(Tile->header->bmin, V);
				OutVerts.Add( Recast2UnrVector(V) );
			}

#if WITH_NAVMESH_SEGMENT_LINKS
			// Segment polys store each segment edge from left to right.
			// Therefore, swap the edge stored at indices 2 and 3 to ensure the poly verts
			// are in proper clockwise order for drawing to avoid twisted segment link polys.
			if (Poly->getType() == DT_POLYTYPE_OFFMESH_SEGMENT)
			{
				OutVerts.SwapMemory(2, 3);
			}
#endif // WITH_NAVMESH_SEGMENT_LINKS

			return true;
		}
	}

	return false;
}

bool FPImplRecastNavMesh::GetRandomPointInPoly(NavNodeRef PolyID, FVector& OutPoint) const
{
	if (DetourNavMesh)
	{
		INITIALIZE_NAVQUERY_SIMPLE(NavQuery, RECAST_MAX_SEARCH_NODES);

		FVector::FReal RandPt[3];
		dtStatus Status = NavQuery.findRandomPointInPoly((dtPolyRef)PolyID, FMath::FRand, RandPt);
		if (dtStatusSucceed(Status))
		{
			OutPoint = Recast2UnrVector(RandPt);
			return true;
		}
	}

	return false;
}

FVector::FReal FPImplRecastNavMesh::GetPolySurfaceArea(NavNodeRef PolyID) const
{
	if (DetourNavMesh)
	{
		dtPoly const* Poly = 0;
		dtMeshTile const* Tile = 0;
		dtStatus Status = DetourNavMesh->getTileAndPolyByRef((dtPolyRef)PolyID, &Tile, &Poly);
		if (dtStatusSucceed(Status))
		{
			// Calc area of the polygon.
			dtReal PolyArea = 0;
			for (int j = 2; j < Poly->vertCount; ++j)
			{
				dtReal VA[3], VB[3], VC[3];
				Tile->verts[Poly->verts[0]].toWorld(Tile->header->bmin, VA);
				Tile->verts[Poly->verts[j - 1]].toWorld(Tile->header->bmin, VB);
				Tile->verts[Poly->verts[j]].toWorld(Tile->header->bmin, VC);
				PolyArea += dtTriArea2D(VA, VB, VC);
			}

			return (FVector::FReal)PolyArea;
		}
	}

	return 0;
}

uint32 FPImplRecastNavMesh::GetPolyAreaID(NavNodeRef PolyID) const
{
	uint32 AreaID = RECAST_NULL_AREA;

	if (DetourNavMesh)
	{
		// get poly data from recast
		dtPoly const* Poly;
		dtMeshTile const* Tile;
		dtStatus Status = DetourNavMesh->getTileAndPolyByRef((dtPolyRef)PolyID, &Tile, &Poly);
		if (dtStatusSucceed(Status))
		{
			AreaID = Poly->getArea();
		}
	}

	return AreaID;
}

void FPImplRecastNavMesh::SetPolyAreaID(NavNodeRef PolyID, uint8 AreaID)
{
	if (DetourNavMesh)
	{
		DetourNavMesh->setPolyArea((dtPolyRef)PolyID, AreaID);
	}
}

bool FPImplRecastNavMesh::GetPolyData(NavNodeRef PolyID, uint16& Flags, uint8& AreaType) const
{
	if (DetourNavMesh)
	{
		// get poly data from recast
		dtPoly const* Poly;
		dtMeshTile const* Tile;
		dtStatus Status = DetourNavMesh->getTileAndPolyByRef((dtPolyRef)PolyID, &Tile, &Poly);
		if (dtStatusSucceed(Status))
		{
			Flags = Poly->flags;
			AreaType = Poly->getArea();
			return true;
		}
	}

	return false;
}

bool FPImplRecastNavMesh::GetPolyNeighbors(NavNodeRef PolyID, TArray<FNavigationPortalEdge>& Neighbors) const
{
	if (DetourNavMesh)
	{
		dtPolyRef PolyRef = (dtPolyRef)PolyID;
		dtPoly const* Poly = 0;
		dtMeshTile const* Tile = 0;

		dtStatus Status = DetourNavMesh->getTileAndPolyByRef(PolyRef, &Tile, &Poly);
		if (dtStatusSucceed(Status))
		{
			INITIALIZE_NAVQUERY_SIMPLE(NavQuery, RECAST_MAX_SEARCH_NODES);

			FVector::FReal RcLeft[3], RcRight[3];
			uint8 DummyType1, DummyType2;

			uint32 LinkIdx = Poly->firstLink;
			while (LinkIdx != DT_NULL_LINK)
			{
				const dtLink& Link = DetourNavMesh->getLink(Tile, LinkIdx);
				LinkIdx = Link.next;
				
				Status = NavQuery.getPortalPoints(PolyRef, Link.ref, RcLeft, RcRight, DummyType1, DummyType2);
				if (dtStatusSucceed(Status))
				{
					FNavigationPortalEdge NeiData;
					NeiData.ToRef = Link.ref;
					NeiData.Left = Recast2UnrealPoint(RcLeft);
					NeiData.Right = Recast2UnrealPoint(RcRight);

					Neighbors.Add(NeiData);
				}
			}

			return true;
		}
	}

	return false;
}

bool FPImplRecastNavMesh::GetPolyNeighbors(NavNodeRef PolyID, TArray<NavNodeRef>& Neighbors) const
{
	if (DetourNavMesh)
	{
		const dtPolyRef PolyRef = static_cast<dtPolyRef>(PolyID);
		dtPoly const* Poly = 0;
		dtMeshTile const* Tile = 0;

		const dtStatus Status = DetourNavMesh->getTileAndPolyByRef(PolyRef, &Tile, &Poly);

		if (dtStatusSucceed(Status))
		{
			uint32 LinkIdx = Poly->firstLink;
			Neighbors.Reserve(DT_VERTS_PER_POLYGON);

			while (LinkIdx != DT_NULL_LINK)
			{
				const dtLink& Link = DetourNavMesh->getLink(Tile, LinkIdx);
				LinkIdx = Link.next;

				Neighbors.Add(Link.ref);
			}

			return true;
		}
	}

	return false;
}

bool FPImplRecastNavMesh::GetPolyEdges(NavNodeRef PolyID, TArray<FNavigationPortalEdge>& Edges) const
{
	if (DetourNavMesh)
	{
		dtPolyRef PolyRef = (dtPolyRef)PolyID;
		dtPoly const* Poly = 0;
		dtMeshTile const* Tile = 0;

		dtStatus Status = DetourNavMesh->getTileAndPolyByRef(PolyRef, &Tile, &Poly);
		if (dtStatusSucceed(Status))
		{
			const bool bIsNavLink = (Poly->getType() != DT_POLYTYPE_GROUND);

			for (uint32 LinkIt = Poly->firstLink; LinkIt != DT_NULL_LINK;)
			{
				const dtLink& LinkInfo = DetourNavMesh->getLink(Tile, LinkIt);
				if (LinkInfo.edge >= 0 && LinkInfo.edge < Poly->vertCount)
				{
					FNavigationPortalEdge NeiData;
					dtReal VLeft[3], VRight[3];
					Tile->verts[Poly->verts[LinkInfo.edge]].toWorld(Tile->header->bmin, VLeft);
					Tile->verts[Poly->verts[(LinkInfo.edge + 1) % Poly->vertCount]].toWorld(Tile->header->bmin, VRight);
					NeiData.Left = Recast2UnrealPoint(VLeft);
					NeiData.Right = bIsNavLink ? NeiData.Left : Recast2UnrealPoint(VRight);
					NeiData.ToRef = LinkInfo.ref;
					Edges.Add(NeiData);
				}

				LinkIt = LinkInfo.next;
			}

			return true;
		}
	}

	return false;
}

bool FPImplRecastNavMesh::GetPolyWallSegments(NavNodeRef PolyID, const FNavigationQueryFilter& InQueryFilter, const UObject* QueryOwner, TArray<FNavigationPortalEdge>& OutNeighbors) const
{
	const FRecastQueryFilter* FilterImplementation = (const FRecastQueryFilter*)(InQueryFilter.GetImplementation());
	if (FilterImplementation == nullptr)
	{
		UE_VLOG(NavMeshOwner, LogNavigation, Error, TEXT("%hs failed due to passed filter having NULL implementation!"), __FUNCTION__);
		return false;
	}

	const dtQueryFilter* QueryFilter = FilterImplementation->GetAsDetourQueryFilter();
	if (QueryFilter == nullptr)
	{
		UE_VLOG(NavMeshOwner, LogNavigation, Warning, TEXT("%hs failed due to QueryFilter == nullptr"), __FUNCTION__);
		return false;
	}
	
	FRecastSpeciaLinkFilter LinkFilter(FNavigationSystem::GetCurrent<UNavigationSystemV1>(NavMeshOwner->GetWorld()), QueryOwner);
	INITIALIZE_NAVQUERY(NavQuery, InQueryFilter.GetMaxSearchNodes(), LinkFilter);

	constexpr int32 MaxSegments = 64;
	constexpr int32 ComponentsPerSegment = 6;
	dtReal RcVertices[MaxSegments * ComponentsPerSegment] = { 0 }; // segments * ax,ay,az,bx,by,bz

	int32 NumSegments = 0;
	dtPolyRef SegmentRefs[MaxSegments] = { 0 };
	
	if (dtStatusSucceed(NavQuery.getPolyWallSegments(PolyID, QueryFilter, RcVertices, SegmentRefs, &NumSegments, MaxSegments)))
	{
		OutNeighbors.SetNum(NumSegments);
	
		for (int32 i = 0; i < NumSegments; ++i)
		{
			FNavigationPortalEdge& Edge = OutNeighbors[i];
			Edge.Left = Recast2UnrealPoint(&RcVertices[i * ComponentsPerSegment]);
			Edge.Right = Recast2UnrealPoint(&RcVertices[i * ComponentsPerSegment + 3]);
			Edge.ToRef = SegmentRefs[i];
		}
		return true;
	}

	OutNeighbors.SetNum(0);
	return false;
}

bool FPImplRecastNavMesh::GetPolyTileIndex(NavNodeRef PolyID, uint32& PolyIndex, uint32& TileIndex) const
{
	if (DetourNavMesh && PolyID)
	{
		uint32 SaltIdx = 0;
		DetourNavMesh->decodePolyId(PolyID, SaltIdx, TileIndex, PolyIndex);
		return true;
	}

	return false;
}

bool FPImplRecastNavMesh::GetPolyTileRef(NavNodeRef PolyId, uint32& OutPolyIndex, FNavTileRef& OutTileRef) const
{
	if (DetourNavMesh && PolyId)
	{
		// Similar to UE::NavMesh::Private::GetTileRefFromPolyRef
		unsigned int Salt = 0;
		unsigned int TileIndex = 0;
		DetourNavMesh->decodePolyId(PolyId, Salt, TileIndex, OutPolyIndex);
		const dtTileRef TileRef = DetourNavMesh->encodePolyId(Salt, TileIndex, 0);
		OutTileRef = FNavTileRef(TileRef);
		return true;
	}

	return false;
}

bool FPImplRecastNavMesh::GetClosestPointOnPoly(NavNodeRef PolyID, const FVector& TestPt, FVector& PointOnPoly) const
{
	if (DetourNavMesh && PolyID)
	{
		INITIALIZE_NAVQUERY_SIMPLE(NavQuery, RECAST_MAX_SEARCH_NODES);

		FVector::FReal RcTestPos[3] = { 0.0f };
		FVector::FReal RcClosestPos[3] = { 0.0f };
		Unr2RecastVector(TestPt, RcTestPos);

		const dtStatus Status = NavQuery.closestPointOnPoly(PolyID, RcTestPos, RcClosestPos);
		if (dtStatusSucceed(Status))
		{
			PointOnPoly = Recast2UnrealPoint(RcClosestPos);
			return true;
		}
	}

	return false;
}

FNavLinkId FPImplRecastNavMesh::GetNavLinkUserId(NavNodeRef LinkPolyID) const
{
	const dtOffMeshConnection* offmeshCon = DetourNavMesh ? DetourNavMesh->getOffMeshConnectionByRef(LinkPolyID) : nullptr;

	return offmeshCon ? FNavLinkId(offmeshCon->userId) : FNavLinkId::Invalid;
}

bool FPImplRecastNavMesh::GetLinkEndPoints(NavNodeRef LinkPolyID, FVector& PointA, FVector& PointB) const
{
	if (DetourNavMesh)
	{
		FVector::FReal RcPointA[3] = { 0 };
		FVector::FReal RcPointB[3] = { 0 };
		
		dtStatus status = DetourNavMesh->getOffMeshConnectionPolyEndPoints(0, LinkPolyID, 0, RcPointA, RcPointB);
		if (dtStatusSucceed(status))
		{
			PointA = Recast2UnrealPoint(RcPointA);
			PointB = Recast2UnrealPoint(RcPointB);
			return true;
		}
	}

	return false;
}

bool FPImplRecastNavMesh::IsCustomLink(NavNodeRef PolyRef) const
{
	if (DetourNavMesh)
	{
		const dtOffMeshConnection* offMeshCon = DetourNavMesh->getOffMeshConnectionByRef(PolyRef);
		return offMeshCon && FNavLinkId(offMeshCon->userId) != FNavLinkId::Invalid;
	}

	return false;
}

#if WITH_NAVMESH_CLUSTER_LINKS
bool FPImplRecastNavMesh::GetClusterBounds(NavNodeRef ClusterRef, FBox& OutBounds) const
{
	if (DetourNavMesh == NULL || !ClusterRef)
	{
		return false;
	}

	const dtMeshTile* Tile = DetourNavMesh->getTileByRef(ClusterRef);
	uint32 ClusterIdx = DetourNavMesh->decodeClusterIdCluster(ClusterRef);

	int32 NumPolys = 0;
	if (Tile && ClusterIdx < (uint32)Tile->header->clusterCount)
	{
		for (int32 i = 0; i < Tile->header->offMeshBase; i++)
		{
			if (Tile->polyClusters[i] == ClusterIdx)
			{
				const dtPoly* Poly = &Tile->polys[i];
				for (int32 iVert = 0; iVert < Poly->vertCount; iVert++)
				{
					FVector::FReal V[3];
					Tile->verts[Poly->verts[iVert]].toWorld(Tile->header->bmin, V);
					OutBounds += Recast2UnrealPoint(V);
				}

				NumPolys++;
			}
		}
	}

	return NumPolys > 0;
}
#endif // WITH_NAVMESH_CLUSTER_LINKS

void FPImplRecastNavMesh::GetEdgesForPathCorridorImpl(const TArray<NavNodeRef>* PathCorridor, TArray<FNavigationPortalEdge>* PathCorridorEdges, const dtNavMeshQuery& NavQuery) const
{
	const int32 CorridorLength = PathCorridor->Num();

	PathCorridorEdges->Empty(FMath::Max(CorridorLength - 1, 0));
	for (int32 i = 0; i < CorridorLength - 1; ++i)
	{
		unsigned char FromType = 0, ToType = 0;
		FVector::FReal Left[3] = {0.f}, Right[3] = {0.f};

		NavQuery.getPortalPoints((*PathCorridor)[i], (*PathCorridor)[i+1], Left, Right, FromType, ToType);

		PathCorridorEdges->Add(FNavigationPortalEdge(Recast2UnrVector(Left), Recast2UnrVector(Right), (*PathCorridor)[i+1]));
	}
}

void FPImplRecastNavMesh::GetEdgesForPathCorridor(const TArray<NavNodeRef>* PathCorridor, TArray<FNavigationPortalEdge>* PathCorridorEdges) const
{
	// sanity check
	if (DetourNavMesh == NULL)
	{
		return;
	}

	INITIALIZE_NAVQUERY_SIMPLE(NavQuery, RECAST_MAX_SEARCH_NODES);

	GetEdgesForPathCorridorImpl(PathCorridor, PathCorridorEdges, NavQuery);
}

bool FPImplRecastNavMesh::FilterPolys(TArray<NavNodeRef>& PolyRefs, const FRecastQueryFilter* Filter, const UObject* Owner) const
{
	if (Filter == NULL || DetourNavMesh == NULL)
	{
		return false;
	}

	for (int32 PolyIndex = PolyRefs.Num() - 1; PolyIndex >= 0; --PolyIndex)
	{
		dtPolyRef TestRef = PolyRefs[PolyIndex];

		// get poly data from recast
		dtPoly const* Poly = NULL;
		dtMeshTile const* Tile = NULL;
		const dtStatus Status = DetourNavMesh->getTileAndPolyByRef(TestRef, &Tile, &Poly);

		if (dtStatusSucceed(Status))
		{
			const bool bPassedFilter = Filter->passFilter(TestRef, Tile, Poly);
			const bool bWalkableArea = Filter->getAreaCost(Poly->getArea()) > 0.0f;
			if (bPassedFilter && bWalkableArea)
			{
				continue;
			}
		}
		
		PolyRefs.RemoveAt(PolyIndex, 1);
	}

	return true;
}

// Deprecated
bool FPImplRecastNavMesh::GetPolysInTile(int32 TileIndex, TArray<FNavPoly>& Polys) const
{
	if (DetourNavMesh == NULL || TileIndex < 0 || TileIndex >= DetourNavMesh->getMaxTiles())
	{
		return false;
	}

	const dtMeshTile* Tile = ((const dtNavMesh*)DetourNavMesh)->getTile(TileIndex);
	const int32 MaxPolys = Tile && Tile->header ? Tile->header->offMeshBase : 0;
	if (MaxPolys > 0)
	{
		// only ground type polys
		int32 BaseIdx = Polys.Num();
		Polys.AddZeroed(MaxPolys);

		dtPoly* Poly = Tile->polys;
		for (int32 i = 0; i < MaxPolys; i++, Poly++)
		{
			FVector PolyCenter(0);
			for (int k = 0; k < Poly->vertCount; ++k)
			{
				dtReal V[3];
				Tile->verts[Poly->verts[k]].toWorld(Tile->header->bmin, V);
				PolyCenter += Recast2UnrealPoint(V);
			}
			PolyCenter /= Poly->vertCount;

			FNavPoly& OutPoly = Polys[BaseIdx + i];
			OutPoly.Ref = DetourNavMesh->encodePolyId(Tile->salt, TileIndex, i);
			OutPoly.Center = PolyCenter;
		}
	}

	return (MaxPolys > 0);
}

/** Internal. Calculates squared 2d distance of given point PT to segment P-Q. Values given in Recast coordinates */
static FORCEINLINE FVector::FReal PointDistToSegment2DSquared(const FVector::FReal* PT, const FVector::FReal* P, const FVector::FReal* Q)
{
	FVector::FReal pqx = Q[0] - P[0];
	FVector::FReal pqz = Q[2] - P[2];
	FVector::FReal dx = PT[0] - P[0];
	FVector::FReal dz = PT[2] - P[2];
	FVector::FReal d = pqx*pqx + pqz*pqz;
	FVector::FReal t = pqx*dx + pqz*dz;
	if (d != 0) t /= d;
	dx = P[0] + t*pqx - PT[0];
	dz = P[2] + t*pqz - PT[2];
	return dx*dx + dz*dz;
}

// Deprecated
void FPImplRecastNavMesh::GetDebugPolyEdges(const dtMeshTile& Tile, bool bInternalEdges, bool bNavMeshEdges, TArray<FVector>& InternalEdgeVerts, TArray<FVector>& NavMeshEdgeVerts) const
{
	GetTilePolyEdges(Tile, bInternalEdges, bNavMeshEdges, InternalEdgeVerts, NavMeshEdgeVerts);
}

void FPImplRecastNavMesh::GetTilePolyEdgesForFilter(const dtMeshTile& Tile, const FNavigationQueryFilter& InQueryFilter, TArray<FNavigationWallEdge>& OutEdges) const
{
	const FRecastQueryFilter* FilterImplementation = (const FRecastQueryFilter*)(InQueryFilter.GetImplementation());
	if (!FilterImplementation)
	{
		return;
	}

	const dtQueryFilter* QueryFilter = FilterImplementation->GetAsDetourQueryFilter();
	for (int i = 0; i < Tile.header->polyCount; ++i)
	{
		const dtPoly* Poly = &Tile.polys[i];

		if (Poly->getType() != DT_POLYTYPE_GROUND)
		{
			continue;
		}

		const dtPolyRef Base = DetourNavMesh->getPolyRefBase(&Tile);
		auto IsPolyExterior = [QueryFilter](const dtPoly& Poly, const dtMeshTile& Tile, const dtPolyRef PolyIndex, const dtPolyRef Base) -> bool
			{
				if (!QueryFilter)
				{
					return Poly.getArea() == RECAST_NULL_AREA;
				}

				return !QueryFilter->passFilter(Base | (dtPolyRef)PolyIndex, &Tile, &Poly);
			};

		const bool bIsPolyiExterior = IsPolyExterior(*Poly, Tile, (dtPolyRef)i, Base);
		for (int j = 0, nj = (int)Poly->vertCount; j < nj; ++j)
		{
			if (bIsPolyiExterior)
			{
				const bool bIsPolyjLink = (Poly->neis[j] & DT_EXT_LINK) != 0;
				if (bIsPolyjLink)
				{
					// Recover poly j from the neighbor tile and test the filter
					unsigned int k = Poly->firstLink;
					while (k != DT_NULL_LINK)
					{
						const dtLink& Link = DetourNavMesh->getLink(&Tile, k);
						if (Link.edge == j)
						{
							const dtMeshTile* PolyjTile;
							const dtPoly* Polyj;
							const dtStatus Status = DetourNavMesh->getTileAndPolyByRef(Link.ref, &PolyjTile, &Polyj);
							if (dtStatusSucceed(Status) && Polyj && PolyjTile)
							{
								if (!IsPolyExterior(*Polyj, *PolyjTile, Link.ref, DetourNavMesh->getPolyRefBase(PolyjTile)))
								{
									// Poly i is interior and neighbor j is exterior
									FVector::FReal V0[3], V1[3];
									Tile.verts[Poly->verts[Link.edge]].toWorld(Tile.header->bmin, V0);
									Tile.verts[Poly->verts[(Link.edge + 1) % nj]].toWorld(Tile.header->bmin, V1);
									OutEdges.Emplace(Recast2UnrVector(V0), Recast2UnrVector(V1));
								}
							}

							break;
						}
						k = Link.next;
					}
				}
				else
				{
					// Check if j is interior to mark that edge
					const bool bIsPolyjExterior = Poly->neis[j] == 0 || IsPolyExterior(Tile.polys[Poly->neis[j] - 1], Tile, Poly->neis[j] - 1, Base);
					if (!bIsPolyjExterior)
					{
						// Poly i is interior and neighbor j is exterior
						FVector::FReal V0[3], V1[3];
						Tile.verts[Poly->verts[j]].toWorld(Tile.header->bmin, V0);
						Tile.verts[Poly->verts[(j + 1) % nj]].toWorld(Tile.header->bmin, V1);
						OutEdges.Emplace(Recast2UnrVector(V0), Recast2UnrVector(V1));
					}
				}
			}
			else
			{
				// Here we only need to check for no poly since the case where i is Interior and j is Exterior through filter is handled above
				// with the polygons interchanged and we don't want to duplicate the edges. The inverted case doesn't happen if j is not a poly since 
				// the for loop won't iterate on it
				const bool bIsPolyjOutsideNavmesh = Poly->neis[j] == 0;
				if (bIsPolyjOutsideNavmesh)
				{
					// Poly i is interior and neighbor j is exterior
					// Swap the vertices to keep the same edges orientations
					FVector::FReal V0[3], V1[3];
					Tile.verts[Poly->verts[(j + 1) % nj]].toWorld(Tile.header->bmin, V0);
					Tile.verts[Poly->verts[j]].toWorld(Tile.header->bmin, V1);
					OutEdges.Emplace(Recast2UnrVector(V0), Recast2UnrVector(V1));
				}
			}
		}
	}
}

void FPImplRecastNavMesh::GetTilePolyEdges(const dtMeshTile& Tile, bool bGatherInteriorPolyEdges, bool bGatherExteriorNavMeshEdges, TArray<FVector>& OutInteriorPolyEdgeVerts, TArray<FVector>& OutExteriorNavMeshEdgeVerts) const
{
	static const FVector::FReal thr = FMath::Square(0.01f);

	ensure(bGatherInteriorPolyEdges || bGatherExteriorNavMeshEdges);
	const bool bExportAllEdges = bGatherInteriorPolyEdges && !bGatherExteriorNavMeshEdges;
	
	for (int i = 0; i < Tile.header->polyCount; ++i)
	{
		const dtPoly* Poly = &Tile.polys[i];

		if (Poly->getType() != DT_POLYTYPE_GROUND)
		{
			continue;
		}

		const dtPolyDetail* pd = &Tile.detailMeshes[i];
		for (int j = 0, nj = (int)Poly->vertCount; j < nj; ++j)
		{
			bool bIsExternal = !bExportAllEdges && (Poly->neis[j] == 0 || Poly->neis[j] & DT_EXT_LINK);
			bool bIsConnected = !bIsExternal;

			if (Poly->getArea() == RECAST_NULL_AREA)
			{
				if (Poly->neis[j] && !(Poly->neis[j] & DT_EXT_LINK) &&
					Poly->neis[j] <= Tile.header->offMeshBase &&
					Tile.polys[Poly->neis[j] - 1].getArea() != RECAST_NULL_AREA)
				{
					bIsExternal = true;
					bIsConnected = false;
				}
				else if (Poly->neis[j] == 0)
				{
					bIsExternal = true;
					bIsConnected = false;
				}
			}
			else if (bIsExternal)
			{
				unsigned int k = Poly->firstLink;
				while (k != DT_NULL_LINK)
				{
					const dtLink& link = DetourNavMesh->getLink(&Tile, k);
					k = link.next;

					if (link.edge == j)
					{
						bIsConnected = true;
						break;
					}
				}
			}

			TArray<FVector>* EdgeVerts = bGatherInteriorPolyEdges && bIsConnected ? &OutInteriorPolyEdgeVerts 
				: (bGatherExteriorNavMeshEdges && bIsExternal && !bIsConnected ? &OutExteriorNavMeshEdgeVerts : NULL);
			if (EdgeVerts == NULL)
			{
				continue;
			}

			FVector::FReal V0[3], V1[3];
			Tile.verts[Poly->verts[j]].toWorld(Tile.header->bmin, V0);
			Tile.verts[Poly->verts[(j + 1) % nj]].toWorld(Tile.header->bmin, V1);

			// Gather detail mesh edges which align with the actual poly edge.
			// This is really slow.
			for (int32 k = 0; k < pd->triCount; ++k)
			{
				const unsigned char* t = &(Tile.detailTris[(pd->triBase + k) * 4]);
				FVector::FReal tvBuf[3][3];
				const FVector::FReal* tv[3];

				for (int32 m = 0; m < 3; ++m)
				{
					if (t[m] < Poly->vertCount)
					{
						Tile.verts[Poly->verts[t[m]]].toWorld(Tile.header->bmin, tvBuf[m]);
						tv[m] = tvBuf[m];
					}
					else
					{
						Tile.detailVerts[pd->vertBase + (t[m] - Poly->vertCount)].toWorld(Tile.header->bmin, tvBuf[m]);
						tv[m] = tvBuf[m];
					}
				}
				for (int m = 0, n = 2; m < 3; n=m++)
				{
					if (((t[3] >> (n*2)) & 0x3) == 0)
					{
						continue;	// Skip inner detail edges.
					}
					
					if (PointDistToSegment2DSquared(tv[n],V0,V1) < thr && PointDistToSegment2DSquared(tv[m],V0,V1) < thr)
					{
						int32 const AddIdx = (*EdgeVerts).AddZeroed(2);
						(*EdgeVerts)[AddIdx] = Recast2UnrVector(tv[n]);
						(*EdgeVerts)[AddIdx+1] = Recast2UnrVector(tv[m]);
					}
				}
			}
		}
	}
}

uint8 GetValidEnds(const dtNavMesh& NavMesh, const dtMeshTile& Tile, const dtPoly& Poly, const bool bIsReversed = false)
{
	if (Poly.getType() == DT_POLYTYPE_GROUND)
	{
		return false;
	}

	uint8 ValidEnds = FRecastDebugGeometry::OMLE_None;

	unsigned int k = Poly.firstLink;
	while (k != DT_NULL_LINK)
	{
		const dtLink& link = NavMesh.getLink(&Tile, k);
		k = link.next;

		if (link.edge == 0)
		{
			ValidEnds |= bIsReversed ? FRecastDebugGeometry::OMLE_Right : FRecastDebugGeometry::OMLE_Left;
		}
		if (link.edge == 1)
		{
			ValidEnds |= bIsReversed ? FRecastDebugGeometry::OMLE_Left : FRecastDebugGeometry::OMLE_Right;
		}
	}

	return ValidEnds;
}

// Deprecated
bool FPImplRecastNavMesh::GetDebugGeometryForTile(FRecastDebugGeometry& OutGeometry, int32 TileIndex) const
{
	FNavTileRef TileRef;
	if (TileIndex != INDEX_NONE)
	{
		TArray<FNavTileRef> TileRefs;
		FNavTileRef::DeprecatedMakeTileRefsFromTileIds(this, { static_cast<uint32>(TileIndex) }, TileRefs);
		TileRef = TileRefs[0];
	}
	return GetDebugGeometryForTile(OutGeometry, TileRef);
}

bool FPImplRecastNavMesh::GetDebugGeometryForTile(FRecastDebugGeometry& OutGeometry, FNavTileRef TileRef) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FPImplRecastNavMesh_GetDebugGeometryForTile);
	
	bool bDone = false;
	if (DetourNavMesh == nullptr)
	{
		bDone = true;
		return bDone;
	}
				
	check(NavMeshOwner);

	const dtNavMesh* const ConstNavMesh = DetourNavMesh;

	// presize our tarrays for efficiency
	int32 NumVertsToReserve = 0;
	int32 NumIndicesToReserve = 0;

	const uint16 ForbiddenFlags = OutGeometry.bMarkForbiddenPolys 
		? GetFilterForbiddenFlags((const FRecastQueryFilter*)NavMeshOwner->GetDefaultQueryFilterImpl()) 
		: 0;

	const FRecastNavMeshGenerator* Generator = static_cast<const FRecastNavMeshGenerator*>(NavMeshOwner->GetGenerator());
	const bool bIsGenerationRestrictedToActiveTiles = Generator && Generator->IsBuildingRestrictedToActiveTiles() && !NavMeshOwner->GetActiveTileSet().IsEmpty();

	auto ComputeSizeToReserve = [](dtMeshTile const* const Tile, int32& OutNumVertsToReserve, int32& OutNumIndicesToReserve)
	{
		if (Tile != nullptr && Tile->header != nullptr)
		{
			OutNumVertsToReserve += Tile->header->vertCount + Tile->header->detailVertCount;

			for (int32 PolyIdx = 0; PolyIdx < Tile->header->polyCount; ++PolyIdx)
			{
				dtPolyDetail const* const DetailPoly = &Tile->detailMeshes[PolyIdx];
				OutNumIndicesToReserve += (DetailPoly->triCount * 3);
			}
		}
	};

	auto ReserveGeometryArrays = [](FRecastDebugGeometry& OutGeometry, const int32 NumVertsToReserve, const int32 NumIndicesToReserve)
	{
		OutGeometry.MeshVerts.Reserve(OutGeometry.MeshVerts.Num() + NumVertsToReserve);
		OutGeometry.AreaIndices[0].Reserve(OutGeometry.AreaIndices[0].Num() + NumIndicesToReserve);
		OutGeometry.BuiltMeshIndices.Reserve(OutGeometry.BuiltMeshIndices.Num() + NumIndicesToReserve);
		for (int32 Index = 0; Index < FRecastDebugGeometry::BuildTimeBucketsCount; Index++)
		{
			OutGeometry.TileBuildTimesIndices[Index].Reserve(OutGeometry.TileBuildTimesIndices[Index].Num() + NumIndicesToReserve);	
		}
	};

	if (TileRef.IsValid())
	{
		dtMeshTile const* const Tile = ConstNavMesh->getTileByRef(static_cast<dtTileRef>(TileRef));
		if (Tile != nullptr && Tile->header != nullptr)
		{
			const FIntPoint TileCoord = FIntPoint(Tile->header->x, Tile->header->y);
			if (!bIsGenerationRestrictedToActiveTiles || Generator->IsInActiveSet(TileCoord))
			{
				ComputeSizeToReserve(Tile, NumVertsToReserve, NumIndicesToReserve);

				ReserveGeometryArrays(OutGeometry, NumVertsToReserve, NumIndicesToReserve);

				const uint32 VertBase = OutGeometry.MeshVerts.Num();
				GetTilesDebugGeometry(Generator, *Tile, VertBase, OutGeometry, ConstNavMesh->getTileIndex(Tile), ForbiddenFlags);
			}
		}
	}
	else if (bIsGenerationRestrictedToActiveTiles)
	{
		TArray<const dtMeshTile*> Tiles;
		const TSet<FIntPoint>& ActiveTiles = NavMeshOwner->GetActiveTileSet();
		for (const FIntPoint& TileLocation : ActiveTiles)
		{
			Tiles.Reset();
			Tiles.AddZeroed(ConstNavMesh->getTileCountAt(TileLocation.X, TileLocation.Y));
			ConstNavMesh->getTilesAt(TileLocation.X, TileLocation.Y, Tiles.GetData(), Tiles.Num());
			for (const dtMeshTile* Tile : Tiles)
			{
				ComputeSizeToReserve(Tile, NumVertsToReserve, NumIndicesToReserve);
			}
		}

		ReserveGeometryArrays(OutGeometry, NumVertsToReserve, NumIndicesToReserve);

		uint32 VertBase = OutGeometry.MeshVerts.Num();
		for (const FIntPoint& TileLocation : ActiveTiles)
		{
			Tiles.Reset();
			Tiles.AddZeroed(ConstNavMesh->getTileCountAt(TileLocation.X, TileLocation.Y));
			ConstNavMesh->getTilesAt(TileLocation.X, TileLocation.Y, Tiles.GetData(), Tiles.Num());
			for (const dtMeshTile* Tile : Tiles)
			{
				if (Tile != nullptr && Tile->header != nullptr)
				{
					VertBase += GetTilesDebugGeometry(Generator, *Tile, VertBase, OutGeometry, INDEX_NONE, ForbiddenFlags);
				}
			}
		}

		bDone = true;
	}
	else
	{
		const int32 NumTiles = ConstNavMesh->getMaxTiles();
		for (int32 TileIdx = 0; TileIdx < NumTiles; ++TileIdx)
		{
			dtMeshTile const* const Tile = ConstNavMesh->getTile(TileIdx);
			ComputeSizeToReserve(Tile, NumVertsToReserve, NumIndicesToReserve);
		}

		ReserveGeometryArrays(OutGeometry, NumVertsToReserve, NumIndicesToReserve);

		uint32 VertBase = OutGeometry.MeshVerts.Num();
		for (int32 TileIdx = 0; TileIdx < NumTiles; ++TileIdx)
		{
			dtMeshTile const* const Tile = ConstNavMesh->getTile(TileIdx);
			if (Tile != nullptr && Tile->header != nullptr)
			{
				VertBase += GetTilesDebugGeometry(Generator, *Tile, VertBase, OutGeometry, TileIdx, ForbiddenFlags);
			}
		}

		bDone = true;
	}

	return bDone;
}

int32 FPImplRecastNavMesh::GetTilesDebugGeometry(const FRecastNavMeshGenerator* Generator, const dtMeshTile& Tile, int32 VertBase, FRecastDebugGeometry& OutGeometry, int32 TileIdx, uint16 ForbiddenFlags) const
{
	check(NavMeshOwner && DetourNavMesh);
	dtMeshHeader const* const Header = Tile.header;
	check(Header);

#if RECAST_INTERNAL_DEBUG_DATA
	OutGeometry.TilesToDisplayInternalData.Push(FIntPoint(Header->x, Header->y));
#endif

	const bool bIsBeingBuilt = Generator != nullptr && !!NavMeshOwner->bDistinctlyDrawTilesBeingBuilt
		&& Generator->IsTileChanged(FNavTileRef(DetourNavMesh->getTileRef(&Tile)));

	UE_SUPPRESS(LogNavigation, VeryVerbose,
	{
		if (bIsBeingBuilt)
		{
			const dtTileRef TileRef = DetourNavMesh->getTileRef(&Tile);
			UE_LOGF(LogNavigation, VeryVerbose, "%s TileId: %d Salt: %d TileRef: 0x%llx bIsBeingBuilt",
				__FUNCTION__, DetourNavMesh->decodePolyIdTile(TileRef), DetourNavMesh->decodePolyIdSalt(TileRef), TileRef);	
		}
	});

	// add all the poly verts
	for (int32 VertIdx = 0; VertIdx < Header->vertCount; ++VertIdx)
	{
		FVector::FReal V[3];
		Tile.verts[VertIdx].toWorld(Header->bmin, V);
		FVector const VertPos = Recast2UnrVector(V);
		OutGeometry.MeshVerts.Add(VertPos);
	}

	int32 const DetailVertIndexBase = Header->vertCount;
	// add the detail verts
	for (int32 DetailVertIdx = 0; DetailVertIdx < Header->detailVertCount; ++DetailVertIdx)
	{
		FVector::FReal V[3];
		Tile.detailVerts[DetailVertIdx].toWorld(Header->bmin, V);
		FVector const VertPos = Recast2UnrVector(V);
		OutGeometry.MeshVerts.Add(VertPos);
	}

#if RECAST_INTERNAL_DEBUG_DATA	
	const FIntPoint TileCoord(Header->x, Header->y);
	const FRecastInternalDebugData* DebugData = DebugDataMap.Find(TileCoord);
#endif // RECAST_INTERNAL_DEBUG_DATA	
	
	// add all the indices
	for (int32 PolyIdx = 0; PolyIdx < Header->polyCount; ++PolyIdx)
	{
		dtPoly const* const Poly = &Tile.polys[PolyIdx];

		if (Poly->getType() == DT_POLYTYPE_GROUND)
		{
			dtPolyDetail const* const DetailPoly = &Tile.detailMeshes[PolyIdx];

			TArray<int32>* Indices = nullptr;
			if (bIsBeingBuilt)
			{
				Indices = &OutGeometry.BuiltMeshIndices;
			}
			else if ((Poly->flags & ForbiddenFlags) != 0)
			{
				Indices = &OutGeometry.ForbiddenIndices;
			}
#if RECAST_INTERNAL_DEBUG_DATA			
			else if (OutGeometry.bGatherTileBuildTimesHeatMap && DebugData)
			{
				const double Range = OutGeometry.MaxTileBuildTime - OutGeometry.MinTileBuildTime;
				int32 Rank = 0;
				if (Range != 0.)
				{
					Rank = static_cast<int32>(FRecastDebugGeometry::BuildTimeBucketsCount * ((DebugData->BuildTime - OutGeometry.MinTileBuildTime) / Range));
					Rank = FMath::Clamp(Rank, 0, FRecastDebugGeometry::BuildTimeBucketsCount-1);
				}
				Indices = &OutGeometry.TileBuildTimesIndices[Rank];
			}
#endif // RECAST_INTERNAL_DEBUG_DATA
			else
			{
				Indices = &OutGeometry.AreaIndices[Poly->getArea()];
			}

			// one triangle at a time
			for (int32 TriIdx = 0; TriIdx < DetailPoly->triCount; ++TriIdx)
			{
				int32 DetailTriIdx = (DetailPoly->triBase + TriIdx) * 4;
				const unsigned char* DetailTri = &Tile.detailTris[DetailTriIdx];

				// calc indices into the vert buffer we just populated
				int32 TriVertIndices[3];
				for (int32 TriVertIdx = 0; TriVertIdx < 3; ++TriVertIdx)
				{
					if (DetailTri[TriVertIdx] < Poly->vertCount)
					{
						TriVertIndices[TriVertIdx] = VertBase + Poly->verts[DetailTri[TriVertIdx]];
					}
					else
					{
						TriVertIndices[TriVertIdx] = VertBase + DetailVertIndexBase + (DetailPoly->vertBase + DetailTri[TriVertIdx] - Poly->vertCount);
					}
				}

				Indices->Add(TriVertIndices[0]);
				Indices->Add(TriVertIndices[1]);
				Indices->Add(TriVertIndices[2]);

#if WITH_NAVMESH_CLUSTER_LINKS
				if (Tile.polyClusters)
				{
					const uint16 ClusterId = Tile.polyClusters[PolyIdx];
					if (ClusterId < MAX_uint8)
					{
						if (ClusterId >= OutGeometry.Clusters.Num())
						{
							OutGeometry.Clusters.AddDefaulted(ClusterId - OutGeometry.Clusters.Num() + 1);
						}

						TArray<int32>& ClusterIndices = OutGeometry.Clusters[ClusterId].MeshIndices;
						ClusterIndices.Add(TriVertIndices[0]);
						ClusterIndices.Add(TriVertIndices[1]);
						ClusterIndices.Add(TriVertIndices[2]);
					}
				}
#endif // WITH_NAVMESH_CLUSTER_LINKS
			}
		}
	}

	for (int32 i = 0; i < Header->offMeshConCount; ++i)
	{
		const dtOffMeshConnection* OffMeshConnection = &Tile.offMeshCons[i];

		if (OffMeshConnection != NULL)
		{
			dtPoly const* const LinkPoly = &Tile.polys[OffMeshConnection->poly];
			FVector::FReal va[3], vb[3];
			Tile.verts[LinkPoly->verts[0]].toWorld(Tile.header->bmin, va); //OffMeshConnection->pos;
			Tile.verts[LinkPoly->verts[1]].toWorld(Tile.header->bmin, vb); //OffMeshConnection->pos[3];

			const FRecastDebugGeometry::FOffMeshLink Link = {
				Recast2UnrVector(va)
				, Recast2UnrVector(vb)
				, LinkPoly->getArea()
				, (uint8)OffMeshConnection->getBiDirectional()
				, GetValidEnds(*DetourNavMesh, Tile, *LinkPoly, OffMeshConnection->getIsReversed())
				, OffMeshConnection->getIsGenerated()
				, UE_REAL_TO_FLOAT_CLAMPED_MAX(OffMeshConnection->rad)
			};

			(LinkPoly->flags & ForbiddenFlags) != 0
				? OutGeometry.ForbiddenLinks.Add(Link)
				: OutGeometry.OffMeshLinks.Add(Link);
		}
	}

#if WITH_NAVMESH_SEGMENT_LINKS

	// Note: Segment Links don't currently support the notion of reversed links yet.

	for (int32 i = 0; i < Header->offMeshSegConCount; ++i)
	{
		const dtOffMeshSegmentConnection* OffMeshSeg = &Tile.offMeshSeg[i];
		if (OffMeshSeg != NULL)
		{
			const int32 polyBase = Header->offMeshSegPolyBase + OffMeshSeg->firstPoly;
			for (int32 j = 0; j < OffMeshSeg->npolys; j++)
			{
				dtPoly const* const LinkPoly = &Tile.polys[polyBase + j];

				FRecastDebugGeometry::FOffMeshSegment Link;
				FVector::FReal vLS[3], vLE[3], vRS[3], vRE[3];
				Tile.verts[LinkPoly->verts[0]].toWorld(Tile.header->bmin, vLS);
				Tile.verts[LinkPoly->verts[1]].toWorld(Tile.header->bmin, vLE);
				Tile.verts[LinkPoly->verts[2]].toWorld(Tile.header->bmin, vRS);
				Tile.verts[LinkPoly->verts[3]].toWorld(Tile.header->bmin, vRE);
				Link.LeftStart = Recast2UnrealPoint(vLS);
				Link.LeftEnd = Recast2UnrealPoint(vLE);
				Link.RightStart = Recast2UnrealPoint(vRS);
				Link.RightEnd = Recast2UnrealPoint(vRE);
				Link.AreaID = LinkPoly->getArea();
				Link.Direction = (uint8)OffMeshSeg->getBiDirectional();
				Link.ValidEnds = GetValidEnds(*DetourNavMesh, Tile, *LinkPoly);

				const int LinkIdx = OutGeometry.OffMeshSegments.Add(Link);
				ensureMsgf((LinkPoly->flags & ForbiddenFlags) == 0, TEXT("Not implemented"));
				OutGeometry.OffMeshSegmentAreas[Link.AreaID].Add(LinkIdx);
			}
		}
	}
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if WITH_NAVMESH_CLUSTER_LINKS
	for (int32 i = 0; i < Header->clusterCount; i++)
	{
		const dtCluster& c0 = Tile.clusters[i];
		uint32 iLink = c0.firstLink;
		while (iLink != DT_NULL_LINK)
		{
			const dtClusterLink& link = DetourNavMesh->getClusterLink(&Tile, iLink);
			iLink = link.next;

			dtMeshTile const* const OtherTile = DetourNavMesh->getTileByRef(link.ref);
			if (OtherTile)
			{
				int32 linkedIdx = DetourNavMesh->decodeClusterIdCluster(link.ref);
				const dtCluster& c1 = OtherTile->clusters[linkedIdx];

				FRecastDebugGeometry::FClusterLink LinkGeom;
				LinkGeom.FromCluster = Recast2UnrealPoint(c0.center);
				LinkGeom.ToCluster = Recast2UnrealPoint(c1.center);

				if (linkedIdx > i || TileIdx > (int32)DetourNavMesh->decodeClusterIdTile(link.ref))
				{
					FVector UpDir(0, 0, 1.0f);
					FVector LinkDir = (LinkGeom.ToCluster - LinkGeom.FromCluster).GetSafeNormal();
					FVector SideDir = FVector::CrossProduct(LinkDir, UpDir);
					LinkGeom.FromCluster += SideDir * 40.0f;
					LinkGeom.ToCluster += SideDir * 40.0f;
				}

				OutGeometry.ClusterLinks.Add(LinkGeom);
			}
		}
	}
#endif // WITH_NAVMESH_CLUSTER_LINKS

	// Get tile edges and navmesh edges
	if (OutGeometry.bGatherPolyEdges || OutGeometry.bGatherNavMeshEdges)
	{
		GetTilePolyEdges(Tile, !!OutGeometry.bGatherPolyEdges, !!OutGeometry.bGatherNavMeshEdges
			, OutGeometry.PolyEdges, OutGeometry.NavMeshEdges);
	}

	return Header->vertCount + Header->detailVertCount;
}

FBox FPImplRecastNavMesh::GetNavMeshBounds() const
{
	FBox Bbox(ForceInit);

	// @todo, calc once and cache it
	if (DetourNavMesh)
	{
		// workaround for privacy issue in the recast API
		dtNavMesh const* const ConstRecastNavMesh = DetourNavMesh;

		// spin through all the tiles and accumulate the bounds
		for (int32 TileIdx=0; TileIdx < DetourNavMesh->getMaxTiles(); ++TileIdx)
		{
			dtMeshTile const* const Tile = ConstRecastNavMesh->getTile(TileIdx);
			if (Tile)
			{
				dtMeshHeader const* const Header = Tile->header;
				if (Header)
				{
					const FBox NodeBox = Recast2UnrealBox(Header->bmin, Header->bmax);
					Bbox += NodeBox;
				}
			}
		}
	}

	return Bbox;
}

bool FPImplRecastNavMesh::GetNavMeshTileXY(const FVector& Point, int32& OutX, int32& OutY) const
{
	if (DetourNavMesh)
	{
		// workaround for privacy issue in the recast API
		dtNavMesh const* const ConstRecastNavMesh = DetourNavMesh;

		const FVector RecastPt = Unreal2RecastPoint(Point);
		int32 TileX = 0;
		int32 TileY = 0;

		ConstRecastNavMesh->calcTileLoc(&RecastPt.X, &TileX, &TileY);
		OutX = TileX;
		OutY = TileY;
		return true;
	}

	return false;
}

void FPImplRecastNavMesh::GetNavMeshTilesIn(const TArray<FBox>& InclusionBounds, TArray<FNavTileRef>& OutRefs) const
{
	if (DetourNavMesh)
	{
		const FVector::FReal* NavMeshOrigin = DetourNavMesh->getParams()->orig;
		const FVector::FReal TileSize = DetourNavMesh->getParams()->tileWidth;

		// Generate a set of all possible tile coordinates that belong to requested bounds
		TSet<FIntPoint>	TileCoords;	
		for (const FBox& Bounds : InclusionBounds)
		{
			if (ensureMsgf(Bounds.IsValid, TEXT("%hs Bounds is not valid"), __FUNCTION__))
			{
				const FVector RcNavMeshOrigin(NavMeshOrigin[0], NavMeshOrigin[1], NavMeshOrigin[2]);
				const FRcTileBox TileBox(Bounds, RcNavMeshOrigin, TileSize);

				for (int32 y = TileBox.YMin; y <= TileBox.YMax; ++y)
				{
					for (int32 x = TileBox.XMin; x <= TileBox.XMax; ++x)
					{
						TileCoords.Add(FIntPoint(x, y));
					}
				}
			}
		}

		// We guess that each tile has 3 layers in average
		OutRefs.Reserve(TileCoords.Num()*3);

		TArray<const dtMeshTile*> MeshTiles;
		MeshTiles.Reserve(3);

		for (const FIntPoint& TileCoord : TileCoords)
		{
			int32 MaxTiles = DetourNavMesh->getTileCountAt(TileCoord.X, TileCoord.Y);
			if (MaxTiles > 0)
			{
				MeshTiles.SetNumZeroed(MaxTiles, EAllowShrinking::No);
				
				const int32 MeshTilesCount = DetourNavMesh->getTilesAt(TileCoord.X, TileCoord.Y, MeshTiles.GetData(), MaxTiles);
				for (int32 i = 0; i < MeshTilesCount; ++i)
				{
					const dtMeshTile* MeshTile = MeshTiles[i];
					dtTileRef TileRef = DetourNavMesh->getTileRef(MeshTile);
					if (TileRef)
					{
						// Consider only mesh tiles that actually belong to a requested bounds
						FBox TileBounds = Recast2UnrealBox(MeshTile->header->bmin, MeshTile->header->bmax);
						for (const FBox& RequestedBounds : InclusionBounds)
						{
							if (TileBounds.Intersect(RequestedBounds))
							{
								OutRefs.Add(FNavTileRef(TileRef));
								break;
							}
						}
					}
				}
			}
		}
	}
}

SIZE_T FPImplRecastNavMesh::GetTotalDataSize() const
{
	SIZE_T TotalBytes = sizeof(*this);

	if (DetourNavMesh)
	{
		// iterate all tiles and sum up their DataSize
		dtNavMesh const* ConstNavMesh = DetourNavMesh;
		for (int i = 0; i < ConstNavMesh->getMaxTiles(); ++i)
		{
			const dtMeshTile* Tile = ConstNavMesh->getTile(i);
			if (Tile != NULL && Tile->header != NULL)
			{
				TotalBytes += Tile->dataSize;
			}
		}
	}

	return TotalBytes / 1024;
}

#if !UE_BUILD_SHIPPING
int32 FPImplRecastNavMesh::GetCompressedTileCacheSize()
{
	int32 CompressedTileCacheSize = 0;

	for (TPair<FIntPoint, TArray<FNavMeshTileData>>& TilePairIter : CompressedTileCacheLayers)
	{
		TArray<FNavMeshTileData>& NavMeshTileDataArray = TilePairIter.Value;

		for (FNavMeshTileData& NavMeshTileDataIter : NavMeshTileDataArray)
		{
			CompressedTileCacheSize += NavMeshTileDataIter.DataSize;
		}
	}

	return CompressedTileCacheSize;
}
#endif

void FPImplRecastNavMesh::ApplyWorldOffset(const FVector& InOffset, bool bWorldShift)
{
	if (DetourNavMesh != NULL)
	{
		// transform offset to Recast space
		const FVector OffsetRC = Unreal2RecastPoint(InOffset);
		// apply offset
		DetourNavMesh->applyWorldOffset(&OffsetRC.X);
	}
}

uint16 FPImplRecastNavMesh::GetFilterForbiddenFlags(const FRecastQueryFilter* Filter)
{
	return ((const dtQueryFilter*)Filter)->getExcludeFlags();
}

void FPImplRecastNavMesh::SetFilterForbiddenFlags(FRecastQueryFilter* Filter, uint16 ForbiddenFlags)
{
	((dtQueryFilter*)Filter)->setExcludeFlags(ForbiddenFlags);
	// include-exclude don't need to be symmetrical, filter will check both conditions
}

void FPImplRecastNavMesh::OnAreaCostChanged()
{
	struct FRealntPair
	{
		FVector::FReal Score;
		int32 Index;

		FRealntPair() : Score(MAX_FLT), Index(0) {}
		FRealntPair(int32 AreaId, FVector::FReal TravelCost, FVector::FReal EntryCost) : Score(TravelCost + EntryCost), Index(AreaId) {}

		bool operator <(const FRealntPair& Other) const { return Score < Other.Score; }
	};

	if (NavMeshOwner && DetourNavMesh)
	{
		const INavigationQueryFilterInterface* NavFilter = NavMeshOwner->GetDefaultQueryFilterImpl();
		const dtQueryFilter* DetourFilter = ((const FRecastQueryFilter*)NavFilter)->GetAsDetourQueryFilter();

		TArray<FRealntPair> AreaData;
		AreaData.Reserve(RECAST_MAX_AREAS);
		for (int32 Idx = 0; Idx < RECAST_MAX_AREAS; Idx++)
		{
			AreaData.Add(FRealntPair(Idx, DetourFilter->getAreaCost(Idx), DetourFilter->getAreaFixedCost(Idx)));
		}

		AreaData.Sort();

		uint8 AreaCostOrder[RECAST_MAX_AREAS];
		for (int32 Idx = 0; Idx < RECAST_MAX_AREAS; Idx++)
		{
			AreaCostOrder[AreaData[Idx].Index] = static_cast<uint8>(Idx);
		}

		DetourNavMesh->applyAreaCostOrder(AreaCostOrder);

		// If off-mesh linking was deferred during batch tile loading,
		// connect them now that the cost order is correct.
		DetourNavMesh->connectDeferredOffMeshLinks();
	}
}

void FPImplRecastNavMesh::RemoveTileCacheLayers(int32 TileX, int32 TileY)
{
	CompressedTileCacheLayers.Remove(FIntPoint(TileX, TileY));
}

void FPImplRecastNavMesh::RemoveTileCacheLayer(int32 TileX, int32 TileY, int32 LayerIdx)
{
	TArray<FNavMeshTileData>* ExistingLayersList = CompressedTileCacheLayers.Find(FIntPoint(TileX, TileY));
	if (ExistingLayersList)
	{
		if (ExistingLayersList->IsValidIndex(LayerIdx))
		{
			ExistingLayersList->RemoveAt(LayerIdx);

			for (int32 Idx = LayerIdx; Idx < ExistingLayersList->Num(); Idx++)
			{
				(*ExistingLayersList)[Idx].LayerIndex = Idx;
			}
		}
		
		if (ExistingLayersList->Num() == 0)
		{
			RemoveTileCacheLayers(TileX, TileY);

#if RECAST_INTERNAL_DEBUG_DATA
			NavMeshOwner->RemoveTileDebugData(TileX, TileY);
#endif
		}
	}
}

void FPImplRecastNavMesh::AddTileCacheLayers(int32 TileX, int32 TileY, const TArray<FNavMeshTileData>& Layers)
{
	CompressedTileCacheLayers.Add(FIntPoint(TileX, TileY), Layers);
}

void FPImplRecastNavMesh::AddTileCacheLayer(int32 TileX, int32 TileY, int32 LayerIdx, const FNavMeshTileData& LayerData)
{
	TArray<FNavMeshTileData>* ExistingLayersList = CompressedTileCacheLayers.Find(FIntPoint(TileX, TileY));
	
	if (ExistingLayersList)
	{
		ExistingLayersList->SetNum(FMath::Max(ExistingLayersList->Num(), LayerIdx + 1));
		(*ExistingLayersList)[LayerIdx] = LayerData;
	}
	else
	{
		TArray<FNavMeshTileData> LayersList;
		LayersList.SetNum(FMath::Max(LayersList.Num(), LayerIdx + 1));
		LayersList[LayerIdx] = LayerData;
		CompressedTileCacheLayers.Add(FIntPoint(TileX, TileY), LayersList);
	}
}

void FPImplRecastNavMesh::MarkEmptyTileCacheLayers(int32 TileX, int32 TileY)
{
	if (!CompressedTileCacheLayers.Contains(FIntPoint(TileX, TileY)))
	{
		TArray<FNavMeshTileData> EmptyLayersList;
		CompressedTileCacheLayers.Add(FIntPoint(TileX, TileY), EmptyLayersList);
	}
}

FNavMeshTileData FPImplRecastNavMesh::GetTileCacheLayer(int32 TileX, int32 TileY, int32 LayerIdx) const
{
	const TArray<FNavMeshTileData>* LayersList = CompressedTileCacheLayers.Find(FIntPoint(TileX, TileY));
	if (LayersList && LayersList->IsValidIndex(LayerIdx))
	{
		return (*LayersList)[LayerIdx];
	}

	return FNavMeshTileData();
}

TArray<FNavMeshTileData> FPImplRecastNavMesh::GetTileCacheLayers(int32 TileX, int32 TileY) const
{
	return CompressedTileCacheLayers.FindRef(FIntPoint(TileX, TileY));
}

bool FPImplRecastNavMesh::HasTileCacheLayers(int32 TileX, int32 TileY) const
{
	return CompressedTileCacheLayers.Contains(FIntPoint(TileX, TileY));
}

#undef INITIALIZE_NAVQUERY

#endif // WITH_RECAST
