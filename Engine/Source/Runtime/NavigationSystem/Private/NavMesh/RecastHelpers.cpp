// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavMesh/RecastHelpers.h"

FVector Unreal2RecastPoint(const FVector& UnrealPoint)
{
	return FVector(-UnrealPoint[0], UnrealPoint[2], -UnrealPoint[1]);
}

FVector Unreal2RecastPoint(const FVector::FReal* UnrealPoint)
{
	return FVector(-UnrealPoint[0], UnrealPoint[2], -UnrealPoint[1]);
}

FVector Unreal2RecastPoint(const float* UnrealPoint)
{
	return FVector(-UnrealPoint[0], UnrealPoint[2], -UnrealPoint[1]);
}

FBox Unreal2RecastBox(const FBox& UnrealBox)
{
	FVector Points[2] = { Unreal2RecastPoint(UnrealBox.Min), Unreal2RecastPoint(UnrealBox.Max) };
	return FBox(Points, 2);
}

FMatrix Unreal2RecastMatrix()
{
	static FMatrix TM(FVector(-1.f,0,0),FVector(0,0,-1.f),FVector(0,1.f,0),FVector::ZeroVector);
	return TM;
}

FVector Recast2UnrealPoint(const FVector::FReal* RecastPoint)
{
	return FVector(-RecastPoint[0], -RecastPoint[2], RecastPoint[1]);
}

FVector Recast2UnrealPoint(const float* RecastPoint)
{
	return FVector(-RecastPoint[0], -RecastPoint[2], RecastPoint[1]);
}

FVector Recast2UnrealPoint(const FVector& RecastPoint)
{
	return FVector(-RecastPoint.X, -RecastPoint.Z, RecastPoint.Y);
}

FBox Recast2UnrealBox(const FVector::FReal* RecastMin, const FVector::FReal* RecastMax)
{
	FVector Points[2] = { Recast2UnrealPoint(RecastMin), Recast2UnrealPoint(RecastMax) };
	return FBox(Points, 2);
}

FBox Recast2UnrealBox(const float* RecastMin, const float* RecastMax)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVector Points[2] = { Recast2UnrealPoint(RecastMin), Recast2UnrealPoint(RecastMax) };
	return FBox(Points, 2);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FBox Recast2UnrealBox(const FBox& RecastBox)
{
	FVector Points[2] = { Recast2UnrealPoint(RecastBox.Min), Recast2UnrealPoint(RecastBox.Max) };
	return FBox(Points, 2);
}

FMatrix Recast2UnrealMatrix()
{
	static FMatrix TM(FVector(-1.f, 0, 0), FVector(0, 0, -1.f), FVector(0, 1.f, 0), FVector::ZeroVector);
	return TM;
}

FColor Recast2UnrealColor(const unsigned int RecastColor)
{
	const uint8 R = (RecastColor & 0xFF);
	const uint8 G = (RecastColor & 0xFF00) >> 8;
	const uint8 B = static_cast<uint8>((RecastColor & 0xFF0000) >> 16);
	const uint8 A = static_cast<uint8>((RecastColor & 0xFF000000) >> 24);
	return FColor(R,G,B,A);	// can't be direct assignation since internally FColor is not RGBA
}