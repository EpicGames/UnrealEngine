// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavMesh/RecastNavMesh.h"
#include "Misc/Paths.h"
#include "EngineGlobals.h"
#include "Engine/World.h"
#include "NavigationSystem.h"
#include "Engine/Engine.h"
#include "Distance/DistSegment2AxisAlignedBox2.h"
#include "DrawDebugHelpers.h"
#include "Misc/ConfigCacheIni.h"
#include "EngineUtils.h"
#include "NavMesh/RecastHelpers.h"
#include "NavMesh/RecastVersion.h"
#include "NavAreas/NavArea.h"
#include "NavAreas/NavArea_Null.h"
#include "NavAreas/NavArea_Default.h"
#include "NavAreas/NavArea_LowHeight.h"
#include "NavLinkCustomInterface.h"
#include "NavMesh/RecastNavMeshDataChunk.h"
#include "NavMesh/RecastQueryFilter.h"
#include "VisualLogger/VisualLogger.h"
#include "WorldPartition/NavigationData/NavigationDataChunkActor.h"
#include "Math/Color.h"
#include "NavigationDataHandler.h"
#include "BaseGeneratedNavLinksProxy.h"
#include "Misc/ScopeLock.h"

#if WITH_EDITOR
#include "EditorSupportDelegates.h"
#include "ObjectEditorUtils.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionEditorLoaderAdapter.h"
#include "WorldPartition/LoaderAdapter/LoaderAdapterShape.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Misc/TransactionObjectEvent.h"
#endif

#if WITH_RECAST
#include "Detour/DetourAlloc.h"
#endif // WITH_RECAST

#include "NavMesh/NavMeshRenderingComponent.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RecastNavMesh)

#if WITH_RECAST
/// Helper for accessing navigation query from different threads
#define INITIALIZE_NAVQUERY(NavQueryVariable, NumNodes)	\
	dtNavMeshQuery NavQueryVariable##Private;	\
	dtNavMeshQuery& NavQueryVariable = IsInGameThread() ? RecastNavMeshImpl->SharedNavQuery : NavQueryVariable##Private; \
	NavQueryVariable.init(RecastNavMeshImpl->DetourNavMesh, NumNodes);

#define INITIALIZE_NAVQUERY_WLINKFILTER(NavQueryVariable, NumNodes, LinkFilter)	\
	dtNavMeshQuery NavQueryVariable##Private;	\
	dtNavMeshQuery& NavQueryVariable = IsInGameThread() ? RecastNavMeshImpl->SharedNavQuery : NavQueryVariable##Private; \
	NavQueryVariable.init(RecastNavMeshImpl->DetourNavMesh, NumNodes, &LinkFilter);

#endif // WITH_RECAST

namespace UE::NavMesh::Private
{
	static bool bAllowLinkGeneration = true;
	static FAutoConsoleVariableRef CVarAllowLinkGeneration(TEXT("ai.nav.AllowLinkGeneration"), bAllowLinkGeneration, TEXT("Set to false to force disabling link generation."), ECVF_Default);

	// Max tile size in voxels. Larger than this tiles will start to get slow to build.
	constexpr int32 ArbitraryMaxTileSizeVoxels = 1024;
	// Min tile size on voxels. Smaller tiles than this waste computation during voxelization because the border are will be larger than usable area.
	constexpr int32 ArbitraryMinTileSizeVoxels = 16;
	// Minimum tile size in multiples of agent radius.
	constexpr int32 ArbitraryMinTileSizeAgentRadius = 4; 

	/** this helper function supplies a consistent way to keep TileSizeUU within defined bounds */
	float GetClampedTileSizeUU(const float InTileSizeUU, const float CellSize, const float AgentRadius)
	{
		const float MinTileSize = FMath::Max3(RECAST_MIN_TILE_SIZE, CellSize * ArbitraryMinTileSizeVoxels, AgentRadius * ArbitraryMinTileSizeAgentRadius);
		const float MaxTileSize = FMath::Max(RECAST_MIN_TILE_SIZE, CellSize * ArbitraryMaxTileSizeVoxels);
		
		return FMath::Clamp<float>(InTileSizeUU, MinTileSize, MaxTileSize);
	}

	// These should reflect the property clamping of FNavMeshResolutionParam::CellSize.  
	// Minimum cell size.
	constexpr float ArbitraryMinCellSize = 1.0f; 
	// Maximum cell size.
	constexpr float ArbitraryMaxCellSize = 1024.0f; 

	float GetClampedCellSize(const float CellSize)
	{
		return FMath::Clamp(CellSize, ArbitraryMinCellSize, ArbitraryMaxCellSize);
	}

#if WITH_RECAST
	FNavTileRef GetTileRefFromPolyRef(const dtNavMesh& DetourMesh, const NavNodeRef PolyRef)
	{
		unsigned int Salt = 0;
		unsigned int TileIndex = 0;
		unsigned int PolyIndex = 0;
		DetourMesh.decodePolyId(PolyRef, Salt, TileIndex, PolyIndex);
		const dtTileRef TileRef = DetourMesh.encodePolyId(Salt, TileIndex, 0);
		return FNavTileRef(TileRef);
	}
#endif // WITH_RECAST
} // namespace UE::NavMesh::Private

FDetourTileLayout::FDetourTileLayout(const dtMeshTile& tile)
{
#if WITH_RECAST
	const dtMeshHeader* header = tile.header;

	if (header && (header->version == DT_NAVMESH_VERSION))
	{
		FDetourTileSizeInfo SizeInfo;

		SizeInfo.VertCount = header->vertCount;
		SizeInfo.PolyCount = header->polyCount;
		SizeInfo.MaxLinkCount = header->maxLinkCount;
		SizeInfo.DetailMeshCount = header->detailMeshCount;
		SizeInfo.DetailVertCount = header->detailVertCount;
		SizeInfo.DetailTriCount = header->detailTriCount;
		SizeInfo.BvNodeCount = header->bvNodeCount;
		SizeInfo.OffMeshConCount = header->offMeshConCount;

#if WITH_NAVMESH_SEGMENT_LINKS
		SizeInfo.OffMeshSegConCount = header->offMeshSegConCount;
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if WITH_NAVMESH_CLUSTER_LINKS
		SizeInfo.ClusterCount = header->clusterCount;
		SizeInfo.OffMeshBase = header->offMeshBase;
#endif // WITH_NAVMESH_CLUSTER_LINKS

		InitFromSizeInfo(SizeInfo);
	}
#endif // WITH_RECAST
}

FDetourTileLayout::FDetourTileLayout(const FDetourTileSizeInfo& SizeInfo)
{
	InitFromSizeInfo(SizeInfo);
}

void FDetourTileLayout::InitFromSizeInfo(const FDetourTileSizeInfo& SizeInfo)
{
#if WITH_RECAST
	// Patch header pointers.
	HeaderSize = dtAlign(sizeof(dtMeshHeader));
	VertsSize = dtAlign(sizeof(dtTileVert) * SizeInfo.VertCount);
	PolysSize = dtAlign(sizeof(dtPoly) * SizeInfo.PolyCount);
	LinksSize = dtAlign(sizeof(dtLink) * (SizeInfo.MaxLinkCount));
	DetailMeshesSize = dtAlign(sizeof(dtPolyDetail) * SizeInfo.DetailMeshCount);
	DetailVertsSize = dtAlign(sizeof(dtTileVert) * SizeInfo.DetailVertCount);
	DetailTrisSize = dtAlign(sizeof(unsigned char) * 4 * SizeInfo.DetailTriCount);
	BvTreeSize = dtAlign(sizeof(dtBVNode) * SizeInfo.BvNodeCount);
	OffMeshConsSize = dtAlign(sizeof(dtOffMeshConnection) * SizeInfo.OffMeshConCount);

#if WITH_NAVMESH_SEGMENT_LINKS
	OffMeshSegsSize = dtAlign(sizeof(dtOffMeshSegmentConnection) * SizeInfo.OffMeshSegConCount);
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if WITH_NAVMESH_CLUSTER_LINKS
	ClustersSize = dtAlign(sizeof(dtCluster) * SizeInfo.ClusterCount);
	PolyClustersSize = dtAlign(sizeof(unsigned short) * SizeInfo.OffMeshBase);
#endif // WITH_NAVMESH_CLUSTER_LINKS

	TileSize = HeaderSize + VertsSize + PolysSize + LinksSize + DetailMeshesSize + DetailVertsSize + DetailTrisSize
		+ BvTreeSize + OffMeshConsSize + OffMeshSegsSize + ClustersSize + PolyClustersSize;
#endif // WITH_RECAST
}


FNavMeshTileData::FNavData::~FNavData()
{
#if WITH_RECAST
	dtFree(RawNavData, DT_ALLOC_PERM_TILE_DATA);
#else
	FMemory::Free(RawNavData);
#endif // WITH_RECAST
}

bool FRecastNavMeshTileGenerationDebug::IsWithinTileCoordinates(const int32 TileX, const int32 TileY) const
{
	if (bUseMaxTileCoordinate)
	{
		return FMath::IsWithinInclusive(TileX, TileCoordinate.X, MaxTileCoordinate.X)
			&& FMath::IsWithinInclusive(TileY, TileCoordinate.Y, MaxTileCoordinate.Y);
	}

	return TileX == TileCoordinate.X && TileY == TileCoordinate.Y;
}

// Temporary test to narrow a crash.
void FNavMeshTileData::FNavData::TestPtr() const
{
	if (RawNavData != nullptr)
	{
		static uint8 Temp = 0;
		Temp = *RawNavData;

		const SIZE_T Size = FMemory::GetAllocSize((void*)RawNavData);
		check(Size == 0 || Size == AllocatedSize);
	}
}

FNavMeshTileData::FNavMeshTileData(uint8* RawData, int32 RawDataSize, int32 LayerIdx, FBox LayerBounds)
	: LayerIndex(LayerIdx)
	, LayerBBox(LayerBounds)
	, DataSize(RawDataSize)
{
	INC_MEMORY_STAT_BY(STAT_Navigation_TileCacheMemory, DataSize);
	NavData = MakeShareable(new FNavData(RawData, DataSize));
}

FNavMeshTileData::~FNavMeshTileData()
{
	if (NavData.IsUnique() && NavData->GetRawNavData())
	{
		// @todo this isn't accounting for the fact that NavData is a shared pointer
		DEC_MEMORY_STAT_BY(STAT_Navigation_TileCacheMemory, DataSize);
	}
}

uint8* FNavMeshTileData::Release()
{
	uint8* RawData = nullptr;

	if (NavData.IsValid() && NavData->GetRawNavData()) 
	{ 
		RawData = NavData->GetMutableRawNavData();
		NavData->Reset();
		DEC_MEMORY_STAT_BY(STAT_Navigation_TileCacheMemory, DataSize);
	} 
 
	DataSize = 0; 
	LayerIndex = 0; 
	return RawData;
}

void FNavMeshTileData::MakeUnique()
{
	if (DataSize > 0 && !NavData.IsUnique())
	{
		INC_MEMORY_STAT_BY(STAT_Navigation_TileCacheMemory, DataSize);
#if WITH_RECAST
		uint8* UniqueRawData = (uint8*)dtAlloc(sizeof(uint8)*DataSize, DT_ALLOC_PERM_TILE_DATA);
#else
		uint8* UniqueRawData = (uint8*)FMemory::Malloc(sizeof(uint8)*DataSize);
#endif //WITH_RECAST

		NavData->TestPtr();
		
		FMemory::Memcpy(UniqueRawData, NavData->GetRawNavData(), DataSize);
		NavData = MakeShareable(new FNavData(UniqueRawData, DataSize));
	}
}

static FAutoConsoleCommandWithWorldAndArgs CmdNavMeshDrawDistance(
	TEXT("ai.debug.nav.DrawDistance"),
	TEXT("Sets the culling distance used by the navmesh rendering for lines and labels."),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& Args, UWorld*)
		{
			const FVector::FReal DrawDistance = (Args.Num() == 0) ? GetDefault<ARecastNavMesh>()->DefaultDrawDistance : FCString::Atof(*Args[0]);
			ARecastNavMesh::SetDrawDistance(DrawDistance);
		}
	));

#if WITH_RECAST && !UE_BUILD_SHIPPING
static FAutoConsoleCommandWithWorldArgsAndOutputDevice CmdDumpNavMeshMemory(
	TEXT("ai.debug.nav.DumpNavMeshMemory"),
	TEXT("Dumps detailed tile memory usage statistics for all active navmeshes."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda(
		[](const TArray<FString>& Args, const UWorld* World, FOutputDevice& OutputDevice)
		{
			if (const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World))
			{
				for (const ANavigationData* NavData : NavSys->NavDataSet)
				{
					if (const ARecastNavMesh* RecastNavMesh = Cast<ARecastNavMesh>(NavData))
					{
						RecastNavMesh->DumpDetailedMemoryStats();
					}
				}
			}
			else
			{
				OutputDevice.Log(ELogVerbosity::Warning, TEXT("Command failed: unable to find the navigation system."));
			}
		}
	));
#endif // WITH_RECAST && !UE_BUILD_SHIPPING

FVector::FReal ARecastNavMesh::DrawDistanceSq = 0.;
float ARecastNavMesh::MinimumSizeForChaosNavMeshInfluenceSq = 0.0f;
#if !WITH_RECAST

PRAGMA_DISABLE_DEPRECATION_WARNINGS
ARecastNavMesh::ARecastNavMesh(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void ARecastNavMesh::Serialize( FArchive& Ar )
{
	Super::Serialize(Ar);

	uint32 NavMeshVersion;
	Ar << NavMeshVersion;

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	//@todo: How to handle loading nav meshes saved w/ recast when recast isn't present????

	// when writing, write a zero here for now.  will come back and fill it in later.
	uint32 RecastNavMeshSizeBytes = 0;
	int64 RecastNavMeshSizePos = Ar.Tell();
	Ar << RecastNavMeshSizeBytes;

	if (Ar.IsLoading())
	{
		// incompatible, just skip over this data.  navmesh needs rebuilt.
		Ar.Seek( RecastNavMeshSizePos + RecastNavMeshSizeBytes );

		// Mark self for delete
		CleanUpAndMarkPendingKill();
	}
}

ARecastNavMesh::~ARecastNavMesh()
{
}

#if WITH_EDITOR
bool ARecastNavMesh::CanEditChange(const FProperty* InProperty) const
{
	return false;
}
#endif // WITH_EDITOR

#else // WITH_RECAST

#include "Detour/DetourNavMesh.h"
#include "Detour/DetourNavMeshQuery.h"
#include "NavMesh/PImplRecastNavMesh.h"
#include "NavMesh/RecastNavMeshGenerator.h"

//----------------------------------------------------------------------//
// FRecastDebugGeometry
//----------------------------------------------------------------------//
uint32 FRecastDebugGeometry::GetAllocatedSize() const
{
	SIZE_T Size = sizeof(*this) + MeshVerts.GetAllocatedSize()
		+ BuiltMeshIndices.GetAllocatedSize()
		+ PolyEdges.GetAllocatedSize()
		+ NavMeshEdges.GetAllocatedSize()
		+ OffMeshLinks.GetAllocatedSize()
#if WITH_NAVMESH_SEGMENT_LINKS
		+ OffMeshSegments.GetAllocatedSize()
#endif // WITH_NAVMESH_SEGMENT_LINKS
		;

	for (int i = 0; i < RECAST_MAX_AREAS; ++i)
	{
		Size += AreaIndices[i].GetAllocatedSize();
	}

#if RECAST_INTERNAL_DEBUG_DATA
	for (int i = 0; i < BuildTimeBucketsCount; ++i)
	{
		Size += TileBuildTimesIndices[i].GetAllocatedSize();
	}
#endif // RECAST_INTERNAL_DEBUG_DATA

#if WITH_NAVMESH_CLUSTER_LINKS
	Size += Clusters.GetAllocatedSize()	+ ClusterLinks.GetAllocatedSize();

	for (int i = 0; i < Clusters.Num(); ++i)
	{
		Size += Clusters[i].MeshIndices.GetAllocatedSize();
	}
#endif // WITH_NAVMESH_CLUSTER_LINKS

	return IntCastChecked<uint32>(Size);
}

//----------------------------------------------------------------------//
// FNavTileRef
//----------------------------------------------------------------------//

// Only use for deprecation
void FNavTileRef::DeprecatedGetTileIdsFromNavTileRefs(const FPImplRecastNavMesh* RecastNavMeshImpl, const TArray<FNavTileRef>& InTileRefs, TArray<uint32>& OutTileIds)
{
	if (RecastNavMeshImpl)
	{
		if (const dtNavMesh* DetourMesh = RecastNavMeshImpl->DetourNavMesh)
		{
			OutTileIds.Reserve(InTileRefs.Num());
			for (const FNavTileRef TileRef : InTileRefs)
			{
				OutTileIds.Add(DetourMesh->decodePolyIdTile((dtTileRef)TileRef));
			}
		}
	}
}

// Only use for deprecation
void FNavTileRef::DeprecatedMakeTileRefsFromTileIds(const FPImplRecastNavMesh* RecastNavMeshImpl, const TArray<uint32>& InTileIds, TArray<FNavTileRef>& OutTileRefs)
{
	if (RecastNavMeshImpl)
	{
		if (const dtNavMesh* DetourMesh = RecastNavMeshImpl->DetourNavMesh)
		{
			OutTileRefs.Reserve(InTileIds.Num());
			for (const uint32 TileId : InTileIds)
			{
				const dtTileRef TileRef = DetourMesh->encodePolyId(0, TileId, 0);
				OutTileRefs.Add(FNavTileRef(TileRef));
			}
		}
	}
}

//----------------------------------------------------------------------//
// ARecastNavMesh
//----------------------------------------------------------------------//

namespace ERecastNamedFilter
{
	FRecastQueryFilter FilterOutNavLinksImpl;
	FRecastQueryFilter FilterOutAreasImpl;
	FRecastQueryFilter FilterOutNavLinksAndAreasImpl;
}

const FRecastQueryFilter* ARecastNavMesh::NamedFilters[] = {
	&ERecastNamedFilter::FilterOutNavLinksImpl
	, &ERecastNamedFilter::FilterOutAreasImpl
	, &ERecastNamedFilter::FilterOutNavLinksAndAreasImpl
};

namespace FNavMeshConfig
{
	ARecastNavMesh::FNavPolyFlags NavLinkFlag = ARecastNavMesh::FNavPolyFlags(0);

	FRecastNamedFiltersCreator::FRecastNamedFiltersCreator(bool bVirtualFilters)
	{
		// setting up the last bit available in dtPoly::flags
		NavLinkFlag = ARecastNavMesh::FNavPolyFlags(1 << (sizeof(((dtPoly*)0)->flags) * 8 - 1));

		ERecastNamedFilter::FilterOutNavLinksImpl.SetIsVirtual(bVirtualFilters);
		ERecastNamedFilter::FilterOutAreasImpl.SetIsVirtual(bVirtualFilters);
		ERecastNamedFilter::FilterOutNavLinksAndAreasImpl.SetIsVirtual(bVirtualFilters);

		ERecastNamedFilter::FilterOutNavLinksImpl.setExcludeFlags(NavLinkFlag);
		ERecastNamedFilter::FilterOutNavLinksAndAreasImpl.setExcludeFlags(NavLinkFlag);

		for (int32 AreaID = 0; AreaID < RECAST_MAX_AREAS; ++AreaID)
		{
			ERecastNamedFilter::FilterOutAreasImpl.setAreaCost(AreaID, RECAST_UNWALKABLE_POLY_COST);
			ERecastNamedFilter::FilterOutNavLinksAndAreasImpl.setAreaCost(AreaID, RECAST_UNWALKABLE_POLY_COST);
		}

		ERecastNamedFilter::FilterOutAreasImpl.setAreaCost(RECAST_DEFAULT_AREA, 1.f);
		ERecastNamedFilter::FilterOutNavLinksAndAreasImpl.setAreaCost(RECAST_DEFAULT_AREA, 1.f);
	}
}

#endif // WITH_RECAST

FRecastNavMeshTileGenerationDebug::FRecastNavMeshTileGenerationDebug()
{
	bEnabled = false;
	TileCoordinate = FIntVector::ZeroValue;
	MaxTileCoordinate = FIntVector::ZeroValue;
	bGenerateDebugTileOnly = false;
	bCollisionGeometry = false;
	bDynamicAreas = false;
	HeightFieldRenderMode = EHeightFieldRenderMode::Solid;
	bHeightfieldFromRasterization = false;
	bHeightfieldPostInclusionBoundsFiltering = false;
	bHeightfieldPostHeightFiltering = false;
	bHeightfieldBounds = false;
	bCompactHeightfield = false;
	bCompactHeightfieldEroded = false;
	bHeightFieldLayers = false;
	bCompactHeightfieldRegions = false;
	bCompactHeightfieldDistances = false;
	bTileCacheLayerAreas = false;
	bTileCacheLayerRegions = false;
	bSkipContourSimplification = false;
	bTileCacheContours = false;
	bTileCachePolyMesh = false;
	bTileCacheDetailMesh = false;
	bUseMaxTileCoordinate = false;
	LinkGenerationDebugFlags = 0;
	LinkGenerationSelectedEdge = -1;
	LinkGenerationSelectedConfig = -1;
}

#if WITH_RECAST

ARecastNavMesh::FNavPolyFlags ARecastNavMesh::NavLinkFlag = ARecastNavMesh::FNavPolyFlags(0);

ARecastNavMesh::ARecastNavMesh(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bDrawFilledPolys(true)
	, bDrawNavMeshEdges(true)
	, bDrawNavLinks(true)
	, bDrawOctreeDetails(true)
	, bDrawMarkedForbiddenPolys(false)
	, bDrawOnScreenInformation(true)
	, bDistinctlyDrawTilesBeingBuilt(true)
	, DrawOffset(10.f)
	, TilePoolSize(1024)
	, AverageLayersPerTile(3)
	, bMinimizeLinkPoolSize(true)
	, MaxVerticalMergeError(INT_MAX) // By default, ignore vertical error
	, MaxSimplificationError(1.3f)	// from RecastDemo
	, SimplificationElevationRatio(0.f)	// By default, ignore contour simplification from elevation
	, DefaultMaxSearchNodes(RECAST_MAX_SEARCH_NODES)
	, DefaultMaxHierarchicalSearchNodes(RECAST_MAX_SEARCH_NODES)
	, bSortNavigationAreasByCost(true)
	, bIsWorldPartitioned(false)
	, bGenerateNavLinks(false)
	, bPerformVoxelFiltering(true)	
	, bMarkLowHeightAreas(false)
	, bUseExtraTopCellWhenMarkingAreas(true)
	, bFilterLowSpanSequences(false)
	, bFilterLowSpanFromTileCache(false)
	, bStoreEmptyTileLayers(false)
	, bUseVirtualFilters(true)
	, bUseVirtualGeometryFilteringAndDirtying(false)
	, bAllowNavLinkAsPathEnd(false)
#if WITH_EDITORONLY_DATA
	, bAllowWorldPartitionedNavMesh(false)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	, NavLinkJumpDownConfig()
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
	, TileSetUpdateInterval(1.0f)
	, NavMeshVersion(NAVMESHVER_LATEST)	
	, RecastNavMeshImpl(nullptr)
{
	HeuristicScale = 0.999f;
	LedgeSlopeFilterMode = ENavigationLedgeSlopeFilterMode::Recast;
	RegionPartitioning = ERecastPartitioning::Watershed;
	LayerPartitioning = ERecastPartitioning::Watershed;
	RegionChunkSplits = 2;
	LayerChunkSplits = 2;
	MaxSimultaneousTileGenerationJobsCount = 1024;
	bDoFullyAsyncNavDataGathering = false;
	TileNumberHardLimit = 1 << 20;
	ExpectedMaxLayersPerTile = 12;

#if RECAST_ASYNC_REBUILDING
	BatchQueryCounter = 0;
#endif // RECAST_ASYNC_REBUILDING

	NavLinkJumpConfigs.AddDefaulted();

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		INC_DWORD_STAT_BY( STAT_NavigationMemory, sizeof(*this) );

		FindPathImplementation = FindPath;
		FindHierarchicalPathImplementation = FindPath;

		TestPathImplementation = TestPath;
		TestHierarchicalPathImplementation = TestHierarchicalPath;

		RaycastImplementationWithAdditionalResults = NavMeshRaycast;

		RecastNavMeshImpl = new FPImplRecastNavMesh(this);
	
		// add predefined areas up front
		SupportedAreas.Add(FSupportedAreaData(UNavArea_Null::StaticClass(), RECAST_NULL_AREA));
		SupportedAreas.Add(FSupportedAreaData(UNavArea_LowHeight::StaticClass(), RECAST_LOW_AREA));
		SupportedAreas.Add(FSupportedAreaData(UNavArea_Default::StaticClass(), RECAST_DEFAULT_AREA));
	}
}

ARecastNavMesh::~ARecastNavMesh()
{
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		DEC_DWORD_STAT_BY( STAT_NavigationMemory, sizeof(*this) );
		DestroyRecastPImpl();
	}
}

void ARecastNavMesh::DestroyRecastPImpl()
{
	if (RecastNavMeshImpl != NULL)
	{
		delete RecastNavMeshImpl;
		RecastNavMeshImpl = NULL;
	}
}

UPrimitiveComponent* ARecastNavMesh::ConstructRenderingComponent() 
{
	return NewObject<UNavMeshRenderingComponent>(this, TEXT("NavRenderingComp"), RF_Transient);
}

void ARecastNavMesh::UpdateNavMeshDrawing()
{
#if !UE_BUILD_SHIPPING
	UNavMeshRenderingComponent* NavMeshRenderComp = Cast<UNavMeshRenderingComponent>(RenderingComp);
	if (NavMeshRenderComp != nullptr && NavMeshRenderComp->GetVisibleFlag() && (NavMeshRenderComp->IsForcingUpdate() || UNavMeshRenderingComponent::IsNavigationShowFlagSet(GetWorld())))
	{
		RenderingComp->MarkRenderStateDirty();
#if WITH_EDITOR
		FEditorSupportDelegates::RedrawAllViewports.Broadcast();
#endif
	}
#endif // UE_BUILD_SHIPPING
}

void ARecastNavMesh::LoadBeforeGeneratorRebuild()
{
#if WITH_EDITOR
	// If it's not a world partitioned navmesh but it's in a partitioned world, we need to make sure the navigable world is loaded before building the navmesh.
	if (!bIsWorldPartitioned)
	{
		UWorld* World = GetWorld();
		if (World && World->IsPartitionedWorld() && !World->IsGameWorld())
		{
			const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);
			if (NavSys)
			{
				UWorldPartition* WorldPartition = World->GetWorldPartition();
				check(WorldPartition);
				
				const FBox Bounds = NavSys->GetNavigableWorldBounds();
				UWorldPartitionEditorLoaderAdapter* EditorLoaderAdapter = WorldPartition->CreateEditorLoaderAdapter<FLoaderAdapterShape>(World, Bounds, TEXT("Navigable World"));
				EditorLoaderAdapter->GetLoaderAdapter()->SetUserCreated(false);
				EditorLoaderAdapter->GetLoaderAdapter()->Load();

				// Make sure level instances are loaded.
				World->BlockTillLevelStreamingCompleted();
			}
		}
	}
#endif //WITH_EDITOR
}

void ARecastNavMesh::CleanUp()
{
	Super::CleanUp();
	bHasNoTileData = false;
	if (NavDataGenerator.IsValid())
	{
		NavDataGenerator->CancelBuild();
		NavDataGenerator.Reset();
	}
	DestroyRecastPImpl();
}

void ARecastNavMesh::PostLoad()
{
	UE_LOGF(LogNavigation, Verbose, "%s %ls", __FUNCTION__, *GetFullNameSafe(this));
	
	Super::PostLoad();
	
	if (const UWorld* World = GetWorld())
	{
		const UNavigationSystemBase* NavSys = World->GetNavigationSystem();
		if (NavSys && NavSys->IsWorldInitDone())
		{
			CheckToDiscardSubLevelNavData(*NavSys);
		}
		else
		{
			UNavigationSystemBase::OnNavigationInitStartStaticDelegate().AddUObject(this, &ARecastNavMesh::CheckToDiscardSubLevelNavData);
		}
	}

#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// If needed, initialize from deprecated value.
	if (NavMeshVersion < NAVMESHVER_TILE_RESOLUTIONS)
	{
		for (int i = 0; i < (uint8)ENavigationDataResolution::MAX; ++i)
		{
			SetCellSize((ENavigationDataResolution)i, CellSize);
		}
	}

	// If needed, initialize CellHeight from the deprecated value.
	if (NavMeshVersion < NAVMESHVER_TILE_RESOLUTIONS_CELLHEIGHT)
	{
		for (int i = 0; i < (uint8)ENavigationDataResolution::MAX; ++i)
		{
			SetCellHeight((ENavigationDataResolution)i, CellHeight);
		}
	}

	// If needed, initialize AgentMaxStepHeight from the deprecated value.
	if (NavMeshVersion < NAVMESHVER_TILE_RESOLUTIONS_AGENTMAXSTEPHEIGHT)
	{
		for (int i = 0; i < (uint8)ENavigationDataResolution::MAX; ++i)
		{
			SetAgentMaxStepHeight((ENavigationDataResolution)i, AgentMaxStepHeight);
		}
	}

	// If needed, initialize NavLinkJumpConfigs from the deprecated version.
	if (NavMeshVersion < NAVMESHVER_NAVLINK_JUMP_CONFIGS && NavLinkJumpConfigs.Num() > 0)
	{
		FNavLinkGenerationJumpConfig& NavLinkConfig = NavLinkJumpConfigs.Top();
		NavLinkConfig.bEnabled = NavLinkJumpDownConfig.bEnabled;
		NavLinkConfig.JumpLength = NavLinkJumpDownConfig.JumpLength;
		NavLinkConfig.JumpDistanceFromEdge = NavLinkJumpDownConfig.JumpDistanceFromEdge;
		NavLinkConfig.JumpMaxDepth = NavLinkJumpDownConfig.JumpMaxDepth;
		NavLinkConfig.JumpHeight = NavLinkJumpDownConfig.JumpHeight;
		NavLinkConfig.JumpEndsHeightTolerance = NavLinkJumpDownConfig.JumpEndsHeightTolerance;
		NavLinkConfig.SamplingSeparationFactor = NavLinkJumpDownConfig.SamplingSeparationFactor;
		NavLinkConfig.FilterDistanceThreshold = NavLinkJumpDownConfig.FilterDistanceThreshold;
		NavLinkConfig.LinkBuilderFlags = NavLinkJumpDownConfig.LinkBuilderFlags;
		NavLinkConfig.AreaClass_DEPRECATED = NavLinkJumpDownConfig.AreaClass_DEPRECATED;
		NavLinkConfig.DownDirectionAreaClass = NavLinkJumpDownConfig.DownDirectionAreaClass;
		NavLinkConfig.UpDirectionAreaClass = NavLinkJumpDownConfig.UpDirectionAreaClass;
		NavLinkConfig.LinkProxyClass = NavLinkJumpDownConfig.LinkProxyClass;
		NavLinkConfig.LinkProxyId = NavLinkJumpDownConfig.LinkProxyId;
		// LinkProxy and bLinkProxyRegistered are Transient.
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

	for (uint8 Index = 0; Index < (uint8)ENavigationDataResolution::MAX; Index++)
	{
		UE_CLOG(TileSizeUU < GetCellSize((ENavigationDataResolution)Index), LogNavigation, Error, TEXT("%s: TileSizeUU (%f) being less than CellSize (%f) is an invalid case and will cause navmesh generation issues.")
			, *GetName(), TileSizeUU, GetCellSize((ENavigationDataResolution)Index));
	}
	
	if (!UWorld::IsPartitionedWorld(GetWorld()))
	{
		bIsWorldPartitioned = false;
	}
	
	RecreateDefaultFilter();
	UpdatePolyRefBitsPreview();
}

void ARecastNavMesh::BeginDestroy()
{
	UNavigationSystemBase::OnNavigationInitStartStaticDelegate().RemoveAll(this);

	Super::BeginDestroy();
}

void ARecastNavMesh::CheckToDiscardSubLevelNavData(const UNavigationSystemBase& BaseNavSys)
{
	// This used to be in ARecastNavMesh::PostInitProperties() but the OwningWorld is not always available, it might be too soon to query it.
	// Moved here so sublevel data can be discarded when requested.
	const UWorld* OwningWorld = GetWorld();
	const UNavigationSystemV1* NavSys = Cast<UNavigationSystemV1>(&BaseNavSys);
	if (OwningWorld && NavSys->ShouldDiscardSubLevelNavData(this))
	{
		// Get rid of instances saved within levels that are streamed-in
		if ((GEngine->IsSettingUpPlayWorld() == false) // this is a @HACK
			&& (OwningWorld->PersistentLevel != GetLevel())
			// If we are cooking, then let them all pass.
			// They will be handled at load-time when running.
			&& (IsRunningCommandlet() == false))
		{
			UE_LOGF(LogNavigation, Verbose, "%ls Discarding %ls due to it not being part of PersistentLevel.", ANSI_TO_TCHAR(__FUNCTION__), *GetFullNameSafe(this));

			// Marking self for deletion 
			CleanUpAndMarkPendingKill();
		}
	}
}

bool ARecastNavMesh::IsGeneratingLinks() const
{
	return bGenerateNavLinks && UE::NavMesh::Private::bAllowLinkGeneration;
}

void ARecastNavMesh::RegisterGeneratedLinksProxy(FNavLinkGenerationJumpConfig& InOutNavLinkJumpConfig)
{
	if (!InOutNavLinkJumpConfig.bLinkProxyRegistered && InOutNavLinkJumpConfig.LinkProxy)
	{
		if (UWorld* World = GetWorld())
		{
			UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);
			if (NavSys)
			{
				UE_LOGF(LogNavLink, Log, "RegisterLinkProxy id: %llu ptr: 0x%p", InOutNavLinkJumpConfig.LinkProxyId.GetId(), InOutNavLinkJumpConfig.LinkProxy.Get());

				NavSys->RegisterCustomLink(*InOutNavLinkJumpConfig.LinkProxy);
				InOutNavLinkJumpConfig.bLinkProxyRegistered = true;
			}
		}
	}
}

void ARecastNavMesh::UnregisterGeneratedLinksProxy(FNavLinkGenerationJumpConfig& InOutNavLinkJumpConfig)
{
	if (InOutNavLinkJumpConfig.bLinkProxyRegistered && InOutNavLinkJumpConfig.LinkProxy)
	{
		if (UWorld* World = GetWorld())
		{
			UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);
			if (NavSys)
			{
				UE_LOGF(LogNavLink, Log, "UnregisterLinkProxy id: %llu ptr: 0x%p", InOutNavLinkJumpConfig.LinkProxyId.GetId(), InOutNavLinkJumpConfig.LinkProxy.Get());

				NavSys->UnregisterCustomLink(*InOutNavLinkJumpConfig.LinkProxy);
				InOutNavLinkJumpConfig.bLinkProxyRegistered = false;
			}
		}
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
// Deprecated
void ARecastNavMesh::RegisterGeneratedLinksProxy()
{
#if	WITH_EDITOR	
	if (!NavLinkJumpDownConfig.bLinkProxyRegistered && NavLinkJumpDownConfig.LinkProxy)
	{
		if (UWorld* World = GetWorld())
		{
			UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);
			if (NavSys)
			{
				UE_LOGF(LogNavLink, Log, "RegisterLinkProxy id: %llu ptr: 0x%p", NavLinkJumpDownConfig.LinkProxyId.GetId(), NavLinkJumpDownConfig.LinkProxy.Get());

				NavSys->RegisterCustomLink(*NavLinkJumpDownConfig.LinkProxy);
				NavLinkJumpDownConfig.bLinkProxyRegistered = true;
			}
		}
	}
#endif // WITH_EDITOR
}

// Deprecated
void ARecastNavMesh::UnregisterGeneratedLinksProxy()
{
#if	WITH_EDITOR
	if (NavLinkJumpDownConfig.bLinkProxyRegistered && NavLinkJumpDownConfig.LinkProxy)
	{
		if (UWorld* World = GetWorld())
		{
			UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);
			if (NavSys)
			{
				UE_LOGF(LogNavLink, Log, "UnregisterLinkProxy id: %llu ptr: 0x%p", NavLinkJumpDownConfig.LinkProxyId.GetId(), NavLinkJumpDownConfig.LinkProxy.Get());

				NavSys->UnregisterCustomLink(*NavLinkJumpDownConfig.LinkProxy);
				NavLinkJumpDownConfig.bLinkProxyRegistered = false;
			}
		}
	}
#endif // WITH_EDITOR	
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

// Deprecated
void ARecastNavMesh::CreateAndRegisterJumpDownLinksProxy(const FNavLinkId LinkProxyId /*= FNavLinkId::GenerateUniqueId()*/)
{
#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS	
	if (!ensureMsgf(NavLinkJumpDownConfig.LinkProxy == nullptr, TEXT("LinkProxy is expected to have been cleanup before creating a new one. A new proxy was not created.")))
	{
		return;
	}
	
	UBaseGeneratedNavLinksProxy* LinkProxy = NewObject<UBaseGeneratedNavLinksProxy>(this, NavLinkJumpDownConfig.LinkProxyClass);
	if (LinkProxy)
	{
		NavLinkJumpDownConfig.LinkProxyId = LinkProxyId;
		LinkProxy->UpdateLinkId(NavLinkJumpDownConfig.LinkProxyId);
		LinkProxy->SetOwner(this);
		NavLinkJumpDownConfig.LinkProxy = LinkProxy;

		RegisterGeneratedLinksProxy();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA	
}

void ARecastNavMesh::CreateAndRegisterJumpLinksProxy(FNavLinkGenerationJumpConfig& InOutNavLinkJumpConfig, 
	const FNavLinkId LinkProxyId /*= FNavLinkId::GenerateUniqueId()*/)
{
	if (!ensureMsgf(InOutNavLinkJumpConfig.LinkProxy == nullptr, TEXT("LinkProxy is expected to have been cleanup before creating a new one. A new proxy was not created.")))
	{
		return;
	}

	UBaseGeneratedNavLinksProxy* LinkProxy = NewObject<UBaseGeneratedNavLinksProxy>(this, InOutNavLinkJumpConfig.LinkProxyClass);
	if (LinkProxy)
	{
		InOutNavLinkJumpConfig.LinkProxyId = LinkProxyId;
		LinkProxy->UpdateLinkId(InOutNavLinkJumpConfig.LinkProxyId);
		LinkProxy->SetOwner(this);
		InOutNavLinkJumpConfig.LinkProxy = LinkProxy;

		RegisterGeneratedLinksProxy(InOutNavLinkJumpConfig);
	}
}

void ARecastNavMesh::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	if (GetActorLocation().IsNearlyZero() == false)
	{
		ApplyWorldOffset(GetActorLocation(), /*unused*/false);
	}

	// Create and register link proxy if enabled.

	if (IsGeneratingLinks())
	{
		for (FNavLinkGenerationJumpConfig& NavLinkJumpConfig : NavLinkJumpConfigs)
		{
			if (NavLinkJumpConfig.bEnabled && NavLinkJumpConfig.LinkProxyClass.Get() != nullptr)
			{
				// If it's the first time, LinkProxyId will have the default invalid value so we need to generate an Id.
				const FNavLinkId Id = NavLinkJumpConfig.LinkProxyId.IsValid() ? NavLinkJumpConfig.LinkProxyId : FNavLinkId::GenerateUniqueId();
				CreateAndRegisterJumpLinksProxy(NavLinkJumpConfig, Id);
			}
		}
	}
}

void ARecastNavMesh::PostUnregisterAllComponents()
{
	Super::PostUnregisterAllComponents();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Deprecated
	UnregisterGeneratedLinksProxy();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	for (FNavLinkGenerationJumpConfig& NavLinkJumpConfig : NavLinkJumpConfigs)
	{
		UnregisterGeneratedLinksProxy(NavLinkJumpConfig);
		
		NavLinkJumpConfig.LinkProxy = nullptr;
	}
}

void ARecastNavMesh::PostInitProperties()
{
	if (HasAnyFlags(RF_ClassDefaultObject) == true)
	{
		SetDrawDistance(DefaultDrawDistance);

		static const FNavMeshConfig::FRecastNamedFiltersCreator RecastNamedFiltersCreator(bUseVirtualFilters);
		NavLinkFlag = FNavMeshConfig::NavLinkFlag;
	}
	
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad) == false)
	{
		RecreateDefaultFilter();
	}

	// voxel cache requires the same rasterization setup for all navmeshes, as it's stored in octree
	if (IsVoxelCacheEnabled() && !HasAnyFlags(RF_ClassDefaultObject))
	{
		ARecastNavMesh* DefOb = (ARecastNavMesh*)ARecastNavMesh::StaticClass()->GetDefaultObject();

		if (TileSizeUU != DefOb->TileSizeUU)
		{
			UE_LOGF(LogNavigation, Warning, "%ls param: TileSizeUU(%f) differs from config settings, forcing value %f so it can be used with voxel cache!",
				*GetNameSafe(this), TileSizeUU, DefOb->TileSizeUU);
			
			TileSizeUU = DefOb->TileSizeUU;
		}

		for (int i = 0; i < (uint8)ENavigationDataResolution::MAX; ++i)
		{
			const float CurrentCellSize = NavMeshResolutionParams[i].CellSize;
			const float DefaultObjectCellSize = DefOb->NavMeshResolutionParams[i].CellSize;
			if (CurrentCellSize != DefaultObjectCellSize)
			{
				UE_LOGF(LogNavigation, Warning, "%ls param: CellSize(%f) differs from config settings, forcing value %f so it can be used with voxel cache!",
					*GetNameSafe(this), CurrentCellSize, DefaultObjectCellSize);

				NavMeshResolutionParams[i].CellSize = DefaultObjectCellSize;
			}

			const float CurrentCellHeight = NavMeshResolutionParams[i].CellHeight;
			const float DefaultObjectCellHeight = DefOb->NavMeshResolutionParams[i].CellHeight;
			if (CurrentCellHeight != DefaultObjectCellHeight)
			{
				UE_LOGF(LogNavigation, Warning, "%ls param: CellHeight(%f) differs from config settings, forcing value %f so it can be used with voxel cache!",
					*GetNameSafe(this), CurrentCellHeight, DefaultObjectCellHeight);

				NavMeshResolutionParams[i].CellHeight = DefaultObjectCellHeight;
			}

			const float CurrentAgentMaxStepHeight = NavMeshResolutionParams[i].AgentMaxStepHeight;
			const float DefaultObjectAgentMaxStepHeight = DefOb->NavMeshResolutionParams[i].AgentMaxStepHeight;
			if (CurrentAgentMaxStepHeight != DefaultObjectAgentMaxStepHeight)
			{
				UE_LOGF(LogNavigation, Warning, "%ls param: AgentMaxStepHeight(%f) differs from config settings, forcing value %f so it can be used with voxel cache!",
					*GetNameSafe(this), CurrentAgentMaxStepHeight, DefaultObjectAgentMaxStepHeight);

				NavMeshResolutionParams[i].AgentMaxStepHeight = DefaultObjectAgentMaxStepHeight;
			}			
		}

		if (AgentMaxSlope != DefOb->AgentMaxSlope)
		{
			UE_LOGF(LogNavigation, Warning, "%ls param: AgentMaxSlope(%f) differs from config settings, forcing value %f so it can be used with voxel cache!",
				*GetNameSafe(this), AgentMaxSlope, DefOb->AgentMaxSlope);

			AgentMaxSlope = DefOb->AgentMaxSlope;
		}
	}
	
#if WITH_EDITORONLY_DATA
	bAllowWorldPartitionedNavMesh = UWorld::IsPartitionedWorld(GetWorld());
#endif // WITH_EDITORONLY_DATA

	UpdatePolyRefBitsPreview();
}

void ARecastNavMesh::RecreateDefaultFilter()
{
	DefaultQueryFilter->SetFilterType<FRecastQueryFilter>();

	check(DefaultMaxSearchNodes >= 0 && DefaultMaxSearchNodes <= (float)TNumericLimits<uint32>::Max());
	DefaultQueryFilter->SetMaxSearchNodes(static_cast<int32>(DefaultMaxSearchNodes));

	FRecastQueryFilter* DetourFilter = static_cast<FRecastQueryFilter*>(DefaultQueryFilter->GetImplementation());
	DetourFilter->SetIsVirtual(bUseVirtualFilters);
	DetourFilter->setHeuristicScale(HeuristicScale);
	// clearing out the 'navlink flag' from included flags since it would make 
	// dtQueryFilter::passInlineFilter pass navlinks of area classes with
	// AreaFlags == 0 (like NavArea_Null), which should mean 'unwalkable'
	DetourFilter->setIncludeFlags(DetourFilter->getIncludeFlags() & (~ARecastNavMesh::GetNavLinkFlag()));

	for (int32 Idx = 0; Idx < SupportedAreas.Num(); Idx++)
	{
		const FSupportedAreaData& AreaData = SupportedAreas[Idx];
		
		UNavArea* DefArea = nullptr;
		if (AreaData.AreaClass)
		{
			DefArea = ((UClass*)AreaData.AreaClass)->GetDefaultObject<UNavArea>();
		}

		if (DefArea)
		{
			DetourFilter->SetAreaCost(IntCastChecked<uint8>(AreaData.AreaID), DefArea->DefaultCost);
			DetourFilter->SetFixedAreaEnteringCost(IntCastChecked<uint8>(AreaData.AreaID), DefArea->GetFixedAreaEnteringCost());
		}
	}
}

void ARecastNavMesh::UpdatePolyRefBitsPreview()
{
	static const int32 TotalBits = (sizeof(dtPolyRef) * 8);

	FRecastNavMeshGenerator::CalcPolyRefBits(this, PolyRefTileBits, PolyRefNavPolyBits);
	PolyRefSaltBits = TotalBits - PolyRefTileBits - PolyRefNavPolyBits;
}

void ARecastNavMesh::OnNavAreaAdded(const UClass* NavAreaClass, int32 AgentIndex)
{
	Super::OnNavAreaAdded(NavAreaClass, AgentIndex);

	// update navmesh query filter with area costs
	const int32 AreaID = GetAreaID(NavAreaClass);
	if (AreaID != INDEX_NONE)
	{
		UNavArea* DefArea = ((UClass*)NavAreaClass)->GetDefaultObject<UNavArea>();

		DefaultQueryFilter->SetAreaCost(IntCastChecked<uint8>(AreaID), DefArea->DefaultCost);
		DefaultQueryFilter->SetFixedAreaEnteringCost(IntCastChecked<uint8>(AreaID), DefArea->GetFixedAreaEnteringCost());
	}

	// update generator's cached data
	FRecastNavMeshGenerator* MyGenerator = static_cast<FRecastNavMeshGenerator*>(GetGenerator());
	if (MyGenerator)
	{
		MyGenerator->OnAreaAdded(NavAreaClass, AreaID);
	}
}

void ARecastNavMesh::OnNavAreaRemoved(const UClass* NavAreaClass)
{
	// In an ideal world we'd reset the DefaultQueryFilter Costs here for the AreaID but its
	// not really worth changing the API as we shouldn't be using these values anyway.

	FRecastNavMeshGenerator* MyGenerator = static_cast<FRecastNavMeshGenerator*>(GetGenerator());
	if (MyGenerator)
	{
		MyGenerator->OnAreaRemoved(NavAreaClass);
	}

	Super::OnNavAreaRemoved(NavAreaClass);
}

void ARecastNavMesh::OnNavAreaChanged()
{
	if (RecastNavMeshImpl)
	{
		RecastNavMeshImpl->OnAreaCostChanged();
	}
}

int32 ARecastNavMesh::GetNewAreaID(const UClass* AreaClass) const
{
	if (AreaClass == FNavigationSystem::GetDefaultWalkableArea())
	{
		return RECAST_DEFAULT_AREA;
	}

	if (AreaClass == UNavArea_Null::StaticClass())
	{
		return RECAST_NULL_AREA;
	}

	if (AreaClass == UNavArea_LowHeight::StaticClass())
	{
		return RECAST_LOW_AREA;
	}

	int32 FreeAreaID = Super::GetNewAreaID(AreaClass);
	while (FreeAreaID == RECAST_NULL_AREA || FreeAreaID == RECAST_DEFAULT_AREA || FreeAreaID == RECAST_LOW_AREA)
	{
		FreeAreaID++;
	}

	check(FreeAreaID < GetMaxSupportedAreas());
	return FreeAreaID;
}

FColor ARecastNavMesh::GetAreaIDColor(uint8 AreaID) const
{
	const UClass* AreaClass = GetAreaClass(AreaID);
	const UNavArea* DefArea = AreaClass ? ((UClass*)AreaClass)->GetDefaultObject<UNavArea>() : NULL;
	return DefArea ? DefArea->DrawColor : FColor::Red;
}

void ARecastNavMesh::SortAreasForGenerator(TArray<FRecastAreaNavModifierElement>& Modifiers) const
{
	// initialize costs for sorting
	float AreaCosts[RECAST_MAX_AREAS];
	float AreaFixedCosts[RECAST_MAX_AREAS];
	DefaultQueryFilter->GetAllAreaCosts(AreaCosts, AreaFixedCosts, RECAST_MAX_AREAS);

	for (auto& Element : Modifiers)
	{
		if (Element.Areas.Num())
		{
			FAreaNavModifier& AreaMod = Element.Areas[0];
			const int32 AreaId = GetAreaID(AreaMod.GetAreaClass());
			if (AreaId >= 0 && AreaId < RECAST_MAX_AREAS)
			{
				AreaMod.Cost = AreaCosts[AreaId];
				AreaMod.FixedCost = AreaFixedCosts[AreaId];
			}
		}
	}

	struct FNavAreaSortPredicate
	{
		FORCEINLINE bool operator()(const FRecastAreaNavModifierElement& ElementA, const FRecastAreaNavModifierElement& ElementB) const
		{
			if (ElementA.Areas.Num() == 0 || ElementB.Areas.Num() == 0)
			{
				return ElementA.Areas.Num() <= ElementB.Areas.Num();
			}

			// assuming composite modifiers has same area type
			const FAreaNavModifier& A = ElementA.Areas[0];
			const FAreaNavModifier& B = ElementB.Areas[0];
			
			const bool bIsAReplacing = (A.GetAreaClassToReplace() != NULL);
			const bool bIsBReplacing = (B.GetAreaClassToReplace() != NULL);
			if (bIsAReplacing != bIsBReplacing)
			{
				// We want the "Replace" modifiers last because we want them to replace any NavArea from the "Apply" modifiers.
				return !bIsAReplacing;
			}

			if (A.Cost != B.Cost)
			{
				return A.Cost < B.Cost;
			}

			if (A.FixedCost != B.FixedCost)
			{
				return A.FixedCost < B.FixedCost;
			}

			return ElementA.NavMeshResolution < ElementB.NavMeshResolution;
		}
	};

	Modifiers.Sort(FNavAreaSortPredicate());
}

const TSet<FIntPoint>& ARecastNavMesh::GetActiveTileSet() const
{
	const FRecastNavMeshGenerator* MyGenerator = static_cast<const FRecastNavMeshGenerator*>(GetGenerator());
	check(MyGenerator);
	return MyGenerator->ActiveTileSet;
}

TSet<FIntPoint>& ARecastNavMesh::GetActiveTileSet()
{
	FRecastNavMeshGenerator* MyGenerator = static_cast<FRecastNavMeshGenerator*>(GetGenerator());
	check(MyGenerator);
	return MyGenerator->ActiveTileSet;
}

void ARecastNavMesh::LogRecastTile(const TCHAR* Caller, const FName& Prefix, const FName& OperationName, const dtNavMesh& DetourMesh, const int32 TileX, const int32 TileY, const int32 LayerIndex, const dtTileRef TileRef) const
{
	UE_LOGF(LogNavigation, VeryVerbose, "%ls> %ls Tile (%d,%d:%d), %ls TileId: %d, Salt: %d, TileRef: 0x%llx (%ls)",
		   *GetName(), *Prefix.ToString(),
		   TileX, TileY, LayerIndex, *OperationName.ToString(),
		   DetourMesh.decodePolyIdTile(TileRef), DetourMesh.decodePolyIdSalt(TileRef), TileRef, 
		   Caller);
}

void ARecastNavMesh::RestrictBuildingToActiveTiles(bool InRestrictBuildingToActiveTiles)
{
	FRecastNavMeshGenerator* MyGenerator = static_cast<FRecastNavMeshGenerator*>(GetGenerator());
	if (MyGenerator)
	{
		MyGenerator->RestrictBuildingToActiveTiles(InRestrictBuildingToActiveTiles);
	}
}

void ARecastNavMesh::OnRegistered()
{
	// (Re)allocate if necessary (might have been destroyed by a call to Cleanup)
	// Note that it needs to be created before calling the base class
	if (RecastNavMeshImpl == nullptr)
	{
		RecastNavMeshImpl = new FPImplRecastNavMesh(this);
	}

	// This check can fail when the NavMeshVersion indicates the map needs the nav mesh rebuilt
	ensure(RecastNavMeshImpl->GetRecastMesh() == nullptr || RecastNavMeshImpl->GetRecastMesh()->getBVQuantFactor((uint8)ENavigationDataResolution::Default) != 0);

	// Off-mesh link deferred connection should have been resolved by OnAreaCostChanged
	// during ProcessNavAreas, which runs before OnRegistered.
	ensure(RecastNavMeshImpl->GetRecastMesh() == nullptr || !RecastNavMeshImpl->GetRecastMesh()->isDeferringOffMeshLinking());

	Super::OnRegistered();

	// Re-register any generated link proxies that couldn't be registered
	// when created (NavSys was null), or that need re-registration after
	// NavSys was destroyed and recreated (flags reset in OnUnregistered).
	for (FNavLinkGenerationJumpConfig& NavLinkJumpConfig : NavLinkJumpConfigs)
	{
		RegisterGeneratedLinksProxy(NavLinkJumpConfig);
	}
}

void ARecastNavMesh::OnUnregistered()
{
	Super::OnUnregistered();

	// Reset link proxy registration flags so they can be re-registered
	// with a new NavSys instance in OnRegistered. The proxies themselves
	// remain valid — only the registration state is cleared, since the
	// NavSys they were registered with is being torn down.
	for (FNavLinkGenerationJumpConfig& NavLinkJumpConfig : NavLinkJumpConfigs)
	{
		NavLinkJumpConfig.bLinkProxyRegistered = false;
	}
}

void ARecastNavMesh::SerializeRecastNavMesh(FArchive& Ar, FPImplRecastNavMesh*& NavMesh, int32 InNavMeshVersion)
{
	if (!Ar.IsLoading()	&& NavMesh == NULL)
	{
		return;
	}

	if (Ar.IsLoading())
	{
		// allocate if necessary
		if (RecastNavMeshImpl == NULL)
		{
			RecastNavMeshImpl = new FPImplRecastNavMesh(this);
		}
	}
	
	if (RecastNavMeshImpl)
	{
		RecastNavMeshImpl->Serialize(Ar, InNavMeshVersion);
	}	
}

void ARecastNavMesh::Serialize( FArchive& Ar )
{
	UE_LOGF(LogNavigation, Verbose, "%s %ls %ls, '%ls'", __FUNCTION__,
		Ar.IsLoading() ? TEXT("'loading'") :
			(Ar.IsSaving() ? TEXT("'saving'") :
				(Ar.IsTransacting() ? TEXT("'transacting'") : TEXT("'unknown'"))),
		*Ar.GetArchiveName(),
		*GetFullNameSafe(this));

	Super::Serialize(Ar);

	Ar << NavMeshVersion;

	//@todo: How to handle loading nav meshes saved w/ recast when recast isn't present????
	
	// when writing, write a zero here for now.  will come back and fill it in later.
	uint32 RecastNavMeshSizeBytes = 0;
	int64 RecastNavMeshSizePos = Ar.Tell();
	{
		Ar << RecastNavMeshSizeBytes;
	}

	if (Ar.IsLoading())
	{
		auto CleanUpBadVersion = [&Ar, RecastNavMeshSizePos, RecastNavMeshSizeBytes, this]()
		{
			// incompatible, just skip over this data.  navmesh needs rebuilt.
			Ar.Seek(RecastNavMeshSizePos + RecastNavMeshSizeBytes);

			// Mark self for delete
			CleanUpAndMarkPendingKill();
		};

		if (NavMeshVersion >= NAVMESHVER_MIN_COMPATIBLE && NavMeshVersion < NAVMESHVER_LATEST)
		{
			UE_LOGF(LogNavigation, Display, "%ls: ARecastNavMesh: Navmesh version %d is compatible but not at latest (%d). Rebuild navmesh to use the latest features. \n", *GetFullName(), NavMeshVersion, NAVMESHVER_LATEST);
		}

		if (NavMeshVersion < NAVMESHVER_MIN_COMPATIBLE)
		{
			UE_LOGF(LogNavigation, Warning, "%ls: ARecastNavMesh: Navmesh version %d < Min compatible %d. Navmesh needs to be rebuilt. \n", *GetFullName(), NavMeshVersion, NAVMESHVER_MIN_COMPATIBLE);
			CleanUpBadVersion();
		}
		else if (NavMeshVersion > NAVMESHVER_LATEST)
		{
			UE_LOGF(LogNavigation, Warning, "%ls: ARecastNavMesh: Navmesh version %d > NAVMESHVER_LATEST %d. Newer navmesh should not be loaded by older code. At a minimum the nav mesh needs to be rebuilt. \n", *GetFullName(), NavMeshVersion, NAVMESHVER_LATEST);
			CleanUpBadVersion();
		}
		else if (RecastNavMeshSizeBytes > 4)
		{
			SerializeRecastNavMesh(Ar, RecastNavMeshImpl, NavMeshVersion);
			if (Ar.IsError())
			{
				return;
			}

#if !(UE_BUILD_SHIPPING)
			RequestDrawingUpdate();
#endif //!(UE_BUILD_SHIPPING)
		}
		else
		{
			// empty, just skip over this data
			Ar.Seek( RecastNavMeshSizePos + RecastNavMeshSizeBytes );
			// if it's not getting filled it's better to just remove it
			if (RecastNavMeshImpl)
			{
				RecastNavMeshImpl->ReleaseDetourNavMesh();
			}
		}
	}
	else
	{
		SerializeRecastNavMesh(Ar, RecastNavMeshImpl, NavMeshVersion);
		if (Ar.IsError())
		{
			return;
		}

		if (Ar.IsSaving())
		{
			int64 CurPos = Ar.Tell();
			RecastNavMeshSizeBytes = IntCastChecked<uint32>(CurPos - RecastNavMeshSizePos);
			Ar.Seek(RecastNavMeshSizePos);
			Ar << RecastNavMeshSizeBytes;
			Ar.Seek(CurPos);
		}
	}
}

#if WITH_EDITOR
bool ARecastNavMesh::CanEditChange(const FProperty* InProperty) const
{
#if !WITH_NAVMESH_CLUSTER_LINKS
	if (InProperty)
	{
		const FName PropertyName = InProperty->GetFName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(ARecastNavMesh, bDrawClusters))
		{
			return false;
		}
	}
#endif // WITH_NAVMESH_CLUSTER_LINKS

	return Super::CanEditChange(InProperty);
}
#endif // WITH_EDITOR

void ARecastNavMesh::SetConfig(const FNavDataConfig& Src) 
{
	// Step 1: set NavDataConfig
	NavDataConfig = Src;

	// Set the NavAgentProperties that are being used to configure navmmesh
	SetNavAgentProperties(Src);

	if (!Src.HasStepHeightOverride())
	{
		// If there is no override, use the navmesh value.
		NavDataConfig.AgentStepHeight = GetAgentMaxStepHeight(ENavigationDataResolution::Default);
	}
	
	// Step 2: update ARecastNavMesh from the new NavDataConfig
	AgentHeight = NavDataConfig.AgentHeight;
	AgentRadius = NavDataConfig.AgentRadius;

	if (Src.HasStepHeightOverride())
	{
		// If there is an override, apply it to all resolutions
		for (int32 Index = 0; Index < (int32)ENavigationDataResolution::MAX; Index++)
		{
			SetAgentMaxStepHeight((ENavigationDataResolution)Index, NavDataConfig.AgentStepHeight);
		}
	}
}

void ARecastNavMesh::FillConfig(FNavDataConfig& Dest)
{
	Dest = NavDataConfig;
	Dest.AgentHeight = AgentHeight;
	Dest.AgentRadius = AgentRadius;
	Dest.AgentStepHeight = GetAgentMaxStepHeight(ENavigationDataResolution::Default);
}

void ARecastNavMesh::BeginBatchQuery() const
{
#if RECAST_ASYNC_REBUILDING
	if (BatchQueryCounter <= 0)
	{
		BatchQueryCounter = 0;
	}

	BatchQueryCounter++;
#endif // RECAST_ASYNC_REBUILDING
}

void ARecastNavMesh::FinishBatchQuery() const
{
#if RECAST_ASYNC_REBUILDING
	BatchQueryCounter--;
#endif // RECAST_ASYNC_REBUILDING
}

FBox ARecastNavMesh::GetNavMeshBounds() const
{
	FBox Bounds;
	if (RecastNavMeshImpl)
	{
		Bounds = RecastNavMeshImpl->GetNavMeshBounds();
	}

	return Bounds;
}

// Deprecated
FBox ARecastNavMesh::GetNavMeshTileBounds(int32 TileIndex) const
{
	if (RecastNavMeshImpl && (TileIndex != INDEX_NONE))
	{
		TArray<FNavTileRef> TileRefs;
		FNavTileRef::DeprecatedMakeTileRefsFromTileIds(RecastNavMeshImpl, { static_cast<uint32>(TileIndex) }, TileRefs);
		return GetNavMeshTileBounds(TileRefs[0]);
	}

	return FBox(ForceInit);
}

FBox ARecastNavMesh::GetNavMeshTileBounds(FNavTileRef TileRef) const
{
	FBox Bounds(ForceInit);

	if (HasValidNavmesh() && TileRef.IsValid())
	{
		// workaround for privacy issue in the recast API
		dtNavMesh const* const ConstRecastNavMesh = RecastNavMeshImpl->DetourNavMesh;

		dtMeshTile const* const Tile = ConstRecastNavMesh->getTileByRef(static_cast<dtTileRef>(TileRef));
		if (Tile)
		{
			dtMeshHeader const* const Header = Tile->header;
			if (Header)
			{
				Bounds = Recast2UnrealBox(Header->bmin, Header->bmax);
			}
		}
	}

	return Bounds;
}

// Deprecated
bool ARecastNavMesh::GetNavMeshTileXY(int32 TileIndex, int32& OutX, int32& OutY, int32& OutLayer) const
{
	if (RecastNavMeshImpl && (TileIndex != INDEX_NONE))
	{
		TArray<FNavTileRef> TileRefs;
		FNavTileRef::DeprecatedMakeTileRefsFromTileIds(RecastNavMeshImpl, { static_cast<uint32>(TileIndex) }, TileRefs);
		return GetNavMeshTileXY(TileRefs[0], OutX, OutY, OutLayer);
	}
	return false;
}

bool ARecastNavMesh::GetNavMeshTileXY(FNavTileRef TileRef, int32& OutX, int32& OutY, int32& OutLayer) const
{
	if (HasValidNavmesh() && TileRef.IsValid())
	{
		// workaround for privacy issue in the recast API
		dtNavMesh const* const ConstRecastNavMesh = RecastNavMeshImpl->DetourNavMesh;

		dtMeshTile const* const Tile = ConstRecastNavMesh->getTileByRef(static_cast<dtTileRef>(TileRef));
		if (Tile)
		{
			dtMeshHeader const* const Header = Tile->header;
			if (Header)
			{
				OutX = Header->x;
				OutY = Header->y;
				OutLayer = Header->layer;
				return true;
			}
		}
	}

	return false;
}

bool ARecastNavMesh::GetNavMeshTileXY(const FVector& Point, int32& OutX, int32& OutY) const
{
	return RecastNavMeshImpl && RecastNavMeshImpl->GetNavMeshTileXY(Point, OutX, OutY);
}

// Deprecated
bool ARecastNavMesh::GetNavmeshTileResolution(int32 TileIndex, ENavigationDataResolution& OutResolution) const
{
	if (RecastNavMeshImpl && (TileIndex != INDEX_NONE))
	{
		TArray<FNavTileRef> TileRefs;
		FNavTileRef::DeprecatedMakeTileRefsFromTileIds(RecastNavMeshImpl, { static_cast<uint32>(TileIndex) }, TileRefs);
		return GetNavmeshTileResolution(TileRefs[0], OutResolution);
	}

	return false;
}

bool ARecastNavMesh::GetNavmeshTileResolution(FNavTileRef TileRef, ENavigationDataResolution& OutResolution) const
{
	if (RecastNavMeshImpl)
	{
		if (const dtNavMesh* const RecastNavMesh = RecastNavMeshImpl->DetourNavMesh)
		{
			if (const dtMeshTile* const Tile = RecastNavMesh->getTileByRef(static_cast<dtTileRef>(TileRef)))
			{
				if (const dtMeshHeader* const Header = Tile->header)
				{
					OutResolution = ENavigationDataResolution(Header->resolution);
					return true;
				}
			}
		}
	}

	return false;
}

bool ARecastNavMesh::CheckTileIndicesInValidRange(const FVector& Point, bool& bOutInRange) const
{
	const dtNavMesh* const DetourNavMesh = RecastNavMeshImpl ? RecastNavMeshImpl->DetourNavMesh : nullptr;
	const bool bValidMesh = DetourNavMesh != nullptr;

	if (bValidMesh)
	{
		const FVector RecastPt = Unreal2RecastPoint(Point);
		bOutInRange = DetourNavMesh->isTileLocInValidRange(&RecastPt.X);
	}

	return bValidMesh;
}

// Deprecated
void ARecastNavMesh::GetNavMeshTilesAt(int32 TileX, int32 TileY, TArray<int32>& Indices) const
{
	if (RecastNavMeshImpl)
	{
		TArray<FNavTileRef> Refs;
		GetNavMeshTilesAt(TileX, TileY, Refs);

		TArray<uint32> UnsignedIndices;
		FNavTileRef::DeprecatedGetTileIdsFromNavTileRefs(RecastNavMeshImpl, Refs, UnsignedIndices);

		Indices.Append(UnsignedIndices);
	}
}

void ARecastNavMesh::GetNavMeshTilesAt(int32 TileX, int32 TileY, TArray<FNavTileRef>& OutRefs) const
{	
	if (HasValidNavmesh())
	{
		// workaround for privacy issue in the recast API
		dtNavMesh const* const ConstRecastNavMesh = RecastNavMeshImpl->DetourNavMesh;

		const int32 MaxTiles = ConstRecastNavMesh->getTileCountAt(TileX, TileY);
		TArray<const dtMeshTile*> Tiles;
		Tiles.AddZeroed(MaxTiles);

		const int32 NumTiles = ConstRecastNavMesh->getTilesAt(TileX, TileY, Tiles.GetData(), MaxTiles);
		for (int32 i = 0; i < NumTiles; i++)
		{
			dtTileRef TileRef = ConstRecastNavMesh->getTileRef(Tiles[i]);
			if (TileRef)
			{
				OutRefs.Add(FNavTileRef(TileRef));
			}
		}
	}
}

void ARecastNavMesh::GetAllNavMeshTiles(TArray<FNavTileRef>& OutRefs) const
{
	if (HasValidNavmesh())
	{
		// workaround for privacy issue in the recast API
		const dtNavMesh* const ConstDetourNavMesh = RecastNavMeshImpl->DetourNavMesh;

		const int32 TileCount = ConstDetourNavMesh->getMaxTiles();

		OutRefs.Reserve(OutRefs.Num() + TileCount);
		for (int32 TileIndex = 0; TileIndex < TileCount; ++TileIndex)
		{
			if (const dtMeshTile* const Tile = ConstDetourNavMesh->getTile(TileIndex))
			{
				if (const dtTileRef TileRef = ConstDetourNavMesh->getTileRef(Tile))
				{
					OutRefs.Add(FNavTileRef(TileRef));
				}
			}
		}
	}
}

// Deprecated
bool ARecastNavMesh::GetPolysInTile(int32 TileIndex, TArray<FNavPoly>& Polys) const
{
	if (RecastNavMeshImpl && (TileIndex != INDEX_NONE))
	{
		TArray<FNavTileRef> TileRefs;
		FNavTileRef::DeprecatedMakeTileRefsFromTileIds(RecastNavMeshImpl, { static_cast<uint32>(TileIndex) }, TileRefs);
		return GetPolysInTile(TileRefs[0], Polys);
	}

	return false;
}

bool ARecastNavMesh::GetPolysInTile(FNavTileRef TileRef, TArray<FNavPoly>& Polys) const
{
	if (HasValidNavmesh() && TileRef.IsValid())
	{
		// workaround for privacy issue in the recast API
		const dtNavMesh* const ConstDetourNavMesh = RecastNavMeshImpl->DetourNavMesh;

		const dtMeshTile* Tile = ConstDetourNavMesh->getTileByRef(static_cast<dtTileRef>(TileRef));
		const int32 MaxPolys = Tile && Tile->header ? Tile->header->offMeshBase : 0;
		if (MaxPolys > 0)
		{
			// only ground type polys
			int32 BaseIdx = Polys.Num();
			Polys.AddZeroed(MaxPolys);

			dtPoly* Poly = Tile->polys;
			for (int32 i = 0; i < MaxPolys; i++, Poly++)
			{
				FVector PolyCenter(0);
				for (int k = 0; k < Poly->vertCount; ++k)
				{
					FVector::FReal Vertex[3];
					Tile->verts[Poly->verts[k]].toWorld(Tile->header->bmin, Vertex);
					PolyCenter += Recast2UnrealPoint(Vertex);
				}
				PolyCenter /= Poly->vertCount;

				FNavPoly& OutPoly = Polys[BaseIdx + i];
				OutPoly.Ref = ConstDetourNavMesh->encodePolyId(Tile->salt, ConstDetourNavMesh->getTileIndex(Tile), i);
				OutPoly.Center = PolyCenter;
			}
		}

		return (MaxPolys > 0);
	}

	return false;
}

bool ARecastNavMesh::GetNavLinksInTile(const int32 TileIndex, TArray<FNavPoly>& Polys, const bool bIncludeLinksFromNeighborTiles) const
{
	if (RecastNavMeshImpl == nullptr || RecastNavMeshImpl->DetourNavMesh == nullptr
		|| TileIndex < 0 || TileIndex >= RecastNavMeshImpl->DetourNavMesh->getMaxTiles())
	{
		return false;
	}

	const dtNavMesh* DetourNavMesh = RecastNavMeshImpl->DetourNavMesh;
	const int32 InitialLinkCount = Polys.Num();

	const dtMeshTile* Tile = DetourNavMesh->getTile(TileIndex);
	if (Tile && Tile->header)
	{
		const int32 LinkCount = Tile->header->offMeshConCount;

		if (LinkCount > 0)
		{
			const int32 BaseIdx = Polys.Num();
			Polys.AddZeroed(LinkCount);

			const dtPoly* Poly = Tile->polys;
			for (int32 LinkIndex = 0; LinkIndex < LinkCount; ++LinkIndex, ++Poly)
			{
				FNavPoly& OutPoly = Polys[BaseIdx + LinkIndex];
				const int32 PolyIndex = Tile->header->offMeshBase + LinkIndex;
				OutPoly.Ref = DetourNavMesh->encodePolyId(Tile->salt, TileIndex, PolyIndex);
				FVector::FReal VertexA[3];
				FVector::FReal VertexB[3];
				Tile->verts[Poly->verts[0]].toWorld(Tile->header->bmin, VertexA);
				Tile->verts[Poly->verts[1]].toWorld(Tile->header->bmin, VertexB);
				OutPoly.Center = (Recast2UnrealPoint(VertexA) + Recast2UnrealPoint(VertexB)) / 2;
			}
		}

		if (bIncludeLinksFromNeighborTiles)
		{
			TArray<const dtMeshTile*> NeighborTiles;
			NeighborTiles.Reserve(32);
			for (int32 SideIndex = 0; SideIndex < 8; ++SideIndex)
			{
				const int32 StartIndex = NeighborTiles.Num();
				const int32 NeighborCount = DetourNavMesh->getNeighbourTilesCountAt(Tile->header->x, Tile->header->y, SideIndex);
				if (NeighborCount > 0)
				{
					const unsigned char oppositeSide = (unsigned char)dtOppositeTile(SideIndex);

					NeighborTiles.AddZeroed(NeighborCount);
					int32 NeighborX = Tile->header->x;
					int32 NeighborY = Tile->header->y;

					if (DetourNavMesh->getNeighbourCoords(Tile->header->x, Tile->header->y, SideIndex, NeighborX, NeighborY))
					{
						DetourNavMesh->getTilesAt(NeighborX, NeighborY, NeighborTiles.GetData() + StartIndex, NeighborCount);
					}

					for (const dtMeshTile* NeighborTile : NeighborTiles)
					{
						if (NeighborTile && NeighborTile->header && NeighborTile->offMeshCons)
						{
							const dtTileRef NeighborTileId = DetourNavMesh->getTileRef(NeighborTile);

							for (int32 LinkIndex = 0; LinkIndex < NeighborTile->header->offMeshConCount; ++LinkIndex)
							{
								dtOffMeshConnection* targetCon = &NeighborTile->offMeshCons[LinkIndex];
								if (targetCon->side != oppositeSide)
								{
									continue;
								}

								const unsigned char biDirFlag = targetCon->getBiDirectional() ? DT_LINK_FLAG_OFFMESH_CON_BIDIR : 0;

								const dtPoly* targetPoly = &NeighborTile->polys[targetCon->poly];
								// Skip off-mesh connections which start location could not be connected at all.
								if (targetPoly->firstLink == DT_NULL_LINK)
								{
									continue;
								}

								FNavPoly& OutPoly = Polys[Polys.AddZeroed()];
								OutPoly.Ref = NeighborTileId | targetCon->poly;
								OutPoly.Center = (Recast2UnrealPoint(&targetCon->pos[0]) + Recast2UnrealPoint(&targetCon->pos[3])) / 2;
							}
						}
					}

					NeighborTiles.Reset();
				}
			}
		}
	}

	return (Polys.Num() - InitialLinkCount > 0);
}

int32 ARecastNavMesh::GetNavMeshTilesCount() const
{
	int32 NumTiles = 0;
	if (RecastNavMeshImpl)
	{
		NumTiles = RecastNavMeshImpl->GetNavMeshTilesCount();
	}

	return NumTiles;
}

int32 ARecastNavMesh::GetNumActiveTiles() const
{
	const FRecastNavMeshGenerator* Generator = static_cast<const FRecastNavMeshGenerator*>(GetGenerator());
	return Generator ? Generator->GetNumActiveTiles() : 0;
}

void ARecastNavMesh::RemoveTileCacheLayers(int32 TileX, int32 TileY)
{
	if (RecastNavMeshImpl)
	{
		RecastNavMeshImpl->RemoveTileCacheLayers(TileX, TileY);
	}
}
	
void ARecastNavMesh::AddTileCacheLayers(int32 TileX, int32 TileY, const TArray<FNavMeshTileData>& InLayers)
{
	if (RecastNavMeshImpl)
	{
		RecastNavMeshImpl->AddTileCacheLayers(TileX, TileY, InLayers);
	}
}

#if RECAST_INTERNAL_DEBUG_DATA
void ARecastNavMesh::RemoveTileDebugData(int32 TileX, int32 TileY)
{
	if (RecastNavMeshImpl)
	{
		RecastNavMeshImpl->DebugDataMap.Remove(FIntPoint(TileX, TileY));
	}
}

void ARecastNavMesh::AddTileDebugData(int32 TileX, int32 TileY, const FRecastInternalDebugData& InTileDebugData)
{
	if (RecastNavMeshImpl)
	{
		RecastNavMeshImpl->DebugDataMap.Add(FIntPoint(TileX, TileY), InTileDebugData);
	}
}
#endif //RECAST_INTERNAL_DEBUG_DATA

void ARecastNavMesh::MarkEmptyTileCacheLayers(int32 TileX, int32 TileY)
{
	if (RecastNavMeshImpl && bStoreEmptyTileLayers)
	{
		RecastNavMeshImpl->MarkEmptyTileCacheLayers(TileX, TileY);
	}
}
	
TArray<FNavMeshTileData> ARecastNavMesh::GetTileCacheLayers(int32 TileX, int32 TileY) const
{
	if (RecastNavMeshImpl)
	{
		return RecastNavMeshImpl->GetTileCacheLayers(TileX, TileY);
	}
	
	return TArray<FNavMeshTileData>();
}

#if !UE_BUILD_SHIPPING
int32 ARecastNavMesh::GetCompressedTileCacheSize()
{
	return RecastNavMeshImpl ? RecastNavMeshImpl->GetCompressedTileCacheSize() : 0;
}
#endif

bool ARecastNavMesh::IsResizable() const
{
	// Ignore bFixedTilePoolSize when in EditorWorldPartitionBuildMode
	if (const UWorld* World = GetWorld())
	{
		const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);
		if (NavSys && NavSys->GetRunMode() == FNavigationSystemRunMode::EditorWorldPartitionBuildMode)
		{
			return true;
		}
	}
	
	return !bFixedTilePoolSize;
}

float ARecastNavMesh::GetTileSizeUU() const
{
	const float DefaultCellSize = GetCellSize(ENavigationDataResolution::Default);
	const float RcTileSize = FMath::TruncToFloat(TileSizeUU / DefaultCellSize);
	return RcTileSize * DefaultCellSize;
}

void ARecastNavMesh::GetEdgesForPathCorridor(const TArray<NavNodeRef>* PathCorridor, TArray<FNavigationPortalEdge>* PathCorridorEdges) const
{
	check(PathCorridor != NULL && PathCorridorEdges != NULL);

	if (RecastNavMeshImpl)
	{
		RecastNavMeshImpl->GetEdgesForPathCorridor(PathCorridor, PathCorridorEdges);
	}
}

FNavLocation ARecastNavMesh::GetRandomPoint(FSharedConstNavQueryFilter Filter, const UObject* QueryOwner) const
{
	FNavLocation RandomPt;
	if (RecastNavMeshImpl)
	{
		RandomPt = RecastNavMeshImpl->GetRandomPoint(GetRightFilterRef(Filter), QueryOwner);
	}

	return RandomPt;
}

bool ARecastNavMesh::GetRandomReachablePointInRadius(const FVector& Origin, float Radius, FNavLocation& OutResult, FSharedConstNavQueryFilter Filter, const UObject* QueryOwner) const
{
	if (RecastNavMeshImpl == nullptr || RecastNavMeshImpl->DetourNavMesh == nullptr || Radius < 0.f)
	{
		return false;
	}

	const FNavigationQueryFilter& FilterInstance = GetRightFilterRef(Filter);

	FRecastSpeciaLinkFilter LinkFilter(FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld()), QueryOwner);
	INITIALIZE_NAVQUERY_WLINKFILTER(NavQuery, FilterInstance.GetMaxSearchNodes(), LinkFilter);

	// inits to "pass all"
	const dtQueryFilter* QueryFilter = (static_cast<const FRecastQueryFilter*>(FilterInstance.GetImplementation()))->GetAsDetourQueryFilter();
	ensure(QueryFilter);
	if (QueryFilter)
	{
		// find starting poly
		const FVector ProjectionExtent(NavDataConfig.DefaultQueryExtent.X, NavDataConfig.DefaultQueryExtent.Y, BIG_NUMBER);
		const FVector RcExtent = Unreal2RecastPoint(ProjectionExtent).GetAbs();
		// convert start/end pos to Recast coords
		const FVector RecastOrigin = Unreal2RecastPoint(Origin);
		NavNodeRef OriginPolyID = INVALID_NAVNODEREF;
		NavQuery.findNearestPoly(&RecastOrigin.X, &RcExtent.X, QueryFilter, &OriginPolyID, nullptr);

		if (OriginPolyID != INVALID_NAVNODEREF)
		{
			dtPolyRef Poly;
			FVector::FReal RandPt[3];
			dtStatus Status = NavQuery.findRandomPointAroundCircle(OriginPolyID, &RecastOrigin.X, Radius
				, QueryFilter, FMath::FRand, &Poly, RandPt);

			if (dtStatusSucceed(Status))
			{
				OutResult = FNavLocation(Recast2UnrealPoint(RandPt), Poly);
				return true;
			}
		}

		OutResult = FNavLocation(Origin, OriginPolyID);
	}

	return false;
}

bool ARecastNavMesh::GetRandomPointInNavigableRadius(const FVector& Origin, float Radius, FNavLocation& OutResult, FSharedConstNavQueryFilter Filter, const UObject* Querier) const
{
	const FVector ProjectionExtent(NavDataConfig.DefaultQueryExtent.X, NavDataConfig.DefaultQueryExtent.Y, BIG_NUMBER);
	OutResult = FNavLocation(FNavigationSystem::InvalidLocation);

	const FVector::FReal RandomAngle = 2.f * PI * FMath::FRand();
	const FVector::FReal U = FMath::FRand() + FMath::FRand();
	const FVector::FReal RandomRadius = Radius * (U > 1 ? 2.f - U : U);
	const FVector RandomOffset(FMath::Cos(RandomAngle) * RandomRadius, FMath::Sin(RandomAngle) * RandomRadius, 0);
	FVector RandomLocationInRadius = Origin + RandomOffset;

	// naive implementation 
	ProjectPoint(RandomLocationInRadius, OutResult, ProjectionExtent, Filter);

	// if failed get a list of all nav polys in the area and do it the hard way
	if (OutResult.HasNodeRef() == false && RecastNavMeshImpl)
	{
		const FVector::FReal RadiusSq = FMath::Square(Radius);
		TArray<FNavPoly> Polys;
		const FVector FallbackExtent(Radius, Radius, HALF_WORLD_MAX); //Using HALF_WORLD_MAX instead of BIG_NUMBER, else the box size will be NaN.
		const FVector BoxOrigin(Origin.X, Origin.Y, 0.f);
		const FBox Box(BoxOrigin - FallbackExtent, BoxOrigin + FallbackExtent);
		GetPolysInBox(Box, Polys, Filter, Querier);
	
		// @todo extremely naive implementation, barely random. To be improved
		while (Polys.Num() > 0)
		{
			const int32 RandomIndex = FMath::RandHelper(Polys.Num());
			const FNavPoly& Poly = Polys[RandomIndex];

			FVector PointOnPoly(0);
			if (RecastNavMeshImpl->GetClosestPointOnPoly(Poly.Ref, Origin, PointOnPoly)
				&& FVector::DistSquared(PointOnPoly, Origin) < RadiusSq)
			{
				OutResult = FNavLocation(PointOnPoly, Poly.Ref);
				break;
			}

			Polys.RemoveAtSwap(RandomIndex, EAllowShrinking::No);
		}
	}

	return OutResult.HasNodeRef() == true;
}

#if WITH_NAVMESH_CLUSTER_LINKS
bool ARecastNavMesh::GetRandomPointInCluster(NavNodeRef ClusterRef, FNavLocation& OutLocation) const
{
	return RecastNavMeshImpl && RecastNavMeshImpl->GetRandomPointInCluster(ClusterRef, OutLocation);
}

NavNodeRef ARecastNavMesh::GetClusterRef(NavNodeRef PolyRef) const
{
	NavNodeRef ClusterRef = 0;
	if (RecastNavMeshImpl)
	{
		ClusterRef = RecastNavMeshImpl->GetClusterRefFromPolyRef(PolyRef);
	}

	return ClusterRef;
}
#endif // WITH_NAVMESH_CLUSTER_LINKS

bool ARecastNavMesh::FindMoveAlongSurface(const FNavLocation& StartLocation, const FVector& TargetPosition, FNavLocation& OutLocation, FSharedConstNavQueryFilter Filter, const UObject* QueryOwner) const
{
	bool bSuccess = false;
	if (RecastNavMeshImpl)
	{
		bSuccess = RecastNavMeshImpl->FindMoveAlongSurface(StartLocation, TargetPosition, OutLocation, GetRightFilterRef(Filter), QueryOwner);
	}

	return bSuccess;
}

void ARecastNavMesh::BatchFindMoveAlongSurface(TArray<FNavigationFindMoveAlongSurfaceWork>& Workload, FSharedConstNavQueryFilter Filter, const UObject* Querier) const
{
	if (RecastNavMeshImpl)
	{
		const FNavigationQueryFilter& FilterToUse = GetRightFilterRef(Filter);
		for (FNavigationFindMoveAlongSurfaceWork& Work : Workload)
		{
			Work.bResult = RecastNavMeshImpl->FindMoveAlongSurface(Work.StartLocation, Work.DesiredEndLocation, Work.ResultNavLocation, FilterToUse, Querier);
		}
	}
}

bool ARecastNavMesh::FindOverlappingEdges(const FNavLocation& StartLocation, TConstArrayView<FVector> ConvexPolygon, TArray<FVector>& OutEdges, FSharedConstNavQueryFilter Filter, const UObject* Querier) const
{
	if (RecastNavMeshImpl == NULL || RecastNavMeshImpl->DetourNavMesh == NULL)
	{
		return false;
	}

	const FNavigationQueryFilter& FilterToUse = GetRightFilterRef(Filter);

	FRecastSpeciaLinkFilter LinkFilter(FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld()), Querier);
	INITIALIZE_NAVQUERY_WLINKFILTER(NavQuery, FilterToUse.GetMaxSearchNodes(), LinkFilter);
	const dtQueryFilter* QueryFilter = static_cast<const FRecastQueryFilter*>(FilterToUse.GetImplementation())->GetAsDetourQueryFilter();

	const int32 MaxWalls = 64;
	int32 NumWalls = 0;
	FVector::FReal WallSegments[MaxWalls * 3 * 2] = { 0 };
	dtPolyRef WallPolys[MaxWalls * 2] = { 0 };

	const int32 MaxNeis = 64;
	int32 NumNeis = 0;
	dtPolyRef NeiPolys[MaxNeis] = { 0 };

	const int32 MaxConvexPolygonPoints = 8;
	int32 NumConvexPolygonPoints = FMath::Min(ConvexPolygon.Num(), MaxConvexPolygonPoints);
	FVector::FReal RcConvexPolygon[MaxConvexPolygonPoints * 3] = { 0 };

	for (int32 i  = 0; i < NumConvexPolygonPoints; i++)
	{
		const FVector RcPoint = Unreal2RecastPoint(ConvexPolygon[i]);
		RcConvexPolygon[i*3+0] = RcPoint.X;
		RcConvexPolygon[i*3+1] = RcPoint.Y;
		RcConvexPolygon[i*3+2] = RcPoint.Z;
	}

	dtStatus Status = NavQuery.findWallsOverlappingShape(StartLocation.NodeRef, RcConvexPolygon, NumConvexPolygonPoints, QueryFilter,
		NeiPolys, &NumNeis, MaxNeis, WallSegments, WallPolys, &NumWalls, MaxWalls);

	if (dtStatusSucceed(Status))
	{
		OutEdges.Reset(NumWalls*2);
		for (int32 Idx = 0; Idx < NumWalls; Idx++)
		{
			OutEdges.Add(Recast2UnrealPoint(&WallSegments[Idx * 6]));
			OutEdges.Add(Recast2UnrealPoint(&WallSegments[Idx * 6 + 3]));
		}

		return true;
	}

	return false;
}

bool ARecastNavMesh::GetPathSegmentBoundaryEdges(const FNavigationPath& Path, const FNavPathPoint& StartPoint, const FNavPathPoint& EndPoint, const TConstArrayView<FVector> SearchArea, TArray<FVector>& OutEdges, const float MaxAreaEnterCost, FSharedConstNavQueryFilter Filter, const UObject* Querier) const
{
	if (RecastNavMeshImpl == NULL || RecastNavMeshImpl->DetourNavMesh == NULL)
	{
		return false;
	}

	const FNavMeshPath* NavMeshPath = Path.CastPath<const FNavMeshPath>();
	if (NavMeshPath == nullptr)
	{
		return false;
	}
	
	const FNavigationQueryFilter& FilterToUse = GetRightFilterRef(Filter);

	FRecastSpeciaLinkFilter LinkFilter(FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld()), Querier);
	INITIALIZE_NAVQUERY_WLINKFILTER(NavQuery, FilterToUse.GetMaxSearchNodes(), LinkFilter);
	const dtQueryFilter* QueryFilter = static_cast<const FRecastQueryFilter*>(FilterToUse.GetImplementation())->GetAsDetourQueryFilter();


	// Find all the polygon refs between the path points.
	TArray<NavNodeRef> SegmentPathPolys;
	if (NavMeshPath->PathCorridor.Num() > 0)
	{
		const int32 StartIndex = NavMeshPath->GetNodeRefIndex(StartPoint.NodeRef);
		if (StartIndex != INDEX_NONE)
		{
			const NavNodeRef NextNodeRef = EndPoint.NodeRef;
			for (int32 NodeIndex = StartIndex; NodeIndex < NavMeshPath->PathCorridor.Num(); NodeIndex++)
			{
				const NavNodeRef NodeRef = NavMeshPath->PathCorridor[NodeIndex];
				SegmentPathPolys.Add(NodeRef);
				if (NodeRef == NextNodeRef)
				{
					break;
				}
			}
		}
	}

	if (SegmentPathPolys.IsEmpty())
	{
		SegmentPathPolys.Add(StartPoint.NodeRef);
	}
	
	const int32 MaxWalls = 64;
	int32 NumWalls = 0;
	FVector::FReal WallSegments[MaxWalls * 3 * 2] = { 0 };
	dtPolyRef WallPolys[MaxWalls * 2] = { 0 };

	const int32 MaxNeis = 64;
	int32 NumNeis = 0;
	dtPolyRef NeiPolys[MaxNeis] = { 0 };

	const int32 MaxConvexPolygonPoints = 8;
	int32 NumConvexPolygonPoints = FMath::Min(SearchArea.Num(), MaxConvexPolygonPoints);
	FVector::FReal RcConvexPolygon[MaxConvexPolygonPoints * 3] = { 0 };

	for (int32 i  = 0; i < NumConvexPolygonPoints; i++)
	{
		const FVector RcPoint = Unreal2RecastPoint(SearchArea[i]);
		RcConvexPolygon[i*3+0] = RcPoint.X;
		RcConvexPolygon[i*3+1] = RcPoint.Y;
		RcConvexPolygon[i*3+2] = RcPoint.Z;
	}

	dtStatus Status = NavQuery.findWallsAroundPath(SegmentPathPolys.GetData(), SegmentPathPolys.Num(), RcConvexPolygon, NumConvexPolygonPoints, MaxAreaEnterCost, QueryFilter,
		NeiPolys, &NumNeis, MaxNeis, WallSegments, WallPolys, &NumWalls, MaxWalls);

	if (dtStatusSucceed(Status))
	{
		OutEdges.Reset(NumWalls*2);
		for (int32 Idx = 0; Idx < NumWalls; Idx++)
		{
			OutEdges.Add(Recast2UnrealPoint(&WallSegments[Idx * 6]));
			OutEdges.Add(Recast2UnrealPoint(&WallSegments[Idx * 6 + 3]));
		}
		return true;
	}

	return false;
}

bool ARecastNavMesh::ProjectPoint(const FVector& Point, FNavLocation& OutLocation, const FVector& Extent, FSharedConstNavQueryFilter Filter, const UObject* QueryOwner) const
{
	bool bSuccess = false;
	if (RecastNavMeshImpl)
	{
		bSuccess = RecastNavMeshImpl->ProjectPointToNavMesh(Point, OutLocation, Extent, GetRightFilterRef(Filter), QueryOwner);
	}

	return bSuccess;
}

bool ARecastNavMesh::IsNodeRefValid(NavNodeRef NodeRef) const
{
	if (NodeRef == INVALID_NAVNODEREF)
	{
		return false;
	}
	const dtNavMesh* NavMesh = RecastNavMeshImpl ? RecastNavMeshImpl->GetRecastMesh() : nullptr;
	if (!NavMesh)
	{
		return false;
	}
	dtPoly const* Poly = 0;
	dtMeshTile const* Tile = 0;
	const dtStatus Status = NavMesh->getTileAndPolyByRef(NodeRef, &Tile, &Poly);
	const bool bSuccess = dtStatusSucceed(Status);
	return bSuccess;
}

void ARecastNavMesh::BatchProjectPoints(TArray<FNavigationProjectionWork>& Workload, const FVector& Extent, FSharedConstNavQueryFilter Filter, const UObject* Querier) const 
{
	if (Workload.Num() == 0 || RecastNavMeshImpl == NULL || RecastNavMeshImpl->DetourNavMesh == NULL)
	{
		return;
	}
	
	const FNavigationQueryFilter& FilterToUse = GetRightFilterRef(Filter);

	FRecastSpeciaLinkFilter LinkFilter(FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld()), Querier);
	INITIALIZE_NAVQUERY_WLINKFILTER(NavQuery, FilterToUse.GetMaxSearchNodes(), LinkFilter);
	const dtQueryFilter* QueryFilter = static_cast<const FRecastQueryFilter*>(FilterToUse.GetImplementation())->GetAsDetourQueryFilter();
	
	if (ensure(QueryFilter))
	{
		const FVector ModifiedExtent = GetModifiedQueryExtent(Extent);
		FVector RcExtent = Unreal2RecastPoint(ModifiedExtent).GetAbs();
		FVector::FReal ClosestPoint[3];
		dtPolyRef PolyRef;

		for (int32 Idx = 0; Idx < Workload.Num(); Idx++)
		{
			FVector RcPoint = Unreal2RecastPoint(Workload[Idx].Point);
			if (Workload[Idx].bHintProjection2D)
			{
				NavQuery.findNearestPoly2D(&RcPoint.X, &RcExtent.X, QueryFilter, &PolyRef, ClosestPoint);
			}
			else
			{
				NavQuery.findNearestPoly(&RcPoint.X, &RcExtent.X, QueryFilter, &PolyRef, ClosestPoint);
			}

			// one last step required due to recast's BVTree imprecision
			if (PolyRef > 0)
			{
				const FVector& UnrealClosestPoint = Recast2UnrealPoint(ClosestPoint);
				if (FVector::DistSquared(UnrealClosestPoint, Workload[Idx].Point) <= ModifiedExtent.SizeSquared())
				{
					Workload[Idx].OutLocation = FNavLocation(UnrealClosestPoint, PolyRef);
					Workload[Idx].bResult = true;
				}
			}
		}
	}
}

void ARecastNavMesh::BatchProjectPoints(TArray<FNavigationProjectionWork>& Workload, FSharedConstNavQueryFilter Filter, const UObject* Querier) const
{
	if (Workload.Num() == 0 || RecastNavMeshImpl == NULL || RecastNavMeshImpl->DetourNavMesh == NULL)
	{
		return;
	}

	const FNavigationQueryFilter& FilterToUse = GetRightFilterRef(Filter);

	FRecastSpeciaLinkFilter LinkFilter(FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld()), Querier);
	INITIALIZE_NAVQUERY_WLINKFILTER(NavQuery, FilterToUse.GetMaxSearchNodes(), LinkFilter);
	const dtQueryFilter* QueryFilter = static_cast<const FRecastQueryFilter*>(FilterToUse.GetImplementation())->GetAsDetourQueryFilter();

	if (ensure(QueryFilter))
	{
		FVector::FReal ClosestPoint[3];
		dtPolyRef PolyRef;

		for (FNavigationProjectionWork& Work : Workload)
		{
			ensure(Work.ProjectionLimit.IsValid);
			const FVector RcReferencePoint = Unreal2RecastPoint(Work.Point);
			const FVector ModifiedExtent = GetModifiedQueryExtent(Work.ProjectionLimit.GetExtent());
			const FVector RcExtent = Unreal2RecastPoint(ModifiedExtent).GetAbs();
			const FVector RcBoxCenter = Unreal2RecastPoint(Work.ProjectionLimit.GetCenter());

			if (Work.bHintProjection2D)
			{
				NavQuery.findNearestPoly2D(&RcBoxCenter.X, &RcExtent.X, QueryFilter, &PolyRef, ClosestPoint, &RcReferencePoint.X);
			}
			else
			{
				NavQuery.findNearestPoly(&RcBoxCenter.X, &RcExtent.X, QueryFilter, &PolyRef, ClosestPoint, &RcReferencePoint.X);
			}

			// one last step required due to recast's BVTree imprecision
			if (PolyRef > 0)
			{
				const FVector& UnrealClosestPoint = Recast2UnrealPoint(ClosestPoint);
				if (FVector::DistSquared(UnrealClosestPoint, Work.Point) <= ModifiedExtent.SizeSquared())
				{
					Work.OutLocation = FNavLocation(UnrealClosestPoint, PolyRef);
					Work.bResult = true;
				}
			}
		}
	}
}

bool ARecastNavMesh::GetPolysInBox(const FBox& Box, TArray<FNavPoly>& Polys, FSharedConstNavQueryFilter Filter, const UObject* InOwner) const
{
	// sanity check
	const dtNavMesh* NavMesh = GetRecastMesh();
	if (NavMesh == nullptr)
	{
		return false;
	}

	bool bSuccess = false;

	const FNavigationQueryFilter& FilterToUse = GetRightFilterRef(Filter);
	FRecastSpeciaLinkFilter LinkFilter(FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld()), InOwner);
	INITIALIZE_NAVQUERY_WLINKFILTER(NavQuery, FilterToUse.GetMaxSearchNodes(), LinkFilter);

	const dtQueryFilter* QueryFilter = ((const FRecastQueryFilter*)(FilterToUse.GetImplementation()))->GetAsDetourQueryFilter();
	ensure(QueryFilter);
	if (QueryFilter)
	{
		const FVector ModifiedExtent = GetModifiedQueryExtent(Box.GetExtent());

		const FVector RcPoint = Unreal2RecastPoint( Box.GetCenter() );
		const FVector RcExtent = Unreal2RecastPoint( ModifiedExtent ).GetAbs();

		const int32 MaxHitPolys = 256;
		dtPolyRef HitPolys[MaxHitPolys];
		int32 NumHitPolys = 0;

		dtStatus status = NavQuery.queryPolygons(&RcPoint.X, &RcExtent.X, QueryFilter, HitPolys, &NumHitPolys, MaxHitPolys);
		if (dtStatusSucceed(status))
		{
			// only ground type polys
			int32 BaseIdx = Polys.Num();
			Polys.AddZeroed(NumHitPolys);

			for (int32 i = 0; i < NumHitPolys; i++)
			{
				dtPoly const* Poly;
				dtMeshTile const* Tile;
				dtStatus Status = NavMesh->getTileAndPolyByRef(HitPolys[i], &Tile, &Poly);
				if (dtStatusSucceed(Status))
				{
					FVector PolyCenter(0);
					for (int k = 0; k < Poly->vertCount; ++k)
					{
						FVector::FReal Vertex[3];
						Tile->verts[Poly->verts[k]].toWorld(Tile->header->bmin, Vertex);
						PolyCenter += Recast2UnrealPoint(Vertex);
					}
					PolyCenter /= Poly->vertCount;

					FNavPoly& OutPoly = Polys[BaseIdx + i];
					OutPoly.Ref = HitPolys[i];
					OutPoly.Center = PolyCenter;
				}
			}

			bSuccess = true;
		}
	}

	return bSuccess;
}

bool ARecastNavMesh::FindEdges(const NavNodeRef CenterNodeRef, const FVector Center, const FVector::FReal Radius, const FSharedConstNavQueryFilter Filter, TArray<FNavigationWallEdge>& OutEdges) const
{
	if (HasValidNavmesh() == false)
	{
		return false;
	}
	
	const FNavigationQueryFilter& FilterToUse = GetRightFilterRef(Filter);
	INITIALIZE_NAVQUERY(NavQuery, FilterToUse.GetMaxSearchNodes());
	const dtQueryFilter* QueryFilter = static_cast<const FRecastQueryFilter*>(FilterToUse.GetImplementation())->GetAsDetourQueryFilter();

	const int32 MaxWalls = 64;
	int32 NumWalls = 0;
	FVector::FReal WallSegments[MaxWalls * 3 * 2] = { 0 };
	dtPolyRef WallPolys[MaxWalls * 2] = { 0 };

	const int32 MaxNeis = 64;
	int32 NumNeis = 0;
	dtPolyRef NeiPolys[MaxNeis] = { 0 };

	const FVector RcCenter = Unreal2RecastPoint(Center);

	dtStatus Status = NavQuery.findWallsInNeighbourhood(CenterNodeRef, &RcCenter.X, Radius, QueryFilter,
		NeiPolys, &NumNeis, MaxNeis, WallSegments, WallPolys, &NumWalls, MaxWalls);

	if (dtStatusSucceed(Status))
	{
		OutEdges.Reset(NumWalls);
		FNavigationWallEdge NewEdge;
		for (int32 Idx = 0; Idx < NumWalls; Idx++)
		{
			NewEdge.Start = Recast2UnrealPoint(&WallSegments[Idx * 6]);
			NewEdge.End = Recast2UnrealPoint(&WallSegments[Idx * 6 + 3]);
			OutEdges.Add(NewEdge);
		}

		return true;
	}

	return false;
}

void ARecastNavMesh::GetTilePolyEdgesForFilter(FNavTileRef TileRef, FSharedConstNavQueryFilter Filter, TArray<FNavigationWallEdge>& OutEdges) const
{
	const dtNavMesh* const DetourNavMesh = GetRecastMesh();
	if (DetourNavMesh)
	{
		if (const dtMeshTile* const Tile = DetourNavMesh->getTileByRef(static_cast<dtTileRef>(TileRef)))
		{
			RecastNavMeshImpl->GetTilePolyEdgesForFilter(*Tile, GetRightFilterRef(Filter), OutEdges);
		}
	}
}

bool ARecastNavMesh::GetEdgesInTile(FNavTileRef TileRef, TArray<FNavigationWallEdge>& OutEdges) const
{
	if (HasValidNavmesh())
	{
		const dtNavMesh* const DetourNavMesh = RecastNavMeshImpl->GetRecastMesh();
		if (const dtMeshTile* const Tile = DetourNavMesh->getTileByRef(static_cast<dtTileRef>(TileRef)))
		{
			const bool bGatherInternalEdges = false;
			TArray<FVector> UNUSED_InternalEdgeVerts;

			const bool bGatherExternalEdges = true;
			TArray<FVector> ExternalEdgeVerts;
			// Guessing that each detail tri in the tile has one external edge
			ExternalEdgeVerts.Reserve(Tile->header->detailTriCount);

			RecastNavMeshImpl->GetTilePolyEdges(*Tile, bGatherInternalEdges, bGatherExternalEdges, UNUSED_InternalEdgeVerts, ExternalEdgeVerts);

			const int32 ExternalEdgeVertCount = ExternalEdgeVerts.Num();
			if (ExternalEdgeVertCount > 0)
			{
				OutEdges.Reset(ExternalEdgeVertCount / 2);
				for (int32 ExternalEdgeVertIndex = 0; ExternalEdgeVertIndex < ExternalEdgeVertCount; ExternalEdgeVertIndex += 2)
				{
					FNavigationWallEdge& Edge = OutEdges.AddDefaulted_GetRef();
					Edge.Start = ExternalEdgeVerts[ExternalEdgeVertIndex];
					Edge.End = ExternalEdgeVerts[ExternalEdgeVertIndex + 1];
				}
			}
		}
	}

	return false;
}

bool ARecastNavMesh::ProjectPointMulti(const FVector& Point, TArray<FNavLocation>& OutLocations, const FVector& Extent,
	FVector::FReal MinZ, FVector::FReal MaxZ, FSharedConstNavQueryFilter Filter, const UObject* QueryOwner) const
{
	return RecastNavMeshImpl && RecastNavMeshImpl->ProjectPointMulti(Point, OutLocations, Extent, MinZ, MaxZ, GetRightFilterRef(Filter), QueryOwner);
}

ENavigationQueryResult::Type ARecastNavMesh::CalcPathCost(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& OutPathCost, FSharedConstNavQueryFilter QueryFilter, const UObject* QueryOwner) const
{
	FVector::FReal TmpPathLength = 0.f;
	ENavigationQueryResult::Type Result = CalcPathLengthAndCost(PathStart, PathEnd, TmpPathLength, OutPathCost, QueryFilter, QueryOwner);
	return Result;
}

ENavigationQueryResult::Type ARecastNavMesh::CalcPathLength(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& OutPathLength, FSharedConstNavQueryFilter QueryFilter, const UObject* QueryOwner) const
{
	FVector::FReal TmpPathCost = 0.f;
	ENavigationQueryResult::Type Result = CalcPathLengthAndCost(PathStart, PathEnd, OutPathLength, TmpPathCost, QueryFilter, QueryOwner);
	return Result;
}

ENavigationQueryResult::Type ARecastNavMesh::CalcPathLengthAndCost(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& OutPathLength, FVector::FReal& OutPathCost, FSharedConstNavQueryFilter QueryFilter, const UObject* QueryOwner) const
{
	ENavigationQueryResult::Type Result = ENavigationQueryResult::Invalid;

	if (RecastNavMeshImpl)
	{
		if ((PathStart - PathEnd).IsNearlyZero() == true)
		{
			OutPathLength = 0.;
			Result = ENavigationQueryResult::Success;
		}
		else
		{
			TSharedRef<FNavMeshPath> Path = MakeShareable(new FNavMeshPath());
			Path->SetWantsStringPulling(false);
			Path->SetWantsPathCorridor(true);
			
			// LWC_TODO_AI: CostLimit should be FVector::FReal. Not until after 5.0!
			constexpr FVector::FReal CostLimit = TNumericLimits<FVector::FReal>::Max();
			constexpr bool bRequireNavigableEndLocation = true;
			Result = RecastNavMeshImpl->FindPath(PathStart, PathEnd, CostLimit, bRequireNavigableEndLocation, Path.Get(), GetRightFilterRef(QueryFilter), QueryOwner);

			if (Result == ENavigationQueryResult::Success || (Result == ENavigationQueryResult::Fail && Path->IsPartial()))
			{
				OutPathLength = Path->GetTotalPathLength();
				OutPathCost = Path->GetCost();
			}
		}
	}

	return Result;
}

bool ARecastNavMesh::DoesNodeContainLocation(NavNodeRef NodeRef, const FVector& WorldSpaceLocation) const
{
	bool bResult = false;
	if (RecastNavMeshImpl != nullptr && RecastNavMeshImpl->GetRecastMesh() != nullptr)
	{
		dtNavMeshQuery NavQuery;
		NavQuery.init(RecastNavMeshImpl->GetRecastMesh(), 0);

		const FVector RcLocation = Unreal2RecastPoint(WorldSpaceLocation);
		if (dtStatusFailed(NavQuery.isPointInsidePoly(NodeRef, &RcLocation.X, bResult)))
		{
			bResult = false;
		}
	}

	return bResult; 
}

bool ARecastNavMesh::FindPolysAroundCircle(const FVector& CenterPos, const NavNodeRef CenterNodeRef, const FVector::FReal Radius, const FSharedConstNavQueryFilter& Filter, const UObject* QueryOwner, TArray<NavNodeRef>* OutPolys, TArray<NavNodeRef>* OutPolysParent, TArray<float>* OutPolysCost, int32* OutPolysCount) const
{
	return RecastNavMeshImpl ? RecastNavMeshImpl->FindPolysAroundCircle(CenterPos, CenterNodeRef, Radius, GetRightFilterRef(Filter), QueryOwner, OutPolys, OutPolysParent, OutPolysCost, OutPolysCount) : false;
}

NavNodeRef ARecastNavMesh::FindNearestPoly(FVector const& Loc, FVector const& Extent, FSharedConstNavQueryFilter Filter, const UObject* QueryOwner) const
{
	NavNodeRef PolyRef = 0;
	if (RecastNavMeshImpl)
	{
		PolyRef = RecastNavMeshImpl->FindNearestPoly(Loc, Extent, GetRightFilterRef(Filter), QueryOwner);
	}

	return PolyRef;
}

FVector::FReal ARecastNavMesh::FindDistanceToWall(const FVector& StartLoc, FSharedConstNavQueryFilter Filter, FVector::FReal MaxDistance, FVector* OutClosestPointOnWall) const
{
	if (HasValidNavmesh() == false)
	{
		return 0.;
	}

	const FNavigationQueryFilter& FilterToUse = GetRightFilterRef(Filter);

	INITIALIZE_NAVQUERY(NavQuery, FilterToUse.GetMaxSearchNodes());
	const dtQueryFilter* QueryFilter = ((const FRecastQueryFilter*)(FilterToUse.GetImplementation()))->GetAsDetourQueryFilter();

	if (QueryFilter == nullptr)
	{
		UE_VLOG(this, LogNavigation, Warning, TEXT("ARecastNavMesh::FindDistanceToWall failing due to QueryFilter == NULL"));
		return 0.;
	}

	const FVector NavExtent = GetModifiedQueryExtent(GetDefaultQueryExtent());
	const FVector::FReal Extent[3] = { NavExtent.X, NavExtent.Z, NavExtent.Y };

	const FVector RecastStart = Unreal2RecastPoint(StartLoc);

	NavNodeRef StartNode = INVALID_NAVNODEREF;
	NavQuery.findNearestPoly(&RecastStart.X, Extent, QueryFilter, &StartNode, NULL);

	if (StartNode != INVALID_NAVNODEREF)
	{
		FVector::FReal TmpHitPos[3], TmpHitNormal[3];
		FVector::FReal DistanceToWall = 0.f;
		const dtStatus RaycastStatus = NavQuery.findDistanceToWall(StartNode, &RecastStart.X, MaxDistance, QueryFilter
			, &DistanceToWall, TmpHitPos, TmpHitNormal);

		if (dtStatusSucceed(RaycastStatus))
		{
			if (OutClosestPointOnWall)
			{
				*OutClosestPointOnWall = Recast2UnrealPoint(TmpHitPos);
			}
			return DistanceToWall;
		}
	}

	return 0.;
}

void ARecastNavMesh::UpdateCustomLink(const INavLinkCustomInterface* CustomLink)
{
	TSubclassOf<UNavArea> AreaClass = CustomLink->GetLinkAreaClass();
	const FNavLinkId UserId = CustomLink->GetId();
	const int32 AreaId = GetAreaID(AreaClass);
	if (AreaId >= 0 && RecastNavMeshImpl)
	{
		UNavArea* DefArea = (UNavArea*)(AreaClass->GetDefaultObject());
		const uint16 PolyFlags = DefArea->GetAreaFlags() | ARecastNavMesh::GetNavLinkFlag();

		RecastNavMeshImpl->UpdateNavigationLinkArea(UserId, IntCastChecked<uint8>(AreaId), PolyFlags);
#if WITH_NAVMESH_SEGMENT_LINKS
		// SegmentLinks are an unsupported feature that was never completed to production quality, for now at least FNavLinkId ids are not supported here.
		RecastNavMeshImpl->UpdateSegmentLinkArea((int32)UserId.GetId(), IntCastChecked<uint8>(AreaId), PolyFlags);
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if !UE_BUILD_SHIPPING
		RequestDrawingUpdate(false);
#endif
	}
}

void ARecastNavMesh::UpdateNavigationLinkArea(FNavLinkId UserId, TSubclassOf<UNavArea> AreaClass) const
{
	int32 AreaId = GetAreaID(AreaClass);
	if (AreaId >= 0 && RecastNavMeshImpl)
	{
		UNavArea* DefArea = (UNavArea*)(AreaClass->GetDefaultObject());
		const uint16 PolyFlags = DefArea->GetAreaFlags() | ARecastNavMesh::GetNavLinkFlag();

		RecastNavMeshImpl->UpdateNavigationLinkArea(UserId, IntCastChecked<uint8>(AreaId), PolyFlags);
	}
}

#if WITH_NAVMESH_SEGMENT_LINKS
void ARecastNavMesh::UpdateSegmentLinkArea(int32 UserId, TSubclassOf<UNavArea> AreaClass) const
{
	int32 AreaId = GetAreaID(AreaClass);
	if (AreaId >= 0 && RecastNavMeshImpl)
	{
		UNavArea* DefArea = (UNavArea*)(AreaClass->GetDefaultObject());
		const uint16 PolyFlags = DefArea->GetAreaFlags() | ARecastNavMesh::GetNavLinkFlag();

		RecastNavMeshImpl->UpdateSegmentLinkArea(UserId, IntCastChecked<uint8>(AreaId), PolyFlags);
	}
}
#endif // WITH_NAVMESH_SEGMENT_LINKS

bool ARecastNavMesh::GetPolyCenter(NavNodeRef PolyID, FVector& OutCenter) const
{
	return RecastNavMeshImpl && RecastNavMeshImpl->GetPolyCenter(PolyID, OutCenter);
}

bool ARecastNavMesh::GetPolyVerts(NavNodeRef PolyID, TArray<FVector>& OutVerts) const
{
	return RecastNavMeshImpl && RecastNavMeshImpl->GetPolyVerts(PolyID, OutVerts);
}

bool ARecastNavMesh::GetRandomPointInPoly(NavNodeRef PolyID, FVector& OutPoint) const
{
	return RecastNavMeshImpl && RecastNavMeshImpl->GetRandomPointInPoly(PolyID, OutPoint);
}

FVector::FReal ARecastNavMesh::GetPolySurfaceArea(NavNodeRef PolyID) const
{
	return RecastNavMeshImpl ? RecastNavMeshImpl->GetPolySurfaceArea(PolyID) : 0;
}

uint32 ARecastNavMesh::GetPolyAreaID(NavNodeRef PolyID) const
{
	uint32 AreaID = RECAST_DEFAULT_AREA;
	if (RecastNavMeshImpl)
	{
		AreaID = RecastNavMeshImpl->GetPolyAreaID(PolyID);
	}

	return AreaID;
}

bool ARecastNavMesh::SetPolyArea(NavNodeRef PolyID, TSubclassOf<UNavArea> AreaClass)
{
	bool bSuccess = false;
	if (AreaClass && RecastNavMeshImpl)
	{
		dtNavMesh* NavMesh = RecastNavMeshImpl->GetRecastMesh();
		const int32 AreaId = GetAreaID(AreaClass);
		const uint16 AreaFlags = AreaClass->GetDefaultObject<UNavArea>()->GetAreaFlags();
		
		if (AreaId != INDEX_NONE && NavMesh)
		{
			// @todo implement a single detour function that would do both
			bSuccess = dtStatusSucceed(NavMesh->setPolyArea(PolyID, IntCastChecked<unsigned char>(AreaId)));
			bSuccess = (bSuccess && dtStatusSucceed(NavMesh->setPolyFlags(PolyID, AreaFlags)));
		}
	}
	return bSuccess;
}

void ARecastNavMesh::SetPolyArrayArea(const TArray<FNavPoly>& Polys, TSubclassOf<UNavArea> AreaClass)
{
	if (AreaClass && RecastNavMeshImpl)
	{
		dtNavMesh* NavMesh = RecastNavMeshImpl->GetRecastMesh();
		const int32 AreaId = GetAreaID(AreaClass);
		const uint16 AreaFlags = AreaClass->GetDefaultObject<UNavArea>()->GetAreaFlags();

		if (AreaId != INDEX_NONE && NavMesh)
		{
			for (int32 Idx = 0; Idx < Polys.Num(); Idx++)
			{
				NavMesh->setPolyArea(Polys[Idx].Ref, IntCastChecked<unsigned char>(AreaId));
				NavMesh->setPolyFlags(Polys[Idx].Ref, AreaFlags);
			}
		}
	}
}

int32 ARecastNavMesh::ReplaceAreaInTileBounds(const FBox& Bounds, TSubclassOf<UNavArea> OldArea, TSubclassOf<UNavArea> NewArea, bool ReplaceLinks, TArray<NavNodeRef>* OutTouchedNodes)
{
	if (!ensureMsgf(Bounds.IsValid, TEXT("%hs Bounds is not valid"), __FUNCTION__))
	{
		return 0;
	}

	int32 PolysTouched = 0;

	if (RecastNavMeshImpl && RecastNavMeshImpl->GetRecastMesh())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMesh_ReplaceAreaInTiles);

		const int32 OldAreaID = GetAreaID(OldArea);
		ensure(OldAreaID != INDEX_NONE);
		const int32 NewAreaID = GetAreaID(NewArea);
		ensure(NewAreaID != INDEX_NONE);
		ensure(NewAreaID != OldAreaID);

		// workaround for privacy issue in the recast API
		const dtNavMesh* DetourNavMesh = RecastNavMeshImpl->GetRecastMesh();
		dtNavMesh const* const ConstDetourNavMesh = RecastNavMeshImpl->GetRecastMesh();

		const FVector RcNavMeshOrigin = Unreal2RecastPoint(NavMeshOriginOffset);
		const float TileSizeInWorldUnits = GetTileSizeUU();
		const FRcTileBox TileBox(Bounds, RcNavMeshOrigin, TileSizeInWorldUnits);

		for (int32 TileY = TileBox.YMin; TileY <= TileBox.YMax; ++TileY)
		{
			for (int32 TileX = TileBox.XMin; TileX <= TileBox.XMax; ++TileX)
			{
				const int32 MaxTiles = ConstDetourNavMesh->getTileCountAt(TileX, TileY);
				if (MaxTiles == 0)
				{
					continue;
				}

				TArray<const dtMeshTile*> Tiles;
				Tiles.AddZeroed(MaxTiles);
				const int32 NumTiles = ConstDetourNavMesh->getTilesAt(TileX, TileY, Tiles.GetData(), MaxTiles);
				for (int32 i = 0; i < NumTiles; i++)
				{
					dtTileRef TileRef = ConstDetourNavMesh->getTileRef(Tiles[i]);
					if (TileRef)
					{
						const int32 TileIndex = (int32)ConstDetourNavMesh->decodePolyIdTile(TileRef);
						const dtMeshTile* Tile = DetourNavMesh->getTile(TileIndex);
						//const int32 MaxPolys = Tile && Tile->header ? Tile->header->offMeshBase : 0;
						const int32 MaxPolys = Tile && Tile->header
							? (ReplaceLinks ? Tile->header->polyCount : Tile->header->offMeshBase)
							: 0;
						if (MaxPolys > 0)
						{
							dtPoly* Poly = Tile->polys;
							for (int32 PolyIndex = 0; PolyIndex < MaxPolys; PolyIndex++, Poly++)
							{
								if (Poly->getArea() == OldAreaID)
								{
									Poly->setArea(IntCastChecked<unsigned char>(NewAreaID));
									++PolysTouched;
								}
							}
						}
					}
				}
			}
		}
	}
	return PolysTouched;
}

bool ARecastNavMesh::GetPolyFlags(NavNodeRef PolyID, uint16& PolyFlags, uint16& AreaFlags) const
{
	bool bFound = false;
	if (RecastNavMeshImpl)
	{
		uint8 AreaType = RECAST_DEFAULT_AREA;
		bFound = RecastNavMeshImpl->GetPolyData(PolyID, PolyFlags, AreaType);
		if (bFound)
		{
			const UClass* AreaClass = GetAreaClass(AreaType);
			const UNavArea* DefArea = AreaClass ? ((UClass*)AreaClass)->GetDefaultObject<UNavArea>() : NULL;
			AreaFlags = DefArea ? DefArea->GetAreaFlags() : 0;
		}
	}

	return bFound;
}

bool ARecastNavMesh::GetPolyFlags(NavNodeRef PolyID, FNavMeshNodeFlags& Flags) const
{
	bool bFound = false;
	if (RecastNavMeshImpl)
	{
		uint16 PolyFlags = 0;

		bFound = RecastNavMeshImpl->GetPolyData(PolyID, PolyFlags, Flags.Area);
		if (bFound)
		{
			const UClass* AreaClass = GetAreaClass(Flags.Area);
			const UNavArea* DefArea = AreaClass ? ((UClass*)AreaClass)->GetDefaultObject<UNavArea>() : NULL;
			Flags.AreaFlags = DefArea ? DefArea->GetAreaFlags() : 0;
			// @todo what is this literal?
			Flags.PathFlags = (PolyFlags & GetNavLinkFlag()) ? 4 : 0;
		}
	}

	return bFound;
}

bool ARecastNavMesh::GetPolyNeighbors(NavNodeRef PolyID, TArray<FNavigationPortalEdge>& Neighbors) const
{
	return RecastNavMeshImpl && RecastNavMeshImpl->GetPolyNeighbors(PolyID, Neighbors);
}

bool ARecastNavMesh::GetPolyNeighbors(NavNodeRef PolyID, TArray<NavNodeRef>& Neighbors) const
{
	return RecastNavMeshImpl && RecastNavMeshImpl->GetPolyNeighbors(PolyID, Neighbors);
}

bool ARecastNavMesh::GetPolyEdges(NavNodeRef PolyID, TArray<FNavigationPortalEdge>& Neighbors) const
{
	bool bFound = false;
	if (RecastNavMeshImpl)
	{
		bFound = RecastNavMeshImpl->GetPolyEdges(PolyID, Neighbors);
	}

	return bFound;
}

bool ARecastNavMesh::GetPolyWallSegments(NavNodeRef PolyID, FSharedConstNavQueryFilter Filter, const UObject* QueryOwner, TArray<FNavigationPortalEdge>& OutNeighbors) const
{
	bool bFound = false;
	if (RecastNavMeshImpl)
	{
		bFound = RecastNavMeshImpl->GetPolyWallSegments(PolyID, GetRightFilterRef(Filter), QueryOwner, OutNeighbors);
	}

	return bFound;
}

bool ARecastNavMesh::GetClosestPointOnPoly(NavNodeRef PolyID, const FVector& TestPt, FVector& PointOnPoly) const
{
	return RecastNavMeshImpl && RecastNavMeshImpl->GetClosestPointOnPoly(PolyID, TestPt, PointOnPoly);
}

bool ARecastNavMesh::GetPolyTileIndex(NavNodeRef PolyID, uint32& PolyIndex, uint32& TileIndex) const
{
	return RecastNavMeshImpl && RecastNavMeshImpl->GetPolyTileIndex(PolyID, PolyIndex, TileIndex);
}

bool ARecastNavMesh::GetLinkEndPoints(NavNodeRef LinkPolyID, FVector& PointA, FVector& PointB) const
{
	return RecastNavMeshImpl && RecastNavMeshImpl->GetLinkEndPoints(LinkPolyID, PointA, PointB);
}

bool ARecastNavMesh::IsCustomLink(NavNodeRef LinkPolyID) const
{
	return RecastNavMeshImpl && RecastNavMeshImpl->IsCustomLink(LinkPolyID);
}

#if WITH_NAVMESH_CLUSTER_LINKS
bool ARecastNavMesh::GetClusterBounds(NavNodeRef ClusterRef, FBox& OutBounds) const
{
	return RecastNavMeshImpl && RecastNavMeshImpl->GetClusterBounds(ClusterRef, OutBounds);
}
#endif // WITH_NAVMESH_CLUSTER_LINKS

bool ARecastNavMesh::GetPolysWithinPathingDistance(FVector const& StartLoc, const FVector::FReal PathingDistance, TArray<NavNodeRef>& FoundPolys,
	FSharedConstNavQueryFilter Filter, const UObject* QueryOwner, FRecastDebugPathfindingData* DebugData) const
{
	return RecastNavMeshImpl && RecastNavMeshImpl->GetPolysWithinPathingDistance(StartLoc, PathingDistance, GetRightFilterRef(Filter), QueryOwner, FoundPolys, DebugData);
}

// Deprecated
bool ARecastNavMesh::GetDebugGeometryForTile(FRecastDebugGeometry& OutGeometry, int32 TileIndex) const
{
	if (RecastNavMeshImpl)
	{
		FNavTileRef TileRef;
		if (TileIndex != INDEX_NONE)
		{
			TArray<FNavTileRef> TileRefs;
			FNavTileRef::DeprecatedMakeTileRefsFromTileIds(RecastNavMeshImpl, { static_cast<uint32>(TileIndex) }, TileRefs);
			TileRef = TileRefs[0];
		}
		return GetDebugGeometryForTile(OutGeometry, TileRef);
	}
	return true;
}

bool ARecastNavMesh::GetDebugGeometryForTile(FRecastDebugGeometry& OutGeometry, FNavTileRef TileRef) const
{
	if (RecastNavMeshImpl)
	{
		return RecastNavMeshImpl->GetDebugGeometryForTile(OutGeometry, TileRef);
	}
	return true;
}

void ARecastNavMesh::RequestDrawingUpdate(bool bForce)
{
#if !UE_BUILD_SHIPPING
	if (bForce || UNavMeshRenderingComponent::IsNavigationShowFlagSet(GetWorld()))
	{
		if (bForce)
		{
			UNavMeshRenderingComponent* NavRenderingComp = Cast<UNavMeshRenderingComponent>(RenderingComp);
			if (NavRenderingComp)
			{
				NavRenderingComp->ForceUpdate();
			}
		}

		DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.Requesting navmesh redraw"),
		STAT_FSimpleDelegateGraphTask_RequestingNavmeshRedraw,
			STATGROUP_TaskGraphTasks);

		FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
			FSimpleDelegateGraphTask::FDelegate::CreateUObject(this, &ARecastNavMesh::UpdateDrawing),
			GET_STATID(STAT_FSimpleDelegateGraphTask_RequestingNavmeshRedraw), NULL, ENamedThreads::GameThread);
	}
#endif // !UE_BUILD_SHIPPING
}

void ARecastNavMesh::UpdateDrawing()
{
	UpdateNavMeshDrawing();
}

void ARecastNavMesh::DrawDebugPathCorridor(NavNodeRef const* PathPolys, int32 NumPathPolys, bool bPersistent) const
{
#if ENABLE_DRAW_DEBUG
	static const FColor PathLineColor(255, 128, 0);
	UWorld* World = GetWorld();

	// draw poly outlines
	TArray<FVector> PolyVerts;
	for (int32 PolyIdx=0; PolyIdx < NumPathPolys; ++PolyIdx)
	{
		if ( GetPolyVerts(PathPolys[PolyIdx], PolyVerts) )
		{
			for (int32 VertIdx=0; VertIdx < PolyVerts.Num()-1; ++VertIdx)
			{
				DrawDebugLine(World, PolyVerts[VertIdx], PolyVerts[VertIdx+1], PathLineColor, bPersistent);
			}
			DrawDebugLine(World, PolyVerts[PolyVerts.Num()-1], PolyVerts[0], PathLineColor, bPersistent);
		}
	}

	// draw ordered poly links
	if (NumPathPolys > 0)
	{
		FVector PolyCenter;
		FVector NextPolyCenter;
		if ( GetPolyCenter(PathPolys[0], NextPolyCenter) )			// prime the pump
		{
			for (int32 PolyIdx=0; PolyIdx < NumPathPolys-1; ++PolyIdx)
			{
				PolyCenter = NextPolyCenter;
				if ( GetPolyCenter(PathPolys[PolyIdx+1], NextPolyCenter) )
				{
					DrawDebugLine(World, PolyCenter, NextPolyCenter, PathLineColor, bPersistent);
					DrawDebugBox(World, PolyCenter, FVector(5.f), PathLineColor, bPersistent);
				}
			}
		}
	}
#endif // ENABLE_DRAW_DEBUG
}

void ARecastNavMesh::OnNavMeshTilesUpdated(const TArray<FNavTileRef>& ChangedTiles)
{
	InvalidateAffectedPaths(ChangedTiles);
}

#if WITH_NAVMESH_SEGMENT_LINKS
void ARecastNavMesh::CreateSegmentLinkConnections(TSet<FNavTileRef>& TilesPendingSegmentLinks)
{
	if(!RecastNavMeshImpl || !RecastNavMeshImpl->DetourNavMesh)
	{
		return;
	}
	
	TSet<FNavTileRef> ProcessedTiles;
	ProcessedTiles.Reserve(TilesPendingSegmentLinks.Num());

	// Segment link connections are created after all the navmesh tiles have been generated.
	// This means care needs to be taken to ensure links don't get created twice. For example,
	// linking together TileA and TileB but also TileB and TileA in the opposite order.
	// Therefore, sort the list of tiles needing link connections.
	// Tiles can only link to other tiles with a greater TileIndex to ensure the segment link connections are not duplicated.
	TArray<FNavTileRef> TilesPendingSegmentLinksArray = TilesPendingSegmentLinks.Array();

	auto TileSorter = [this](const FNavTileRef A, const FNavTileRef B)
	{
		const dtMeshTile* TileA = RecastNavMeshImpl->DetourNavMesh->getTileByRef(static_cast<dtTileRef>(A));
		const dtMeshTile* TileB = RecastNavMeshImpl->DetourNavMesh->getTileByRef(static_cast<dtTileRef>(B));
		return	TileA && TileA->header &&
				TileB && TileB->header &&
				(TileA->header->x < TileB->header->x ||
				TileA->header->y < TileB->header->y ||
				TileA->header->layer < TileB->header->layer);	
	};
	TilesPendingSegmentLinksArray.Sort(TileSorter);

	for (FNavTileRef TileRef : TilesPendingSegmentLinksArray)
	{
		if (ProcessedTiles.Contains(TileRef))
		{
			continue;
		}

		// We need to know what tiles were skipped due to those tiles having a lower index than the tile
		// being processed. These tiles may need to be processed if they were not in the original list
		// of tiles pending segment links.
		// 
		// All 8 tile neighbors could potentially have a lower tile index than the source tile and will be skipped.
		// The skipped neighbors array is pre-allocated for use with the Detour library.
		constexpr uint32 MaxNeighborsToBeSkipped = 8;
		const uint32 MaxSkippedNeighborTiles = MaxNeighborsToBeSkipped * ExpectedMaxLayersPerTile;

		TArray<dtTileRef> SkippedNeighborTiles;
		SkippedNeighborTiles.AddUninitialized(MaxSkippedNeighborTiles);

		// Process the tile and store the neighbor tiles that were skipped. They will need to be processed if they haven't been already.
		uint32 NumSkippedNeighborTiles = 0;
		UE_LOGF(LogSegmentLink, Log, "1) %s and store skipped tiles", __FUNCTION__);
		RecastNavMeshImpl->ProcessSegmentLinksForTile(TileRef, MaxSkippedNeighborTiles, SkippedNeighborTiles.GetData(), NumSkippedNeighborTiles);
		ProcessedTiles.Add(TileRef);

		if (NumSkippedNeighborTiles > MaxSkippedNeighborTiles)
		{
			UE_LOGF(LogNavigation, Warning, "The number of neighbor tiles skipped (%d) while connecting segment links exceeds the max allowed (%d). Consider increasing ExpectedMaxLayersPerTile in the NavMesh settings.", NumSkippedNeighborTiles, MaxSkippedNeighborTiles);

			NumSkippedNeighborTiles = MaxSkippedNeighborTiles;
		}

		// Process skipped tiles if they have not been already to get full link coverage.
		for (uint32 Index = 0; Index < NumSkippedNeighborTiles; ++Index)
		{
			const FNavTileRef NeighborTileRef = FNavTileRef(SkippedNeighborTiles[Index]);
			if (!ProcessedTiles.Contains(NeighborTileRef))
			{
				// No need to worry about skipped neighbors here. This tile was not in the original update
				// list and only needs to be processed to connect the links in the previously processed tile.
				UE_LOGF(LogSegmentLink, Log, "2) %s process skipped tileRef %llu", __FUNCTION__, uint64(NeighborTileRef));
				RecastNavMeshImpl->ProcessSegmentLinksForTile(NeighborTileRef);

				ProcessedTiles.Add(NeighborTileRef);
			}
		}
	}
}
#endif // WITH_NAVMESH_SEGMENT_LINKS

void ARecastNavMesh::InvalidateAffectedPaths(const TArray<FNavTileRef>& ChangedTiles)
{
	const int32 PathsCount = ActivePaths.Num();
	const int32 ChangedTilesCount = ChangedTiles.Num();
	
	if (ChangedTilesCount == 0 || PathsCount == 0)
	{
		return;
	}
	
	const dtNavMesh* DetourMesh = RecastNavMeshImpl->DetourNavMesh;
	if (DetourMesh == nullptr)
	{
		return;
	}

	// Paths can be registered from async pathfinding thread.
	// Theoretically paths are invalidated synchronously by the navigation system 
	// before starting async queries task but protecting ActivePaths will make
	// the system safer in case of future timing changes.
	{
		UE::TScopeLock PathLock(ActivePathsLock);

		for (int32 PathIndex = PathsCount - 1; PathIndex >= 0; --PathIndex)
		{
			FNavPathWeakPtr* WeakPathPtr = &ActivePaths[PathIndex];
			FNavPathSharedPtr SharedPath = WeakPathPtr->Pin();
			if (WeakPathPtr->IsValid() == false)
			{
				ActivePaths.RemoveAtSwap(PathIndex, EAllowShrinking::No);
			}
			else
			{
				const FNavigationPath* NavPath = SharedPath.Get();
				const FNavMeshPath* Path = NavPath ? NavPath->CastPath<FNavMeshPath>() : nullptr;

				if (Path == nullptr ||
					Path->IsReady() == false ||
					Path->GetIgnoreInvalidation() == true)
				{
					// path not filled yet or doesn't care about invalidation
					continue;
				}

				const int32 PathLength = Path->PathCorridor.Num();
				const NavNodeRef* PathPoly = Path->PathCorridor.GetData();
				for (int32 NodeIndex = 0; NodeIndex < PathLength; ++NodeIndex, ++PathPoly)
				{
					const FNavTileRef NavTileRef = UE::NavMesh::Private::GetTileRefFromPolyRef(*DetourMesh, *PathPoly);
					if (ChangedTiles.Contains(NavTileRef))
					{
						SharedPath->Invalidate();
						ActivePaths.RemoveAtSwap(PathIndex, EAllowShrinking::No);
						break;
					}
				}
			}
		}
	}
}

URecastNavMeshDataChunk* ARecastNavMesh::GetNavigationDataChunk(ULevel* InLevel) const
{
	return GetNavigationDataChunk(InLevel->NavDataChunks);
}

URecastNavMeshDataChunk* ARecastNavMesh::GetNavigationDataChunk(const ANavigationDataChunkActor& InActor) const
{
	return GetNavigationDataChunk(InActor.GetNavDataChunk());
}

URecastNavMeshDataChunk* ARecastNavMesh::GetNavigationDataChunk(const TArray<UNavigationDataChunk*>& InChunks) const
{
	FName ThisName = GetFName();
	int32 ChunkIndex = InChunks.IndexOfByPredicate([&](UNavigationDataChunk* Chunk)
	{
		return Chunk->NavigationDataName == ThisName;
	});
	
	URecastNavMeshDataChunk* RcNavDataChunk = nullptr;
	if (ChunkIndex != INDEX_NONE)
	{
		RcNavDataChunk = Cast<URecastNavMeshDataChunk>(InChunks[ChunkIndex]);
	}
		
	return RcNavDataChunk;
}

void ARecastNavMesh::EnsureBuildCompletion()
{
	Super::EnsureBuildCompletion();

	// Doing this as a safety net solution due to UE-20646, which was basically a result of random 
	// over-releasing of default filter's shared pointer (it seemed). We might have time to get 
	// back to this time some time in next 3 years :D
	RecreateDefaultFilter();
}

void ARecastNavMesh::OnNavMeshGenerationFinished()
{
	// If this navmesh was spawned at runtime to fill a missing nav data gap,
	// the one-time rebuild is now complete. Clear the flag so it behaves
	// as a Static navmesh going forward.
	if (RequiresInitialRebuild())
	{
		ClearRequiresInitialRebuild();
	}

	UWorld* World = GetWorld();

	if (IsValid(World))
	{
#if WITH_EDITOR	
		// For navmeshes that support streaming create navigation data holders in each streaming level
		// so parts of navmesh can be streamed in/out with those levels
		if (!World->IsGameWorld())
		{
			const auto& Levels = World->GetLevels();
			for (auto Level : Levels)
			{
				if (Level->IsPersistentLevel())
				{
					continue;
				}

				URecastNavMeshDataChunk* NavDataChunk = GetNavigationDataChunk(Level);

				if (SupportsStreaming())
				{
					// We use navigation volumes that belongs to this streaming level to find tiles we want to save
					TArray<FNavTileRef> LevelTiles;
					TArray<FBox> LevelNavBounds = GetNavigableBoundsInLevel(Level);
					RecastNavMeshImpl->GetNavMeshTilesIn(LevelNavBounds, LevelTiles);

					if (LevelTiles.Num())
					{
						// Create new chunk only if we have something to save in it			
						if (NavDataChunk == nullptr)
						{
							NavDataChunk = NewObject<URecastNavMeshDataChunk>(Level);
							NavDataChunk->NavigationDataName = GetFName();
							Level->NavDataChunks.Add(NavDataChunk);
						}

						const EGatherTilesCopyMode CopyMode = RecastNavMeshImpl->NavMeshOwner->SupportsRuntimeGeneration() ? EGatherTilesCopyMode::CopyDataAndCacheData  : EGatherTilesCopyMode::CopyData;
						NavDataChunk->GetTiles(RecastNavMeshImpl, LevelTiles, CopyMode);
						NavDataChunk->MarkPackageDirty();
						continue;
					}
				}

				// stale data that is left in the level
				if (NavDataChunk)
				{
					// clear it
					NavDataChunk->ReleaseTiles();
					NavDataChunk->MarkPackageDirty();
					Level->NavDataChunks.Remove(NavDataChunk);
				}
			}
		}

		// force navmesh drawing update
		RequestDrawingUpdate(/*bForce=*/true);		
#endif// WITH_EDITOR

		UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);
		if (NavSys)
		{
			NavSys->OnNavigationGenerationFinished(*this);
		}
	}
}

#if WITH_RECAST && !UE_BUILD_SHIPPING
uint32 ARecastNavMesh::LogMemUsed() const
{
	const uint32 SuperMemUsed = Super::LogMemUsed();

	uint32 MemUsed = 0;

	if (RecastNavMeshImpl && RecastNavMeshImpl->DetourNavMesh)
	{
		const dtNavMesh* const ConstNavMesh = RecastNavMeshImpl->DetourNavMesh;

		for (int TileIndex = 0; TileIndex < RecastNavMeshImpl->DetourNavMesh->getMaxTiles(); ++TileIndex)
		{
			const dtMeshTile* Tile = ConstNavMesh->getTile(TileIndex);
			if (Tile)
			{
				dtMeshHeader* const H = (dtMeshHeader*)(Tile->header);
				const FDetourTileLayout TileLayout(*Tile);

				MemUsed += TileLayout.TileSize;
			}
		}
	}

	UE_LOGF(LogNavigation, Warning, "%ls: ARecastNavMesh: %u\n    self: %zu", *GetName(), MemUsed, sizeof(ARecastNavMesh));

	return MemUsed + SuperMemUsed;
}

void ARecastNavMesh::DumpDetailedMemoryStats() const
{
#if WITH_RECAST
	if (!RecastNavMeshImpl || !RecastNavMeshImpl->DetourNavMesh)
	{
		UE_LOGF(LogNavigation, Display, "%ls: No Detour navmesh data.", *GetName());
		return;
	}

	const dtNavMesh* NavMesh = RecastNavMeshImpl->DetourNavMesh;
	const int32 MaxTiles = NavMesh->getMaxTiles();

	// Aggregate counters
	int32 TotalActiveTiles = 0;
	uint64 TotalTileDataBytes = 0;

	// Per-section aggregates
	uint64 TotalHeaderBytes = 0;
	uint64 TotalVertBytes = 0;
	uint64 TotalPolyBytes = 0;
	uint64 TotalLinkBytes = 0;
	uint64 TotalDetailMeshBytes = 0;
	uint64 TotalDetailVertBytes = 0;
	uint64 TotalDetailTriBytes = 0;
	uint64 TotalBvTreeBytes = 0;
	uint64 TotalOffMeshConBytes = 0;
	uint64 TotalOffMeshSegBytes = 0;
	uint64 TotalClusterBytes = 0;
	uint64 TotalPolyClusterBytes = 0;

	// Count trackers
	int32 TilesWithBvTree = 0;
	int32 TilesWithoutBvTree = 0;
	int32 TotalPolyCount = 0;
	int32 TotalVertCount = 0;
	int32 TotalLinkCount = 0;
	int32 TotalBvNodeCount = 0;
	int32 TotalDetailVertCount = 0;
	int32 TotalDetailTriCount = 0;
	int32 TotalOffMeshConCount = 0;
	int32 MinPolysPerTile = MAX_int32;
	int32 MaxPolysPerTile = 0;

	// Link utilization tracking
	uint64 TotalBaseLinksAllocated = 0;
	uint64 TotalLinksUsed = 0;
	int32 MaxPreAllocatedLinksInAnyTile = 0;
	uint64 TotalLinksPerPolySum = 0;
	int32 TotalPolyCountForAvg = 0;

	// Link utilization distribution buckets (0-10%, 10-20%, ..., 90-100%)
	static constexpr int32 UtilBuckets = 10;
	int32 UtilBucketCount[UtilBuckets] = {0};

	// Dynamic allocation tracking
	int32 TilesWithDynamic = 0;
	uint64 TotalDynamicLinks = 0;
	int32 MaxDynamicInTile = 0;

	// Histogram: poly count per tile
	// Buckets: [1-2], [3-4], [5-8], [9-16], [17-32], [33-64], [65-128], [129-256], [257+]
	static constexpr int32 NumBuckets = 9;
	static constexpr int32 BucketMaxPolys[NumBuckets] = { 2, 4, 8, 16, 32, 64, 128, 256, MAX_int32 };
	static const char* BucketLabels[NumBuckets] = {
		"1-2", "3-4", "5-8", "9-16", "17-32", "33-64", "65-128", "129-256", "257+"
	};
	int32 BucketTileCount[NumBuckets] = {};
	uint64 BucketTotalBytes[NumBuckets] = {};
	uint64 BucketBvTreeBytes[NumBuckets] = {};
	uint64 BucketDetailBytes[NumBuckets] = {};

	// Iterate all tiles
	for (int32 TileIndex = 0; TileIndex < MaxTiles; ++TileIndex)
	{
		const dtMeshTile* Tile = NavMesh->getTile(TileIndex);
		if (!Tile || !Tile->header)
		{
			continue;
		}

		TotalActiveTiles++;
		const dtMeshHeader* Header = Tile->header;
		const FDetourTileLayout TileLayout(*Tile);

		TotalTileDataBytes += TileLayout.TileSize;
		TotalHeaderBytes += TileLayout.HeaderSize;
		TotalVertBytes += TileLayout.VertsSize;
		TotalPolyBytes += TileLayout.PolysSize;
		TotalLinkBytes += TileLayout.LinksSize;
		TotalDetailMeshBytes += TileLayout.DetailMeshesSize;
		TotalDetailVertBytes += TileLayout.DetailVertsSize;
		TotalDetailTriBytes += TileLayout.DetailTrisSize;
		TotalBvTreeBytes += TileLayout.BvTreeSize;
		TotalOffMeshConBytes += TileLayout.OffMeshConsSize;
		TotalOffMeshSegBytes += TileLayout.OffMeshSegsSize;
		TotalClusterBytes += TileLayout.ClustersSize;
		TotalPolyClusterBytes += TileLayout.PolyClustersSize;

		TotalPolyCount += Header->polyCount;
		TotalVertCount += Header->vertCount;
		TotalLinkCount += Header->maxLinkCount;
		TotalBvNodeCount += Header->bvNodeCount;
		TotalDetailVertCount += Header->detailVertCount;
		TotalDetailTriCount += Header->detailTriCount;
		TotalOffMeshConCount += Header->offMeshConCount;

		// Count actually used links by traversing polygon link chains
		int32 UsedBaseLinks = 0;
		int32 UsedDynamicLinks = 0;
		for (int32 PolyIdx = 0; PolyIdx < Header->polyCount; ++PolyIdx)
		{
			const dtPoly* Poly = &Tile->polys[PolyIdx];
			unsigned int LinkIdx = Poly->firstLink;
			while (LinkIdx != DT_NULL_LINK)
			{
				if (LinkIdx < (unsigned int)Header->maxLinkCount)
				{
					UsedBaseLinks++;
				}
				else
				{
					UsedDynamicLinks++;
				}
				const dtLink& Link = NavMesh->getLink(Tile, LinkIdx);
				LinkIdx = Link.next;
			}
		}

		const int32 UsedLinksInTile = UsedBaseLinks + UsedDynamicLinks;
		TotalBaseLinksAllocated += Header->maxLinkCount;
		TotalLinksUsed += UsedLinksInTile;
		MaxPreAllocatedLinksInAnyTile = FMath::Max(MaxPreAllocatedLinksInAnyTile, (int32)Header->maxLinkCount);

		// Track dynamic allocations
		TotalDynamicLinks += UsedDynamicLinks;
		if (UsedDynamicLinks > 0)
		{
			TilesWithDynamic++;
			MaxDynamicInTile = FMath::Max(MaxDynamicInTile, UsedDynamicLinks);
		}

		// Links per polygon stats
		TotalLinksPerPolySum += UsedLinksInTile;
		TotalPolyCountForAvg += Header->polyCount;

		// Utilization bucket (base pool only)
		const float Utilization = Header->maxLinkCount > 0
			? (float)UsedBaseLinks / Header->maxLinkCount : 0.0f;
		const int32 BucketIdx = FMath::Clamp((int32)(Utilization * 10.0f), 0, UtilBuckets - 1);
		UtilBucketCount[BucketIdx]++;

		if (Header->bvNodeCount > 0)
		{
			TilesWithBvTree++;
		}
		else
		{
			TilesWithoutBvTree++;
		}

		MinPolysPerTile = FMath::Min(MinPolysPerTile, (int32)Header->polyCount);
		MaxPolysPerTile = FMath::Max(MaxPolysPerTile, (int32)Header->polyCount);

		// Bucket by poly count
		for (int32 Bucket = 0; Bucket < NumBuckets; ++Bucket)
		{
			if ((int32)Header->polyCount <= BucketMaxPolys[Bucket])
			{
				BucketTileCount[Bucket]++;
				BucketTotalBytes[Bucket] += TileLayout.TileSize;
				BucketBvTreeBytes[Bucket] += TileLayout.BvTreeSize;
				BucketDetailBytes[Bucket] += TileLayout.DetailMeshesSize
					+ TileLayout.DetailVertsSize + TileLayout.DetailTrisSize;
				break;
			}
		}
	}

	if (TotalActiveTiles == 0)
	{
		UE_LOGF(LogNavigation, Display, "%ls: No active tiles.", *GetName());
		return;
	}

	// Tile pool overhead
	const uint64 TilePoolBytes = (uint64)MaxTiles * sizeof(dtMeshTile);
	const uint64 LookupTableBytes =
		(uint64)FMath::Max(1u, dtNextPow2(static_cast<unsigned int>(MaxTiles) / 4)) * sizeof(dtMeshTile*);

	// Log everything
	constexpr float ToKB = 1.0f / 1024.0f;
	constexpr float ToMB = 1.0f / (1024.0f * 1024.0f);

	UE_LOGF(LogNavigation, Display, "=======================================================================================================");
	UE_LOGF(LogNavigation, Display, "  NavMesh Memory Report: %ls", *GetName());
	UE_LOGF(LogNavigation, Display, "=======================================================================================================");
	UE_LOGF(LogNavigation, Display, "  Max tiles (pool):    %d", MaxTiles);
	UE_LOGF(LogNavigation, Display, "  Active tiles:        %d", TotalActiveTiles);
	UE_LOGF(LogNavigation, Display, "  Pool utilization:    %.1f%%",
		100.0f * (float)TotalActiveTiles / FMath::Max(MaxTiles, 1.f));
	UE_LOGF(LogNavigation, Display, "");

	// Overall memory
	UE_LOGF(LogNavigation, Display, "--- Overall Memory ---");
	UE_LOGF(LogNavigation, Display, "  Tile data total:     %10llu bytes  (%.2f MB)", TotalTileDataBytes, (float)TotalTileDataBytes * ToMB);
	UE_LOGF(LogNavigation, Display, "  Tile pool structs:   %10llu bytes  (%.2f MB)", TilePoolBytes, (float)TilePoolBytes * ToMB);
	UE_LOGF(LogNavigation, Display, "  Lookup table:        %10llu bytes  (%.2f MB)", LookupTableBytes, (float)LookupTableBytes * ToMB);
	UE_LOGF(LogNavigation, Display, "  TOTAL:               %10llu bytes  (%.2f MB)",
		(TotalTileDataBytes + TilePoolBytes + LookupTableBytes),
		(float)(TotalTileDataBytes + TilePoolBytes + LookupTableBytes) * ToMB);
	UE_LOGF(LogNavigation, Display, "");

	// Per-section breakdown
	UE_LOGF(LogNavigation, Display, "--- Per-Section Breakdown (tile data only) ---");
	UE_LOGF(LogNavigation, Display, "  %-20s %9s %7s %6s    %s",
		"Section", "Bytes", "MB", "%", "Description");

	auto LogSection = [&](const char* Name, uint64 Bytes, const char* Desc)
	{
		const float Pct = TotalTileDataBytes > 0
			? 100.0f * (float)Bytes / (float)TotalTileDataBytes : 0.0f;

		UE_LOGF(LogNavigation, Display, "  %-20s %9llu %7.2f %6.1f%%   %s",
			Name, Bytes, (float)Bytes * ToMB, Pct, Desc);
	};

	LogSection("Header",        TotalHeaderBytes,      "");
	LogSection("Vertices",      TotalVertBytes,
		TCHAR_TO_ANSI(*FString::Printf(TEXT("%d verts, %d bytes/vert"), TotalVertCount, TotalVertCount > 0 ? (int32)(TotalVertBytes / TotalVertCount) : 0)));
	LogSection("Polygons",      TotalPolyBytes, TCHAR_TO_ANSI(*FString::Printf(TEXT("%d polys"), TotalPolyCount)));
	LogSection("Links",         TotalLinkBytes, TCHAR_TO_ANSI(*FString::Printf(TEXT("%d max links allocated"), TotalLinkCount)));
	LogSection("Detail Meshes", TotalDetailMeshBytes, "");
	LogSection("Detail Verts",  TotalDetailVertBytes, TCHAR_TO_ANSI(*FString::Printf(TEXT("%d detail verts"), TotalDetailVertCount)));
	LogSection("Detail Tris",   TotalDetailTriBytes, TCHAR_TO_ANSI(*FString::Printf(TEXT("%d detail tris"), TotalDetailTriCount)));
	LogSection("BV Tree",       TotalBvTreeBytes, TCHAR_TO_ANSI(*FString::Printf(TEXT("%d nodes, %d/%d tiles have BV tree"), TotalBvNodeCount, TilesWithBvTree, TotalActiveTiles)));
	LogSection("OffMesh Cons",  TotalOffMeshConBytes, TCHAR_TO_ANSI(*FString::Printf(TEXT("%d connections"), TotalOffMeshConCount)));
	LogSection("OffMesh Segs",  TotalOffMeshSegBytes, "");
	LogSection("Clusters",      TotalClusterBytes, "");
	LogSection("PolyClusters",  TotalPolyClusterBytes, "");
	UE_LOGF(LogNavigation, Display, "");

	// Per-tile statstistics
	UE_LOGF(LogNavigation, Display, "--- Per-Tile Statistics ---");
	const uint64 AvgTileBytes = TotalTileDataBytes / TotalActiveTiles;
	UE_LOGF(LogNavigation, Display, "  Avg tile data size:  %llu bytes (%.4f MB)", AvgTileBytes, (float)AvgTileBytes * ToMB);
	UE_LOGF(LogNavigation, Display, "  Min polys/tile:      %d", MinPolysPerTile);
	UE_LOGF(LogNavigation, Display, "  Max polys/tile:      %d", MaxPolysPerTile);
	UE_LOGF(LogNavigation, Display, "  Avg polys/tile:      %.1f", (float)TotalPolyCount / (float)TotalActiveTiles);
	UE_LOGF(LogNavigation, Display, "");

	// Poly count histogram
	UE_LOGF(LogNavigation, Display, "--- Tile Distribution by Poly Count ---");
	UE_LOGF(LogNavigation, Display, "  %-16s %6s %6s     %9s  %10s  %10s  %10s   %10s  %10s",
		"Polys", "Tiles", "%",
		"Total KB", "Total MB", "BvTree KB", "BvTree MB", "Detail KB", "Detail MB");

	for (int32 Bucket = 0; Bucket < NumBuckets; ++Bucket)
	{
		if (BucketTileCount[Bucket] == 0)
		{
			continue;
		}
		
		const float TilePercentage = 100.0f * (float)BucketTileCount[Bucket] / (float)TotalActiveTiles;
		UE_LOGF(LogNavigation, Display,
			"  %-16hs %6d %6.1f%%     %9.1f  %10.2f  %10.1f  %10.2f   %10.1f  %10.2f",
			BucketLabels[Bucket],
			BucketTileCount[Bucket],
			TilePercentage,
			(float)BucketTotalBytes[Bucket] * ToKB,
			(float)BucketTotalBytes[Bucket] * ToMB,
			(float)BucketBvTreeBytes[Bucket] * ToKB,
			(float)BucketBvTreeBytes[Bucket] * ToMB,
			(float)BucketDetailBytes[Bucket] * ToKB,
			(float)BucketDetailBytes[Bucket] * ToMB);
	}
	UE_LOGF(LogNavigation, Display, "");

	// Link utilization analysis
	const uint64 TotalAllocated = TotalBaseLinksAllocated + TotalDynamicLinks;
	UE_LOGF(LogNavigation, Display, "--- Link Utilization Analysis ---");
	UE_LOGF(LogNavigation, Display, "  Base pool allocated:   %llu links", TotalBaseLinksAllocated);
	UE_LOGF(LogNavigation, Display, "  Dynamic used:        %llu links", TotalDynamicLinks);
	UE_LOGF(LogNavigation, Display, "  Total allocated:       %llu links", TotalAllocated);
	UE_LOGF(LogNavigation, Display, "  Links actually used:   %llu (%.1f%%)", TotalLinksUsed, 100.0f * (float)TotalLinksUsed / (float)FMath::Max((uint64)1, TotalAllocated));

	const uint64 WastedLinks = TotalAllocated > TotalLinksUsed
		? TotalAllocated - TotalLinksUsed : 0;

	UE_LOGF(LogNavigation, Display, "  Wasted links:          %llu (%.2f MB)",
		WastedLinks,
		(float)WastedLinks * (float)sizeof(dtLink) * ToMB);
	UE_LOGF(LogNavigation, Display, "  Max preallocated links in any tile: %d", MaxPreAllocatedLinksInAnyTile);

	if (TotalPolyCountForAvg > 0)
	{
		UE_LOGF(LogNavigation, Display, "  Links per poly:        avg=%.2f",
			(float)TotalLinksPerPolySum / (float)TotalPolyCountForAvg);
	}

	UE_LOGF(LogNavigation, Display, "");
	UE_LOGF(LogNavigation, Display, "--- Tile Utilization Distribution ---");
	UE_LOGF(LogNavigation, Display, "  Utilization    Tiles     %%");
	for (int32 i = 0; i < UtilBuckets; ++i)
	{
		if (UtilBucketCount[i] > 0)
		{
			UE_LOGF(LogNavigation, Display, "  %3d%%-%3d%%     %6d   %5.1f%%",
				i * 10, (i + 1) * 10,
				UtilBucketCount[i],
				100.0f * (float)UtilBucketCount[i] / (float)TotalActiveTiles);
		}
	}
	UE_LOGF(LogNavigation, Display, "");

	// Dynamic allocation statistics
	UE_LOGF(LogNavigation, Display, "--- Dynamic Allocation Statistics ---");
	UE_LOGF(LogNavigation, Display, "  Tiles using dynamic: %d (%.1f%%)",
		TilesWithDynamic, 100.0f * (float)TilesWithDynamic / (float)FMath::Max(TotalActiveTiles, 1));
	UE_LOGF(LogNavigation, Display, "  Total dynamic links: %llu (%.1f%% of used links)",
		TotalDynamicLinks,
		TotalLinksUsed > 0 ? 100.0f * (float)TotalDynamicLinks / (float)TotalLinksUsed : 0.0f);
	UE_LOGF(LogNavigation, Display, "  Max dynamic in tile: %d links", MaxDynamicInTile);
	UE_LOGF(LogNavigation, Display, "  Dynamic memory:      %.2f MB",
		(float)TotalDynamicLinks * (float)sizeof(dtLink) * ToMB);

	UE_LOGF(LogNavigation, Display, "=======================================================================================================");
	UE_LOGF(LogNavigation, Display, "");

#else  // WITH_RECAST
	UE_LOGF(LogNavigation, Display, "%ls: Recast not available.", *GetName());
#endif // WITH_RECAST
}
#endif // WITH_RECAST && !UE_BUILD_SHIPPING

uint16 ARecastNavMesh::GetDefaultForbiddenFlags() const
{
	return FPImplRecastNavMesh::GetFilterForbiddenFlags((const FRecastQueryFilter*)DefaultQueryFilter->GetImplementation());
}

void ARecastNavMesh::SetDefaultForbiddenFlags(uint16 ForbiddenAreaFlags)
{
	FPImplRecastNavMesh::SetFilterForbiddenFlags((FRecastQueryFilter*)DefaultQueryFilter->GetImplementation(), ForbiddenAreaFlags);
}

void ARecastNavMesh::SetMaxSimultaneousTileGenerationJobsCount(int32 NewJobsCountLimit) 
{
	const int32 NewCount = NewJobsCountLimit > 0 ? NewJobsCountLimit : 1;
	if (MaxSimultaneousTileGenerationJobsCount != NewCount)
	{
		MaxSimultaneousTileGenerationJobsCount = NewCount;
		if (GetGenerator() != nullptr)
		{
			FRecastNavMeshGenerator* MyGenerator = static_cast<FRecastNavMeshGenerator*>(GetGenerator());
			MyGenerator->SetMaxTileGeneratorTasks(NewCount);
		}
	}
}

bool ARecastNavMesh::FilterPolys(TArray<NavNodeRef>& PolyRefs, const FRecastQueryFilter* Filter, const UObject* QueryOwner) const
{
	bool bSuccess = false;
	if (RecastNavMeshImpl)
	{
		bSuccess = RecastNavMeshImpl->FilterPolys(PolyRefs, Filter, QueryOwner);
	}

	return bSuccess;
}

void ARecastNavMesh::ApplyWorldOffset(const FVector& InOffset, bool bWorldShift)
{
	if (RecastNavMeshImpl)
	{
		RecastNavMeshImpl->ApplyWorldOffset(InOffset, bWorldShift);
	}

	Super::ApplyWorldOffset(InOffset, bWorldShift);
	RequestDrawingUpdate();
}

void ARecastNavMesh::FillNavigationDataChunkActor(const FBox& QueryBounds, ANavigationDataChunkActor& DataChunkActor, FBox& OutTilesBounds) const
{
	if (!RecastNavMeshImpl)
	{
		return;
	}

	if (!ensureMsgf(QueryBounds.IsValid, TEXT("%hs QueryBounds is not valid"), __FUNCTION__))
	{
		return;
	}

	UE_LOGF(LogNavigation, Verbose, "%ls Bounds pos: (%ls)  size: (%ls).", ANSI_TO_TCHAR(__FUNCTION__), *QueryBounds.GetCenter().ToString(), *QueryBounds.GetSize().ToString());

	const TArray<FBox> Boxes({ QueryBounds });
	TArray<FNavTileRef> TileRefs;
	RecastNavMeshImpl->GetNavMeshTilesIn(Boxes, TileRefs);
	if (!TileRefs.IsEmpty())
	{
		// Add a data chunk for this navmesh
		URecastNavMeshDataChunk* DataChunk = NewObject<URecastNavMeshDataChunk>(&DataChunkActor);
		DataChunk->NavigationDataName = GetFName();
		DataChunkActor.GetMutableNavDataChunk().Add(DataChunk);

		DataChunk->GetTiles(RecastNavMeshImpl, TileRefs, SupportsRuntimeGeneration() ? EGatherTilesCopyMode::CopyDataAndCacheData : EGatherTilesCopyMode::CopyData);
		DataChunk->GetTilesBounds(*RecastNavMeshImpl, TileRefs, OutTilesBounds);
	}
}

void ARecastNavMesh::OnStreamingNavDataAdded(ANavigationDataChunkActor& InActor)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMesh_OnStreamingNavDataAdded);

	if (SupportsStreaming() && RecastNavMeshImpl)
	{
		UE_VLOG_BOX(this, LogNavigation, Log, InActor.GetBounds(), FColor::Blue, TEXT(""));
		
		URecastNavMeshDataChunk* NavDataChunk = GetNavigationDataChunk(InActor);
		if (NavDataChunk)
		{
			AttachNavMeshDataChunk(*NavDataChunk);
		}

		if (IsWorldPartitionedDynamicNavmesh())
		{
			// Add dirtiness for preexisting elements that are not part of the base navmesh.
			UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
			if (NavSys)
			{
				FNavigationOctreeFilter Filter;
				Filter.bIncludeGeometry = true;
				Filter.bExcludeLoadedData = true;
			
				TArray<FNavigationOctreeElement> NavElements;
				NavSys->FindElementsInNavOctree(InActor.GetBounds(), Filter, NavElements);

				for (const FNavigationOctreeElement& NavElement : NavElements)
				{
					UE_VLOG_BOX(this, LogNavigation, Verbose, NavElement.Bounds.GetBox(), FColor::Orange, TEXT(""));

					NavSys->AddDirtyArea(
						NavElement.Bounds.GetBox(),
						ENavigationDirtyFlag::All,
						[SourceElement = NavElement.Data->SourceElement]
						{
							return SourceElement;
						},
						"Streaming data added");
				}
			}
		}
	}
}

void ARecastNavMesh::OnStreamingNavDataRemoved(ANavigationDataChunkActor& InActor)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMesh_OnStreamingNavDataRemoved);

	if (SupportsStreaming() && RecastNavMeshImpl)
	{
		UE_VLOG_BOX(this, LogNavigation, Log, InActor.GetBounds(), FColor::Red, TEXT(""));
		
		URecastNavMeshDataChunk* NavDataChunk = GetNavigationDataChunk(InActor);
		if (NavDataChunk)
		{
			DetachNavMeshDataChunk(*NavDataChunk);
		}
	}
}

void ARecastNavMesh::OnStreamingLevelAdded(ULevel* InLevel, UWorld* InWorld)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMesh_OnStreamingLevelAdded);
	
	if (SupportsStreaming() && RecastNavMeshImpl)
	{
		URecastNavMeshDataChunk* NavDataChunk = GetNavigationDataChunk(InLevel);
		if (NavDataChunk)
		{
			AttachNavMeshDataChunk(*NavDataChunk);
		}
	}
}

void ARecastNavMesh::AttachNavMeshDataChunk(URecastNavMeshDataChunk& NavDataChunk)
{
	const TArray<FNavTileRef> AttachedIndices = NavDataChunk.AttachTiles(*this);
	if (AttachedIndices.Num() > 0)
	{
		InvalidateAffectedPaths(AttachedIndices);
		RequestDrawingUpdate();
	}
}

void ARecastNavMesh::OnStreamingLevelRemoved(ULevel* InLevel, UWorld* InWorld)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMesh_OnStreamingLevelRemoved);
	
	if (SupportsStreaming() && RecastNavMeshImpl)
	{
		URecastNavMeshDataChunk* NavDataChunk = GetNavigationDataChunk(InLevel);
		if (NavDataChunk)
		{
			DetachNavMeshDataChunk(*NavDataChunk);
		}
	}
}

#if WITH_EDITOR
double ARecastNavMesh::GetWorldPartitionNavigationDataBuilderOverlap() const
{
	return TileSizeUU;
}
#endif //WITH_EDITOR

void ARecastNavMesh::DetachNavMeshDataChunk(URecastNavMeshDataChunk& NavDataChunk)
{
	const TArray<FNavTileRef> DetachedIndices = NavDataChunk.DetachTiles(*this);
	if (DetachedIndices.Num() > 0)
	{
		InvalidateAffectedPaths(DetachedIndices);
		RequestDrawingUpdate();
	}
}

bool ARecastNavMesh::AdjustLocationWithFilter(const FVector& StartLoc, FVector& OutAdjustedLocation, const FNavigationQueryFilter& Filter, const UObject* QueryOwner) const
{
	if (HasValidNavmesh() == false)
	{
		return false;
	}

	INITIALIZE_NAVQUERY(NavQuery, Filter.GetMaxSearchNodes());

	const FVector NavExtent = GetModifiedQueryExtent(GetDefaultQueryExtent());
	const FVector::FReal Extent[3] = { NavExtent.X, NavExtent.Z, NavExtent.Y };

	const dtQueryFilter* QueryFilter = ((const FRecastQueryFilter*)(Filter.GetImplementation()))->GetAsDetourQueryFilter();
	ensure(QueryFilter);

	FVector RecastStart = Unreal2RecastPoint(StartLoc);
	FVector RecastAdjustedPoint = Unreal2RecastPoint(StartLoc);
	NavNodeRef StartPolyID = INVALID_NAVNODEREF;
	NavQuery.findNearestPoly(&RecastStart.X, Extent, QueryFilter, &StartPolyID, &RecastAdjustedPoint.X);

	if (FVector::DistSquared(RecastStart, RecastAdjustedPoint) < KINDA_SMALL_NUMBER)
	{
		OutAdjustedLocation = StartLoc;
		return false;
	}
	else
	{
		OutAdjustedLocation = Recast2UnrealPoint(RecastAdjustedPoint);
		// move it just a bit further - otherwise recast can still pick "wrong" poly when 
		// later projecting StartLoc (meaning a poly we want to filter out with 
		// QueryFilter here)
		OutAdjustedLocation += (OutAdjustedLocation - StartLoc).GetSafeNormal() * 0.1f;
		return true;
	}
}

FPathFindingResult ARecastNavMesh::FindPath(const FNavAgentProperties& AgentProperties, const FPathFindingQuery& Query)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastPathfinding);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Pathfinding);

	const ANavigationData* Self = Query.NavData.Get();
	if (Self == nullptr)
	{
		return ENavigationQueryResult::Error;
	}
	
	check(Cast<const ARecastNavMesh>(Self));
	const ARecastNavMesh* RecastNavMesh = (const ARecastNavMesh*)Self;
	if (RecastNavMesh->RecastNavMeshImpl == nullptr)
	{
		return ENavigationQueryResult::Error;
	}
		
	FPathFindingResult Result(ENavigationQueryResult::Error);

	FNavigationPath* NavPath = Query.PathInstanceToFill.Get();
	FNavMeshPath* NavMeshPath = NavPath ? NavPath->CastPath<FNavMeshPath>() : nullptr;

	if (NavMeshPath)
	{
		Result.Path = Query.PathInstanceToFill;
		NavMeshPath->ResetForRepath();
	}
	else
	{
		Result.Path = Self->CreatePathInstance<FNavMeshPath>(Query);
		NavPath = Result.Path.Get();
		NavMeshPath = NavPath ? NavPath->CastPath<FNavMeshPath>() : nullptr;
	}

	const FNavigationQueryFilter* NavFilter = Query.QueryFilter.Get();
	if (NavMeshPath && NavFilter)
	{
		NavMeshPath->ApplyFlags(Query.NavDataFlags);

		const FVector AdjustedEndLocation = NavFilter->GetAdjustedEndLocation(Query.EndLocation);
		if ((Query.StartLocation - AdjustedEndLocation).IsNearlyZero() == true)
		{
			Result.Path->GetPathPoints().Reset();
			Result.Path->GetPathPoints().Add(FNavPathPoint(AdjustedEndLocation));
			Result.Result = ENavigationQueryResult::Success;
		}
		else
		{
			Result.Result = RecastNavMesh->RecastNavMeshImpl->FindPath(Query.StartLocation, AdjustedEndLocation, Query.CostLimit, Query.bRequireNavigableEndLocation, *NavMeshPath, *NavFilter, Query.Owner.Get());

			const bool bPartialPath = Result.IsPartial();
			if (bPartialPath)
			{
				Result.Result = Query.bAllowPartialPaths ? ENavigationQueryResult::Success : ENavigationQueryResult::Fail;
			}
		}
	}

	return Result;
}

bool ARecastNavMesh::TestPath(const FNavAgentProperties& AgentProperties, const FPathFindingQuery& Query, int32* NumVisitedNodes)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastTestPath);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Pathfinding);

	const ANavigationData* Self = Query.NavData.Get();
	check(Cast<const ARecastNavMesh>(Self));

	const ARecastNavMesh* RecastNavMesh = (const ARecastNavMesh*)Self;
	if (Self == NULL || RecastNavMesh->RecastNavMeshImpl == NULL)
	{
		return false;
	}

	bool bPathExists = true;

	const FNavigationQueryFilter* NavFilter = Query.QueryFilter.Get();
	if (NavFilter)
	{
		const FVector AdjustedEndLocation = NavFilter->GetAdjustedEndLocation(Query.EndLocation);
		if ((Query.StartLocation - AdjustedEndLocation).IsNearlyZero() == false)
		{
			ENavigationQueryResult::Type Result = RecastNavMesh->RecastNavMeshImpl->TestPath(Query.StartLocation, AdjustedEndLocation, Query.bRequireNavigableEndLocation, *NavFilter, Query.Owner.Get(), NumVisitedNodes);
			bPathExists = (Result == ENavigationQueryResult::Success);
		}
	}

	return bPathExists;
}

bool ARecastNavMesh::TestHierarchicalPath(const FNavAgentProperties& AgentProperties, const FPathFindingQuery& Query, int32* NumVisitedNodes)
{
	const ANavigationData* Self = Query.NavData.Get();
	check(Cast<const ARecastNavMesh>(Self));

	const ARecastNavMesh* RecastNavMesh = (const ARecastNavMesh*)Self;
	if (Self == nullptr || RecastNavMesh->RecastNavMeshImpl == nullptr || RecastNavMesh->RecastNavMeshImpl->DetourNavMesh == nullptr)
	{
		return false;
	}

	const bool bCanUseHierachicalPath = (Query.QueryFilter == RecastNavMesh->GetDefaultQueryFilter());
	bool bPathExists = true;

	const FNavigationQueryFilter* NavFilter = Query.QueryFilter.Get();
	if (NavFilter)
	{
		const FVector AdjustedEndLocation = NavFilter->GetAdjustedEndLocation(Query.EndLocation);
		if ((Query.StartLocation - AdjustedEndLocation).IsNearlyZero() == false)
		{
			bool bUseFallbackSearch = false;
			if (bCanUseHierachicalPath)
			{
#if WITH_NAVMESH_CLUSTER_LINKS
				ENavigationQueryResult::Type Result = RecastNavMesh->RecastNavMeshImpl->TestClusterPath(Query.StartLocation, AdjustedEndLocation, NumVisitedNodes);
				bPathExists = (Result == ENavigationQueryResult::Success);

				if (Result == ENavigationQueryResult::Error)
				{
					bUseFallbackSearch = true;
				}
#else
				UE_LOGF(LogNavigation, Error, "Navmesh requires generation of clusters for hierarchical path. Set WITH_NAVMESH_CLUSTER_LINKS to 1 to generate them.");
				bPathExists = false;
#endif // WITH_NAVMESH_CLUSTER_LINKS
			}
			else
			{
				UE_LOGF(LogNavigation, Log, "Hierarchical path finding test failed: filter doesn't match!");
				bUseFallbackSearch = true;
			}

			if (bUseFallbackSearch)
			{
				ENavigationQueryResult::Type Result = RecastNavMesh->RecastNavMeshImpl->TestPath(Query.StartLocation, AdjustedEndLocation, Query.bRequireNavigableEndLocation, *NavFilter, Query.Owner.Get(), NumVisitedNodes);
				bPathExists = (Result == ENavigationQueryResult::Success);
			}
		}
	}

	return bPathExists;
}

bool ARecastNavMesh::NavMeshRaycast(const ANavigationData* Self, const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, FSharedConstNavQueryFilter QueryFilter,const UObject* QueryOwner, FRaycastResult& Result)
{
	check(Cast<const ARecastNavMesh>(Self));

	const ARecastNavMesh* RecastNavMesh = (const ARecastNavMesh*)Self;
	if (Self == NULL || RecastNavMesh->RecastNavMeshImpl == NULL)
	{
		HitLocation = RayStart;
		return true;
	}

	RecastNavMesh->RecastNavMeshImpl->Raycast(RayStart, RayEnd, RecastNavMesh->GetRightFilterRef(QueryFilter), QueryOwner, Result);
	HitLocation = Result.HasHit() ? (RayStart + (RayEnd - RayStart) * Result.HitTime) : RayEnd;

	return Result.HasHit();
}

bool ARecastNavMesh::NavMeshRaycast(const ANavigationData* Self, NavNodeRef RayStartNode, const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, FSharedConstNavQueryFilter QueryFilter, const UObject* Querier)
{
	return NavMeshRaycast(Self, RayStartNode, RayStart, RayEnd, HitLocation, nullptr, QueryFilter, Querier);
}

bool ARecastNavMesh::NavMeshRaycast(const ANavigationData* Self, NavNodeRef RayStartNode, const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, FNavigationRaycastAdditionalResults* AdditionalResults, FSharedConstNavQueryFilter QueryFilter, const UObject* QueryOwner)
{
	check(Cast<const ARecastNavMesh>(Self));

	const ARecastNavMesh* RecastNavMesh = (const ARecastNavMesh*)Self;
	if (Self == NULL || RecastNavMesh->RecastNavMeshImpl == NULL)
	{
		HitLocation = RayStart;
		if (AdditionalResults)
		{
			AdditionalResults->bIsRayEndInCorridor = false;
		}
		return true;
	}

	FRaycastResult Result;
	RecastNavMesh->RecastNavMeshImpl->Raycast(RayStart, RayEnd, RecastNavMesh->GetRightFilterRef(QueryFilter), QueryOwner, Result, RayStartNode);

	HitLocation = Result.HasHit() ? (RayStart + (RayEnd - RayStart) * Result.HitTime) : RayEnd;
	if (AdditionalResults)
	{
		AdditionalResults->bIsRayEndInCorridor = Result.bIsRaycastEndInCorridor;
	}
	return Result.HasHit();
}

void ARecastNavMesh::BatchRaycast(TArray<FNavigationRaycastWork>& Workload, FSharedConstNavQueryFilter Filter, const UObject* Querier) const
{
	if (RecastNavMeshImpl == NULL || Workload.Num() == 0 || RecastNavMeshImpl->DetourNavMesh == NULL)
	{
		return;
	}

	const FNavigationQueryFilter& FilterToUse = GetRightFilterRef(Filter);

	FRecastSpeciaLinkFilter LinkFilter(FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld()), Querier);
	INITIALIZE_NAVQUERY_WLINKFILTER(NavQuery, FilterToUse.GetMaxSearchNodes(), LinkFilter);
	const dtQueryFilter* QueryFilter = ((const FRecastQueryFilter*)(FilterToUse.GetImplementation()))->GetAsDetourQueryFilter();
	
	if (QueryFilter == NULL)
	{
		UE_VLOG(this, LogNavigation, Warning, TEXT("FPImplRecastNavMesh::FindPath failing due to QueryFilter == NULL"));
		return;
	}
	
	const FVector NavExtent = GetModifiedQueryExtent(GetDefaultQueryExtent());
	const FVector::FReal Extent[3] = { NavExtent.X, NavExtent.Z, NavExtent.Y };

	for (FNavigationRaycastWork& WorkItem : Workload)
	{
		ARecastNavMesh::FRaycastResult RaycastResult;

		const FVector RecastStart = Unreal2RecastPoint(WorkItem.RayStart);
		const FVector RecastEnd = Unreal2RecastPoint(WorkItem.RayEnd);

		NavNodeRef StartNode = INVALID_NAVNODEREF;
		NavQuery.findNearestContainingPoly(&RecastStart.X, Extent, QueryFilter, &StartNode, NULL);

		if (StartNode != INVALID_NAVNODEREF)
		{
			FVector::FReal RecastHitNormal[3];

			const dtStatus RaycastStatus = NavQuery.raycast(StartNode, &RecastStart.X, &RecastEnd.X
				, QueryFilter, &RaycastResult.HitTime, RecastHitNormal
				, RaycastResult.CorridorPolys, &RaycastResult.CorridorPolysCount, RaycastResult.GetMaxCorridorSize());

			if (dtStatusSucceed(RaycastStatus))
			{
				if (RaycastResult.HasHit())
				{
					WorkItem.bDidHit = true;
					WorkItem.HitLocation = FNavLocation(WorkItem.RayStart + (WorkItem.RayEnd - WorkItem.RayStart) * RaycastResult.HitTime, RaycastResult.GetLastNodeRef());
				}
				else
				{
					WorkItem.bIsRayEndInCorridor = RaycastResult.bIsRaycastEndInCorridor;
				}
			}
		}
	}
}

bool ARecastNavMesh::IsSegmentOnNavmesh(const FVector& SegmentStart, const FVector& SegmentEnd, FSharedConstNavQueryFilter Filter, const UObject* QueryOwner) const
{
	FVector HitLocation;
	FRaycastResult Result;
	const bool bDidHit = NavMeshRaycast(this, SegmentStart, SegmentEnd, HitLocation, Filter, QueryOwner, Result);
	return Result.bIsRaycastEndInCorridor && !bDidHit;
}

bool ARecastNavMesh::FindStraightPath(const FVector& StartLoc, const FVector& EndLoc, const TArray<NavNodeRef>& PathCorridor, TArray<FNavPathPoint>& PathPoints, TArray<FNavLinkId>* CustomLinks) const
{
	return RecastNavMeshImpl && RecastNavMeshImpl->FindStraightPath(StartLoc, EndLoc, PathCorridor, PathPoints, CustomLinks);
}

int32 ARecastNavMesh::DebugPathfinding(const FPathFindingQuery& Query, TArray<FRecastDebugPathfindingData>& Steps)
{
	int32 NumSteps = 0;

	const ANavigationData* Self = Query.NavData.Get();
	check(Cast<const ARecastNavMesh>(Self));

	const ARecastNavMesh* RecastNavMesh = (const ARecastNavMesh*)Self;
	if (Self == NULL || RecastNavMesh->RecastNavMeshImpl == NULL)
	{
		return false;
	}

	if ((Query.StartLocation - Query.EndLocation).IsNearlyZero() == false)
	{
		NumSteps = RecastNavMesh->RecastNavMeshImpl->DebugPathfinding(Query.StartLocation, Query.EndLocation, Query.CostLimit, Query.bRequireNavigableEndLocation, *(Query.QueryFilter.Get()), Query.Owner.Get(), Steps);
	}

	return NumSteps;
}

void ARecastNavMesh::UpdateNavVersion() 
{ 
	NavMeshVersion = NAVMESHVER_LATEST; 
}

#if WITH_EDITOR

bool ARecastNavMesh::CanUpdateNavMeshInEditor() const
{
	const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	if (!HasAnyFlags(RF_ClassDefaultObject)
		&& NavSys && NavSys->GetIsAutoUpdateEnabled())
	{
		return true;
	}

	return false;
}

void ARecastNavMesh::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	static const FName NAME_Generation = FName("Generation");
	static const FName NAME_NavMeshResolutionParams = FName("NavMeshResolutionParams");

	Super::PostEditChangeChainProperty(PropertyChangedChainEvent);
	
	if (PropertyChangedChainEvent.Property != NULL)
	{
		FName CategoryName = FObjectEditorUtils::GetCategoryFName(PropertyChangedChainEvent.Property);

		// If any, get the category of the parent node. 
		FProperty* MemberProperty = nullptr;

		const FEditPropertyChain::TDoubleLinkedListNode* PropertyNode = PropertyChangedChainEvent.PropertyChain.GetActiveMemberNode();
		if (PropertyNode)
		{
			MemberProperty = PropertyNode->GetValue();
			if (MemberProperty)
			{
				CategoryName = FObjectEditorUtils::GetCategoryFName(MemberProperty);
			}
		}
		
		if (CategoryName == NAME_Generation)
		{
			static const FName NAME_NavLinkJumpConfigs = GET_MEMBER_NAME_CHECKED(ARecastNavMesh, NavLinkJumpConfigs);
			
			const FName PropName = PropertyChangedChainEvent.Property->GetFName();
			bool bRebuild = false;
			
			if (PropName == GET_MEMBER_NAME_CHECKED(FNavMeshResolutionParam, CellSize))
			{
				const int32 ChangedIndex = PropertyChangedChainEvent.GetArrayIndex(NAME_NavMeshResolutionParams.ToString());
				if (ChangedIndex != INDEX_NONE)
				{
					float& RefCellSize = NavMeshResolutionParams[ChangedIndex].CellSize;
					RefCellSize = UE::NavMesh::Private::GetClampedCellSize(RefCellSize);
				 
					TileSizeUU = UE::NavMesh::Private::GetClampedTileSizeUU(TileSizeUU, RefCellSize, AgentRadius);

					// Adjust tile size to be a multiple of RefCellSize
					const float RefCellCount = FMath::TruncToFloat( TileSizeUU / RefCellSize);
					TileSizeUU = RefCellCount * RefCellSize;

					// Adjust the other cell size (find count of cells and set the size of the cell)
					for (uint8 Index = 0; Index < (uint8)ENavigationDataResolution::MAX; Index++)
					{
						if (Index != ChangedIndex)
						{
							float& ResolutionCellSize = NavMeshResolutionParams[Index].CellSize;
							const int32 ResolutionCellCount = FMath::RoundToInt(TileSizeUU / ResolutionCellSize);
							ResolutionCellSize = UE::NavMesh::Private::GetClampedCellSize(TileSizeUU / (float)ResolutionCellCount);
						}
					}

					bRebuild = true;
				}
			}
			else if (PropName == GET_MEMBER_NAME_CHECKED(FNavMeshResolutionParam, CellHeight))
			{
				bRebuild = true;
			}
			else if (MemberProperty && (MemberProperty->GetFName() == NAME_NavLinkJumpConfigs))
			{
				bRebuild = true;
			}

			if (bRebuild)
			{
				// update config
				FillConfig(NavDataConfig);

				if (CanUpdateNavMeshInEditor()
					&& PropName != GET_MEMBER_NAME_CHECKED(ARecastNavMesh, MaxSimultaneousTileGenerationJobsCount))
				{
					RebuildAll();
				}				
			}
		}
	}
}

void ARecastNavMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName NAME_Generation = FName("Generation");
	static const FName NAME_Display = FName("Display");
	static const FName NAME_RuntimeGeneration = FName("RuntimeGeneration");
	static const FName NAME_TileNumberHardLimit = GET_MEMBER_NAME_CHECKED(ARecastNavMesh, TileNumberHardLimit);
	static const FName NAME_Query = FName("Query");

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property != NULL)
	{
		const FName CategoryName = FObjectEditorUtils::GetCategoryFName(PropertyChangedEvent.Property);
		if (CategoryName == NAME_Generation)
		{
			const FName PropName = PropertyChangedEvent.Property->GetFName();
			
			if (PropName == GET_MEMBER_NAME_CHECKED(ARecastNavMesh, AgentRadius))
			{
				// changing AgentRadius is no longer affecting TileSizeUU since 
				// that's not how we use it. It's actually not really supported to 
				// modify AgentRadius directly on navmesh instance, since such
				// a navmesh will get discarded during navmesh registration with
				// the navigation system. 
				// @todo consider hiding it (we might already have a ticket for that).
				UE_LOGF(LogNavigation, Warning, "Changing AgentRadius directly on RecastNavMesh instance is unsupported. Please use Project Settings > NavigationSystem > SupportedAgents to change AgentRadius");
			}
			else if (PropName == GET_MEMBER_NAME_CHECKED(ARecastNavMesh, TileSizeUU))
			{
				SetCellSize(ENavigationDataResolution::Default, UE::NavMesh::Private::GetClampedCellSize(GetCellSize(ENavigationDataResolution::Default)));
				TileSizeUU = UE::NavMesh::Private::GetClampedTileSizeUU(TileSizeUU, GetCellSize(ENavigationDataResolution::Default), AgentRadius);

				// Match cell sizes to tile size.
				for (uint8 Index = 0; Index < (uint8)ENavigationDataResolution::MAX; Index++)
				{
					const int32 CellCount = FMath::RoundToInt(TileSizeUU / GetCellSize((ENavigationDataResolution)Index));
					const float NewCellSize = UE::NavMesh::Private::GetClampedCellSize(TileSizeUU / (float)CellCount);
					SetCellSize((ENavigationDataResolution)Index, NewCellSize);
				}

				// update config
				FillConfig(NavDataConfig);
			}
			else if (PropName == NAME_TileNumberHardLimit)
			{
				TileNumberHardLimit = 1 << (FMath::CeilToInt(FMath::Log2(static_cast<float>(TileNumberHardLimit))));
				UpdatePolyRefBitsPreview();
			}

			if (CanUpdateNavMeshInEditor()
				&& PropName != GET_MEMBER_NAME_CHECKED(ARecastNavMesh, MaxSimultaneousTileGenerationJobsCount))
			{
				RebuildAll();
			}
		}
		else if (CategoryName == NAME_Display)
		{
			RequestDrawingUpdate();
		}
		else if (PropertyChangedEvent.Property->GetFName() == NAME_RuntimeGeneration)
		{
			// @todo this contraption is required to clear RuntimeGeneration value in DefaultEngine.ini
			// if it gets set to its default value (UE-23762). This is hopefully a temporary solution
			// since it's an Core-level issue (UE-23873).
			if (RuntimeGeneration == ERuntimeGenerationType::Static)
			{
				const FString EngineIniFilename = FPaths::ConvertRelativePathToFull(GetDefault<UEngine>()->GetDefaultConfigFilename());
				GConfig->SetString(TEXT("/Script/NavigationSystem.RecastNavMesh"), *NAME_RuntimeGeneration.ToString(), TEXT("Static"), *EngineIniFilename);
				GConfig->Flush(false);
			}
		}
		else if (CategoryName == NAME_Query)
		{
			RecreateDefaultFilter();
		}
	}
}

void ARecastNavMesh::PostEditUndo()
{
	// ANavigationData will register again this navdata in Super::PostEditUndo(). 
    // Since it was removed, we need to recreate it's RecastNavMeshImpl.
	if (RecastNavMeshImpl == nullptr)
	{
		RecastNavMeshImpl = new FPImplRecastNavMesh(this);
	}
	
	Super::PostEditUndo();
}

void ARecastNavMesh::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);

	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		// (Note, that this doesn't handle property chains at all. It can also match identical property names in a category outside "Generation", which is undesirable.)

		if (CanUpdateNavMeshInEditor())
		{
			struct FNavMeshGenerationPropertyNames
			{
				FNavMeshGenerationPropertyNames()
				{
					static const FName NAME_Generation = FName("Generation");

					// Assume almost all matches in the "Generation" category will trigger a rebuild.

					for (TFieldIterator<FProperty> PropertyIt(ARecastNavMesh::StaticClass()); PropertyIt; ++PropertyIt)
					{
						const FProperty* Property = *PropertyIt;
						const FName CategoryName = FObjectEditorUtils::GetCategoryFName(Property);
						if (CategoryName != NAME_Generation)
							continue;

						const FName PropName = Property->GetFName();
						if (PropName != GET_MEMBER_NAME_CHECKED(ARecastNavMesh, MaxSimultaneousTileGenerationJobsCount))
							ValidProperties.Add(PropName);
					}

				}

				TArray<FName> ValidProperties;
			};
			static FNavMeshGenerationPropertyNames BuiltPropertyNames;

			const TArray<FName>& PropertiesChanged = TransactionEvent.GetChangedProperties();

			const bool bRebuildNavMesh = PropertiesChanged.ContainsByPredicate([](const FName Name)
				{
					return BuiltPropertyNames.ValidProperties.Contains(Name);
				});

			if (bRebuildNavMesh)
			{
				// update config
				FillConfig(NavDataConfig);

				RebuildAll();
			}
		}
	}
}


#endif // WITH_EDITOR

bool ARecastNavMesh::NeedsRebuild() const
{
	bool bLooksLikeNeeded = !RecastNavMeshImpl || RecastNavMeshImpl->GetRecastMesh() == 0 || bHasNoTileData;
	if (NavDataGenerator.IsValid())
	{
		return bLooksLikeNeeded || NavDataGenerator->GetNumRemaningBuildTasks() > 0;
	}

	return bLooksLikeNeeded;
}

bool ARecastNavMesh::SupportsRuntimeGeneration() const
{
	// Generator should be disabled for Static navmesh
	return (RuntimeGeneration != ERuntimeGenerationType::Static) || RequiresInitialRebuild();
}

bool ARecastNavMesh::SupportsStreaming() const
{
	// Actually nothing prevents us to support streaming with dynamic generation
	// Right now streaming in sub-level causes navmesh to build itself, so no point to stream tiles in
	return (RuntimeGeneration != ERuntimeGenerationType::Dynamic) || bIsWorldPartitioned;
}

bool ARecastNavMesh::IsWorldPartitionedDynamicNavmesh() const
{
	return bIsWorldPartitioned && SupportsRuntimeGeneration();
}

FRecastNavMeshGenerator* ARecastNavMesh::CreateGeneratorInstance()
{
	return new FRecastNavMeshGenerator(*this);
}

bool ARecastNavMesh::IsUsingActiveTilesGeneration(const UNavigationSystemV1& NavSys) const
{
	return SupportsRuntimeGeneration() && (NavSys.IsActiveTilesGenerationEnabled() || bIsWorldPartitioned);
}

void ARecastNavMesh::PostLoadPreRebuild()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Deprecated
	// If any, unregister previous generated links proxy.
	UnregisterGeneratedLinksProxy();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	for (FNavLinkGenerationJumpConfig& NavLinkJumpConfig : NavLinkJumpConfigs)
	{
		// If any, unregister previous generated links proxy.
		UnregisterGeneratedLinksProxy(NavLinkJumpConfig);
		
		NavLinkJumpConfig.LinkProxy = nullptr;

		// Register new one if needed.
		if (NavLinkJumpConfig.bEnabled && NavLinkJumpConfig.LinkProxyClass.Get() != nullptr)
		{
			// Since this is on rebuild, create a new proxy id.
			CreateAndRegisterJumpLinksProxy(NavLinkJumpConfig);
		}
	}
}

void ARecastNavMesh::ConditionalConstructGenerator()
{	
	if (NavDataGenerator.IsValid())
	{
		NavDataGenerator->CancelBuild();
		NavDataGenerator.Reset();
	}

	UWorld* World = GetWorld();
	check(World);
	const bool bRequiresGenerator = SupportsRuntimeGeneration() || !World->IsGameWorld();
	if (bRequiresGenerator)
	{
		FRecastNavMeshGenerator* Generator = CreateGeneratorInstance();
		if (Generator)
		{
			NavDataGenerator = MakeShareable((FNavDataGenerator*)Generator);
			Generator->Init();
		}

		UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);
		if (NavSys)
		{
			RestrictBuildingToActiveTiles(IsUsingActiveTilesGeneration(*NavSys));
		}
	}
}

bool ARecastNavMesh::IsVoxelCacheEnabled()
{
#if RECAST_ASYNC_REBUILDING
	// voxel cache is using static buffers to minimize memory impact
	// therefore it can run only with synchronous navmesh rebuilds
	return false;
#else
	ARecastNavMesh* DefOb = (ARecastNavMesh*)ARecastNavMesh::StaticClass()->GetDefaultObject();
	return DefOb && DefOb->bUseVoxelCache;
#endif
}

const FRecastQueryFilter* ARecastNavMesh::GetNamedFilter(ERecastNamedFilter::Type FilterType)
{
	check(FilterType < ERecastNamedFilter::NamedFiltersCount); 
	return NamedFilters[FilterType];
}

#undef INITIALIZE_NAVQUERY

void ARecastNavMesh::UpdateNavObject()
{
	OnNavMeshUpdate.Broadcast();
}
#endif	//WITH_RECAST

bool ARecastNavMesh::HasValidNavmesh() const
{
#if WITH_RECAST
	return (RecastNavMeshImpl && RecastNavMeshImpl->DetourNavMesh && RecastNavMeshImpl->DetourNavMesh->isEmpty() == false);
#else
	return false;
#endif // WITH_RECAST
}

#if WITH_RECAST
bool ARecastNavMesh::HasCompleteDataInTile(const int32 TileX, const int32 TileY) const
{
	check(RecastNavMeshImpl && RecastNavMeshImpl->DetourNavMesh);
	const int32 NumTiles = RecastNavMeshImpl->DetourNavMesh->getTileCountAt(TileX, TileY);
	if (NumTiles <= 0)
	{
		const bool bHasFailsafeData = bStoreEmptyTileLayers && RecastNavMeshImpl->HasTileCacheLayers(TileX, TileY);
		if (!bHasFailsafeData)
		{
			return false;
		}
	}

	return true;
}

bool ARecastNavMesh::HasCompleteDataInRadius(const FVector& TestLocation, FVector::FReal TestRadius) const
{
	if (HasValidNavmesh() == false)
	{
		return false;
	}

	const dtNavMesh* NavMesh = RecastNavMeshImpl->DetourNavMesh;
	const dtNavMeshParams* NavParams = RecastNavMeshImpl->DetourNavMesh->getParams();
	const float NavTileSize = GetTileSizeUU();
	const FVector RcNavOrigin(NavParams->orig[0], NavParams->orig[1], NavParams->orig[2]);

	const FBox RcBounds = Unreal2RecastBox(FBox::BuildAABB(TestLocation, FVector(TestRadius, TestRadius, 0)));
	const FVector RcTestLocation = Unreal2RecastPoint(TestLocation);

	const int32 MinTileX = IntCastChecked<int32>(FMath::FloorToInt((RcBounds.Min.X - RcNavOrigin.X) / NavTileSize));
	const int32 MaxTileX = IntCastChecked<int32>(FMath::CeilToInt((RcBounds.Max.X - RcNavOrigin.X) / NavTileSize));
	const int32 MinTileY = IntCastChecked<int32>(FMath::FloorToInt((RcBounds.Min.Z - RcNavOrigin.Z) / NavTileSize));
	const int32 MaxTileY = IntCastChecked<int32>(FMath::CeilToInt((RcBounds.Max.Z - RcNavOrigin.Z) / NavTileSize));
	const FVector RcTileExtent2D(NavTileSize * 0.5f, 0.f, NavTileSize * 0.5f);
	const FVector::FReal RadiusSq = FMath::Square(TestRadius);

	for (int32 TileX = MinTileX; TileX <= MaxTileX; TileX++)
	{
		for (int32 TileY = MinTileY; TileY <= MaxTileY; TileY++)
		{
			const FVector RcTileCenter(
				RcNavOrigin.X + ((static_cast<FVector::FReal>(TileX) + 0.5) * NavTileSize),
				RcTestLocation.Y,
				RcNavOrigin.Z + ((static_cast<FVector::FReal>(TileY) + 0.5) * NavTileSize));

			const bool bInside = FMath::SphereAABBIntersection(RcTestLocation, RadiusSq, FBox::BuildAABB(RcTileCenter, RcTileExtent2D));
			if (bInside)
			{
				if (!HasCompleteDataInTile(TileX, TileY))
				{
					return false;
				}
			}
		}
	}

	return true;
}

bool ARecastNavMesh::HasCompleteDataAroundSegment(const FVector& StartLocation, const FVector& EndLocation, FVector::FReal TestRadius) const
{
	if (HasValidNavmesh() == false)
	{
		return false;
	}

	const dtNavMeshParams* NavParams = RecastNavMeshImpl->DetourNavMesh->getParams();
	const float NavTileSize = GetTileSizeUU();
	const FVector RcNavOrigin(NavParams->orig[0], NavParams->orig[1], NavParams->orig[2]);

	FBox CapsuleBox(ForceInit);
	CapsuleBox += StartLocation;
	CapsuleBox += EndLocation;
	CapsuleBox = CapsuleBox.ExpandBy(FVector(TestRadius, TestRadius, 0));
	const FRcTileBox TileBox(CapsuleBox, RcNavOrigin, NavTileSize);

	const FVector RcStartLocation = Unreal2RecastPoint(StartLocation);
	const FVector RcEndLocation = Unreal2RecastPoint(EndLocation);
	const UE::Geometry::TSegment2<FVector::FReal> Segment (FVector2D(RcStartLocation.X, RcStartLocation.Z), FVector2D(RcEndLocation.X, RcEndLocation.Z));
	const FVector::FReal TestRadiusSquared = FMath::Square(TestRadius);

	for (int32 TileX = TileBox.XMin; TileX <= TileBox.XMax; TileX++)
	{
		for (int32 TileY = TileBox.YMin; TileY <= TileBox.YMax; TileY++)
		{
			const FVector2D RcTileMin(RcNavOrigin.X + (static_cast<float>(TileX) * NavTileSize), RcNavOrigin.Z + (static_cast<float>(TileY) * NavTileSize));
			const FBox2D RcTileAABB = FBox2D(RcTileMin, RcTileMin + FVector2D(NavTileSize, NavTileSize));

			UE::Geometry::TDistSegment2AxisAlignedBox2<FVector::FReal> SegmentTileDist(Segment, RcTileAABB);
			const FVector::FReal DistanceSquared = SegmentTileDist.GetSquared();
			if (DistanceSquared <= TestRadiusSquared)
			{
				if (!HasCompleteDataInTile(TileX, TileY))
				{
					return false;
				}
			}
		}
	}

	return true;
}

//----------------------------------------------------------------------//
// RecastNavMesh: Active Tiles 
//----------------------------------------------------------------------//
void ARecastNavMesh::UpdateActiveTiles(const TArray<FNavigationInvokerRaw>& InvokerLocations)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ARecastNavMesh::UpdateActiveTiles);
	
	if (HasValidNavmesh() == false)
	{
		return;
	}

	const FRecastNavMeshGenerator* MyGenerator = static_cast<FRecastNavMeshGenerator*>(GetGenerator());
	if (MyGenerator == nullptr)
	{
		return;
	}

	const dtNavMeshParams* NavParams = GetRecastNavMeshImpl()->DetourNavMesh->getParams();
	check(NavParams && MyGenerator);
	const FRecastBuildConfig& Config = MyGenerator->GetConfig();
	const FVector NavmeshOrigin = Recast2UnrealPoint(NavParams->orig);
	const FVector::FReal TileDim = Config.GetTileSizeUU();
	if (!ensureMsgf(TileDim > 0, TEXT("TileSize can't be 0. Check the navmesh configuration.")))
	{
		return;
	}

	TSet<FIntPoint>& OldActiveSet = UpdateActiveTilesWorkingMem.OldActiveSet;
	TMap<FIntPoint, FNavMeshDirtyTileElement>& TilesInMinDistance = UpdateActiveTilesWorkingMem.TilesInMinDistanceMap;
	TSet<FIntPoint>& TilesInMaxDistance = UpdateActiveTilesWorkingMem.TilesInMaxDistance;
	TArray<FIntPoint>& TileToAppend = UpdateActiveTilesWorkingMem.TileToAppend;
	
	TSet<FIntPoint>& ActiveTiles = GetActiveTileSet();
	const int32 ActiveTilesCount = ActiveTiles.Num();
	OldActiveSet = ActiveTiles;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ARecastNavMesh::Reserving);
		TilesInMinDistance.Reset();
		TilesInMaxDistance.Reset();
		TileToAppend.Reset();

		const int32 ShrinkThreshold = static_cast<int32>(1.2 * ActiveTilesCount);
		if (TilesInMinDistance.GetMaxIndex() > ShrinkThreshold)
		{
			TilesInMinDistance.Shrink();
		}
		if (TilesInMaxDistance.GetMaxIndex() > ShrinkThreshold)
		{
			TilesInMaxDistance.Shrink();
		}
		if (TileToAppend.Max() > ShrinkThreshold)
		{
			TileToAppend.Shrink();
		}
		
		TilesInMinDistance.Reserve(ActiveTilesCount);
		TilesInMaxDistance.Reserve(ActiveTilesCount);
		TileToAppend.Reserve(ActiveTilesCount);
	}

	ActiveTiles.Reset();

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ARecastNavMesh::MinMaxDistance);
		for (const FNavigationInvokerRaw& Invoker : InvokerLocations)
		{
			if (!Invoker.SupportedAgents.Contains(Config.AgentIndex))
			{
				continue;
			}

			const FVector InvokerRelativeLocation = (NavmeshOrigin - Invoker.Location);
			const  FVector::FReal TileCenterDistanceToRemoveSq = FMath::Square(TileDim * UE_SQRT_2 / 2 + Invoker.RadiusMax);
			const  FVector::FReal TileCenterDistanceToAddSq = FMath::Square(TileDim * UE_SQRT_2 / 2 + Invoker.RadiusMin);

			const int64 MinX = FMath::FloorToInt((InvokerRelativeLocation.X - Invoker.RadiusMax) / TileDim);
			const int64 MaxX = FMath::CeilToInt((InvokerRelativeLocation.X + Invoker.RadiusMax) / TileDim);
			const int64 MinY = FMath::FloorToInt((InvokerRelativeLocation.Y - Invoker.RadiusMax) / TileDim);
			const int64 MaxY = FMath::CeilToInt((InvokerRelativeLocation.Y + Invoker.RadiusMax) / TileDim);
			ensureMsgf(IntFitsIn<int32>(MinX), TEXT("MinX %lld is out of tile coordinate range. Location: %f, Radius %f, TileSize: %f"), MinX, InvokerRelativeLocation.X, Invoker.RadiusMax, TileDim);
			ensureMsgf(IntFitsIn<int32>(MaxX), TEXT("MaxX %lld is out of tile coordinate range. Location: %f, Radius %f, TileSize: %f"), MaxX, InvokerRelativeLocation.X, Invoker.RadiusMax, TileDim);
			ensureMsgf(IntFitsIn<int32>(MinY), TEXT("MinY %lld is out of tile coordinate range. Location: %f, Radius %f, TileSize: %f"), MinY, InvokerRelativeLocation.Y, Invoker.RadiusMax, TileDim);
			ensureMsgf(IntFitsIn<int32>(MaxY), TEXT("MaxY %lld is out of tile coordinate range. Location: %f, Radius %f, TileSize: %f"), MaxY, InvokerRelativeLocation.Y, Invoker.RadiusMax, TileDim);
			
			const int32 MinTileX = static_cast<int32>(FMath::Clamp(MinX, TNumericLimits<int32>::Min(), TNumericLimits<int32>::Max()));
			const int32 MaxTileX = static_cast<int32>(FMath::Clamp(MaxX, TNumericLimits<int32>::Min(), TNumericLimits<int32>::Max()));
			const int32 MinTileY = static_cast<int32>(FMath::Clamp(MinY, TNumericLimits<int32>::Min(), TNumericLimits<int32>::Max()));
			const int32 MaxTileY = static_cast<int32>(FMath::Clamp(MaxY, TNumericLimits<int32>::Min(), TNumericLimits<int32>::Max()));

			for (int32 X = MinTileX; X <= MaxTileX; ++X)
			{
				for (int32 Y = MinTileY; Y <= MaxTileY; ++Y)
				{
					const FVector::FReal DistanceSq = (InvokerRelativeLocation - FVector(X * TileDim + TileDim / 2, Y * TileDim + TileDim / 2, 0.f)).SizeSquared2D();
					if (DistanceSq < TileCenterDistanceToRemoveSq)
					{
						TilesInMaxDistance.FindOrAdd(FIntPoint(X, Y));

						if (DistanceSq < TileCenterDistanceToAddSq)
						{
							// Add unique tile 
							FNavMeshDirtyTileElement* FoundTile = TilesInMinDistance.Find(FIntPoint(X, Y));
							if (FoundTile)
							{
								// Update the priority if already existing
								FoundTile->InvokerPriority = FMath::Max(FoundTile->InvokerPriority, Invoker.Priority);
							}
							else
							{
								TilesInMinDistance.Add(FIntPoint(X, Y), FNavMeshDirtyTileElement{FIntPoint(X,Y), DistanceSq, Invoker.Priority});
								TileToAppend.Add(FIntPoint(X,Y));
							}
						}
					}
				}
			}
		}
	}

	ActiveTiles.Append(TileToAppend);

	TArray<FIntPoint> TilesToRemove;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ARecastNavMesh::CategorizeTiles);

		TilesToRemove.Reserve(OldActiveSet.Num());
		for(TSet<FIntPoint>::TIterator TileIt = OldActiveSet.CreateIterator(); TileIt; ++TileIt)
		{
			if(!TilesInMaxDistance.Contains(*TileIt))
			{
				TilesToRemove.Add(*TileIt);
				TileIt.RemoveCurrent();
			}
			else
			{
				ActiveTiles.FindOrAdd(*TileIt);
			}
		}
	}

	// Find tiles to update
	TArray<FNavMeshDirtyTileElement> TilesToUpdate;
	TilesToUpdate.Reserve(ActiveTiles.Num());
	for (const TTuple<FIntPoint, FNavMeshDirtyTileElement>& Pair : TilesInMinDistance)
	{
		// Check if it's a new tile (not in the active set)
		const FNavMeshDirtyTileElement& Tile = Pair.Value;
		if (!OldActiveSet.Contains(Tile.Coordinates))
		{
			TilesToUpdate.Add(Tile);
		}
	}

	UE_SUPPRESS(LogNavigation, Log,
	{
		if (TilesToRemove.Num() != 0 || TilesToUpdate.Num() != 0)
		{
			UE_VLOG(this, LogNavigation, Log, TEXT("Updating active tiles: %d to remove, %d to update"), TilesToRemove.Num(), TilesToUpdate.Num());
		}
	});

	RemoveTiles(TilesToRemove);
	RebuildTile(TilesToUpdate);

	if (TilesToRemove.Num() > 0 || TilesToUpdate.Num() > 0)
	{
		UpdateNavMeshDrawing();
	}
}

void ARecastNavMesh::RemoveTiles(const TArray<FIntPoint>& Tiles)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ARecastNavMesh::RemoveTiles);
	
	if (Tiles.Num() > 0)
	{
		FRecastNavMeshGenerator* MyGenerator = static_cast<FRecastNavMeshGenerator*>(GetGenerator());
		if (MyGenerator)
		{
			MyGenerator->RemoveTiles(Tiles);
		}
	}
}

void ARecastNavMesh::RebuildTile(const TArray<FNavMeshDirtyTileElement>& Tiles)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ARecastNavMesh::RebuildTile);
	
	if (Tiles.Num() > 0)
	{
		FRecastNavMeshGenerator* MyGenerator = static_cast<FRecastNavMeshGenerator*>(GetGenerator());
		if (MyGenerator)
		{
			MyGenerator->ReAddTiles(Tiles);
		}
	}
}

void ARecastNavMesh::DirtyTilesInBounds(const FBox& Bounds)
{
	if (!ensureMsgf(Bounds.IsValid, TEXT("%hs Bounds is not valid"), __FUNCTION__))
	{
		return;
	}

	if (HasValidNavmesh() == false)
	{
		return;
	}
	
	FRecastNavMeshGenerator* MyGenerator = static_cast<FRecastNavMeshGenerator*>(GetGenerator());
	if (MyGenerator)
	{
		MyGenerator->DiscardCurrentBuildingTasks();
	}
	
	const dtNavMesh* DetourNavMesh = RecastNavMeshImpl->GetRecastMesh();

	// Remove all tiles
	const int32 TileCount = GetNavMeshTilesCount();
	TArray<FIntPoint> TilesToRemove;
	TilesToRemove.Reserve(TileCount);
	for (int32 TileIndex = 0; TileIndex < TileCount; ++TileIndex)
	{
		const dtMeshTile* Tile = DetourNavMesh->getTile(TileIndex);
		const dtMeshHeader* Header = Tile != nullptr ? Tile->header : nullptr;
		if (Header)
		{
			TilesToRemove.Add(FIntPoint(Header->x, Header->y));
		}
	}
	RemoveTiles(TilesToRemove);

	const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	const FBox OverlappingBounds = Bounds.Overlap(NavSys->GetWorldBounds());

	if (OverlappingBounds.IsValid)
	{
		// Add tiles within the overlapping bounds
		TArray<FNavMeshDirtyTileElement> Tiles;
		const FVector RcNavMeshOrigin = Unreal2RecastPoint(NavMeshOriginOffset);
		const float TileSizeInWorldUnits = GetTileSizeUU();
		const FRcTileBox TileBox(OverlappingBounds, RcNavMeshOrigin, TileSizeInWorldUnits);

		UE_LOGF(LogNavigation, VeryVerbose, "RebuildTilesFromBounds %i tiles: (%i,%i) to (%i,%i)", (TileBox.XMax-TileBox.XMin)*(TileBox.YMax-TileBox.YMin), TileBox.XMin, TileBox.YMin, TileBox.XMax, TileBox.YMax);

		for (int32 TileY = TileBox.YMin; TileY <= TileBox.YMax; ++TileY)
		{
			for (int32 TileX = TileBox.XMin; TileX <= TileBox.XMax; ++TileX)
			{
				// For now, new dirtiness is made with default priority.
				Tiles.Add(FNavMeshDirtyTileElement{FIntPoint(TileX, TileY), TNumericLimits<FVector::FReal>::Max(), ENavigationInvokerPriority::Default});
			}
		}
		RebuildTile(Tiles);
	}
}

#if RECAST_INTERNAL_DEBUG_DATA
const TMap<FIntPoint, struct FRecastInternalDebugData>* ARecastNavMesh::GetDebugDataMap() const
{
	if (RecastNavMeshImpl)
	{
		return &RecastNavMeshImpl->DebugDataMap;
	}
	return nullptr;
}
#endif //RECAST_INTERNAL_DEBUG_DATA

//----------------------------------------------------------------------//
// FRecastNavMeshCachedData
//----------------------------------------------------------------------//

FRecastNavMeshCachedData FRecastNavMeshCachedData::Construct(const ARecastNavMesh* RecastNavMeshActor)
{
	check(RecastNavMeshActor);
	
	FRecastNavMeshCachedData CachedData;

	CachedData.ActorOwner = RecastNavMeshActor;
	// create copies from crucial ARecastNavMesh data
	CachedData.bUseSortFunction = RecastNavMeshActor->bSortNavigationAreasByCost;

	TArray<FSupportedAreaData> Areas;
	RecastNavMeshActor->GetSupportedAreas(Areas);
	FMemory::Memzero(CachedData.FlagsPerArea, sizeof(ARecastNavMesh::FNavPolyFlags) * RECAST_MAX_AREAS);

	for (int32 i = 0; i < Areas.Num(); i++)
	{
		const UClass* AreaClass = Areas[i].AreaClass;
		const UNavArea* DefArea = AreaClass ? ((UClass*)AreaClass)->GetDefaultObject<UNavArea>() : NULL;
		if (DefArea)
		{
			CachedData.AreaClassToIdMap.Add(AreaClass, Areas[i].AreaID);
			CachedData.FlagsPerArea[Areas[i].AreaID] = DefArea->GetAreaFlags();
		}
	}

	FMemory::Memcpy(CachedData.FlagsPerOffMeshLinkArea, CachedData.FlagsPerArea, sizeof(CachedData.FlagsPerArea));
	static const ARecastNavMesh::FNavPolyFlags NavLinkFlag = ARecastNavMesh::GetNavLinkFlag();
	if (NavLinkFlag != 0)
	{
		ARecastNavMesh::FNavPolyFlags* AreaFlag = CachedData.FlagsPerOffMeshLinkArea;
		for (int32 AreaIndex = 0; AreaIndex < RECAST_MAX_AREAS; ++AreaIndex, ++AreaFlag)
		{
			*AreaFlag |= NavLinkFlag;
		}
	}

	return CachedData;
}

void FRecastNavMeshCachedData::OnAreaAdded(const UClass* AreaClass, int32 AreaID)
{
	const UNavArea* DefArea = AreaClass ? ((UClass*)AreaClass)->GetDefaultObject<UNavArea>() : NULL;
	if (DefArea && AreaID >= 0)
	{
		AreaClassToIdMap.Add(AreaClass, AreaID);
		FlagsPerArea[AreaID] = DefArea->GetAreaFlags();

		static const ARecastNavMesh::FNavPolyFlags NavLinkFlag = ARecastNavMesh::GetNavLinkFlag();
		if (NavLinkFlag != 0)
		{
			FlagsPerOffMeshLinkArea[AreaID] = FlagsPerArea[AreaID] | NavLinkFlag;
		}
	}		
}

void FRecastNavMeshCachedData::OnAreaRemoved(const UClass* AreaClass)
{
	const int32* AreaID = AreaClass ? AreaClassToIdMap.Find(AreaClass) : nullptr;

	if (AreaID != nullptr)
	{
		AreaClassToIdMap.Remove(AreaClass);

		FlagsPerArea[*AreaID] = 0;
		FlagsPerOffMeshLinkArea[*AreaID] = 0;
	}
}

FNavLinkId ARecastNavMesh::GetNavLinkUserId(NavNodeRef LinkPolyID) const
{
	return RecastNavMeshImpl ? RecastNavMeshImpl->GetNavLinkUserId(LinkPolyID) : FNavLinkId::Invalid;
}

dtNavMesh* ARecastNavMesh::GetRecastMesh()
{
	return RecastNavMeshImpl ? RecastNavMeshImpl->GetRecastMesh() : nullptr;
}

const dtNavMesh* ARecastNavMesh::GetRecastMesh() const
{
	return RecastNavMeshImpl ? RecastNavMeshImpl->GetRecastMesh() : nullptr;
}
#endif// WITH_RECAST

//----------------------------------------------------------------------//
// BP API
//----------------------------------------------------------------------//
bool ARecastNavMesh::K2_ReplaceAreaInTileBounds(FBox Bounds, TSubclassOf<UNavArea> OldArea, TSubclassOf<UNavArea> NewArea, bool ReplaceLinks)
{
	if (!ensureMsgf(Bounds.IsValid, TEXT("%hs Attempting to use Bounds which are not valid"), __FUNCTION__))
	{
		return false;
	}

	bool bReplaced = false;
#if WITH_RECAST
	bReplaced = ReplaceAreaInTileBounds(Bounds, OldArea, NewArea, ReplaceLinks) > 0;
	if (bReplaced)
	{
		RequestDrawingUpdate();
	}
#endif // WITH_RECAST
	return bReplaced;
}

