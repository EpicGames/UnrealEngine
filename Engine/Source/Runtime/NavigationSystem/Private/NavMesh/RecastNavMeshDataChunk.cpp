// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavMesh/RecastNavMeshDataChunk.h"
#include "Engine/World.h"
#include "NavigationSystem.h"
#include "NavMesh/RecastNavMesh.h"
#include "NavMesh/PImplRecastNavMesh.h"
#include "NavMesh/RecastHelpers.h"
#include "NavMesh/RecastVersion.h"
#include "NavMesh/RecastNavMeshGenerator.h"

#if WITH_RECAST
#include "Detour/DetourNavMeshBuilder.h"
#endif // WITH_RECAST

#include UE_INLINE_GENERATED_CPP_BY_NAME(RecastNavMeshDataChunk)

//----------------------------------------------------------------------//
// FRecastTileData                                                                
//----------------------------------------------------------------------//
FRecastTileData::FRawData::FRawData(uint8* InData)
	: RawData(InData)
{
}

FRecastTileData::FRawData::~FRawData()
{
#if WITH_RECAST
	dtFree(RawData, DT_ALLOC_PERM_TILE_DATA);
#else
	FMemory::Free(RawData);
#endif
}

FRecastTileData::FRecastTileData()
	: OriginalX(0)
	, OriginalY(0)
	, X(0)
	, Y(0)
	, Layer(0)
	, TileDataSize(0)
	, TileCacheDataSize(0)
	, bAttached(false)
{
}

FRecastTileData::FRecastTileData(int32 DataSize, uint8* RawData, int32 CacheDataSize, uint8* CacheRawData)
	: OriginalX(0)
	, OriginalY(0)
	, X(0)
	, Y(0)
	, Layer(0)
	, TileDataSize(DataSize)
	, TileCacheDataSize(CacheDataSize)
	, bAttached(false)
{
	TileRawData = MakeShareable(new FRawData(RawData));
	TileCacheRawData = MakeShareable(new FRawData(CacheRawData));
}

// Helper to duplicate recast raw data
static uint8* DuplicateRecastRawData(const uint8* Src, int32 SrcSize)
{
#if WITH_RECAST	
	uint8* DupData = (uint8*)dtAlloc(SrcSize, DT_ALLOC_PERM_TILE_DATA);
#else
	uint8* DupData = (uint8*)FMemory::Malloc(SrcSize);
#endif
	FMemory::Memcpy(DupData, Src, SrcSize);
	return DupData;
}

namespace UE::NavMesh::Private
{
	bool IsUsingActiveTileGeneration(const ARecastNavMesh& NavMesh)
	{
#if WITH_RECAST
		const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(NavMesh.GetWorld());
		if (NavSys)
		{
			return NavMesh.IsUsingActiveTilesGeneration(*NavSys);
		}
#endif // WITH_RECAST
		return false;
	}
} // namespace UE::NavMesh::Private

//----------------------------------------------------------------------//
// URecastNavMeshDataChunk                                                                
//----------------------------------------------------------------------//
URecastNavMeshDataChunk::URecastNavMeshDataChunk(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void URecastNavMeshDataChunk::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	int32 NavMeshVersion = NAVMESHVER_LATEST;
	Ar << NavMeshVersion;

	// when writing, write a zero here for now.  will come back and fill it in later.
	int64 RecastNavMeshSizeBytes = 0;
	int64 RecastNavMeshSizePos = Ar.Tell();
	Ar << RecastNavMeshSizeBytes;

	if (Ar.IsLoading())
	{
		auto CleanUpBadVersion = [&Ar, RecastNavMeshSizePos, RecastNavMeshSizeBytes]()
		{
			// incompatible, just skip over this data. Navmesh needs rebuilt.
			Ar.Seek(RecastNavMeshSizePos + RecastNavMeshSizeBytes);
		};

		if (NavMeshVersion < NAVMESHVER_MIN_COMPATIBLE)
		{
			UE_LOGF(LogNavigation, Warning, "%ls: URecastNavMeshDataChunk: Nav mesh version %d < Min compatible %d. Nav mesh needs to be rebuilt. \n", *GetFullName(), NavMeshVersion, NAVMESHVER_MIN_COMPATIBLE);

			CleanUpBadVersion();
		}
		else if (NavMeshVersion > NAVMESHVER_LATEST)
		{
			UE_LOGF(LogNavigation, Warning, "%ls: URecastNavMeshDataChunk: Nav mesh version %d > NAVMESHVER_LATEST %d. Newer nav mesh should not be loaded by older versioned code. At a minimum the nav mesh needs to be rebuilt. \n", *GetFullName(), NavMeshVersion, NAVMESHVER_LATEST);

			CleanUpBadVersion();
		}
#if WITH_RECAST
		else if (RecastNavMeshSizeBytes > 4)
		{
			SerializeRecastData(Ar, NavMeshVersion);
		}
#endif// WITH_RECAST
		else
		{
			// empty, just skip over this data
			Ar.Seek(RecastNavMeshSizePos + RecastNavMeshSizeBytes);
		}
	}
	else if (Ar.IsSaving())
	{
#if WITH_RECAST
		SerializeRecastData(Ar, NavMeshVersion);
#endif// WITH_RECAST

		int64 CurPos = Ar.Tell();
		RecastNavMeshSizeBytes = CurPos - RecastNavMeshSizePos;
		Ar.Seek(RecastNavMeshSizePos);
		Ar << RecastNavMeshSizeBytes;
		Ar.Seek(CurPos);
	}
}

#if WITH_RECAST
void URecastNavMeshDataChunk::SerializeRecastData(FArchive& Ar, int32 NavMeshVersion)
{
	int32 TileNum = Tiles.Num();
	Ar << TileNum;

	if (Ar.IsLoading())
	{
		Tiles.Empty(TileNum);
		for (int32 TileIdx = 0; TileIdx < TileNum; TileIdx++)
		{
			int32 TileDataSize = 0;
			Ar << TileDataSize;

			// Load tile data 
			uint8* TileRawData = nullptr;
			FPImplRecastNavMesh::SerializeRecastMeshTile(Ar, NavMeshVersion, TileRawData, TileDataSize); //allocates TileRawData on load
			
			if (TileRawData != nullptr)
			{
				// Load compressed tile cache layer
				int32 TileCacheDataSize = 0;
				uint8* TileCacheRawData = nullptr;
				FPImplRecastNavMesh::SerializeCompressedTileCacheData(Ar, NavMeshVersion, TileCacheRawData, TileCacheDataSize); //allocates TileCacheRawData on load
				
				// We are owner of tile raw data
				FRecastTileData TileData(TileDataSize, TileRawData, TileCacheDataSize, TileCacheRawData);
				Tiles.Add(TileData);
			}
		}
	}
	else if (Ar.IsSaving())
	{
		for (FRecastTileData& TileData : Tiles)
		{
			if (TileData.TileRawData.IsValid())
			{
				// Save tile itself
				Ar << TileData.TileDataSize;
				FPImplRecastNavMesh::SerializeRecastMeshTile(Ar, NavMeshVersion, TileData.TileRawData->RawData, TileData.TileDataSize);
				// Save compressed tile cache layer
				FPImplRecastNavMesh::SerializeCompressedTileCacheData(Ar, NavMeshVersion, TileData.TileCacheRawData->RawData, TileData.TileCacheDataSize);
			}
		}
	}
}
#endif// WITH_RECAST

#if WITH_RECAST
TArray<FNavTileRef> URecastNavMeshDataChunk::AttachTiles(ARecastNavMesh& NavMesh)
{
	check(NavMesh.GetWorld());
	const bool bIsGameWorld = NavMesh.GetWorld()->IsGameWorld();

	// In editor we still need to own the data so a copy will be made.
	const bool bKeepCopyOfData = !bIsGameWorld;
	const bool bKeepCopyOfCacheData = !bIsGameWorld;

	return AttachTiles(NavMesh, bKeepCopyOfData, bKeepCopyOfCacheData);
}

TArray<FNavTileRef> URecastNavMeshDataChunk::AttachTiles(ARecastNavMesh& NavMesh, const bool bKeepCopyOfData, const bool bKeepCopyOfCacheData)
{
	UE_LOGF(LogNavigation, Verbose, "%ls Attaching to NavMesh - %ls", ANSI_TO_TCHAR(__FUNCTION__), *NavigationDataName.ToString());
	
	TArray<FNavTileRef> Result;
	Result.Reserve(Tiles.Num());

	dtNavMesh* DetourNavMesh = NavMesh.GetRecastMesh();

	if (DetourNavMesh != nullptr)
	{
		TSet<FIntPoint>* ActiveTiles = nullptr;
		if (UE::NavMesh::Private::IsUsingActiveTileGeneration(NavMesh))
		{
			ActiveTiles = &NavMesh.GetActiveTileSet();
			ActiveTiles->Reserve(ActiveTiles->Num() + Tiles.Num());
		}
		
		for (FRecastTileData& TileData : Tiles)
		{
			if (!TileData.bAttached && TileData.TileRawData.IsValid())
			{
				if (TileData.TileRawData->RawData == nullptr)
				{
					UE_LOGF(LogNavigation, Warning, "Null rawdata. This can be caused by the reuse of unloaded sublevels. 'LevelStreaming.ShouldReuseUnloadedButStillAroundLevels 0' can be used until this gets fixed.");
					continue;
				}
				
				const dtMeshHeader* Header = (dtMeshHeader*)TileData.TileRawData->RawData;
				if (Header->version != DT_NAVMESH_VERSION)
				{
					continue;
				}
				
				// If there was a previous tile at the location remove it
				if (const dtMeshTile* PreExistingTile = DetourNavMesh->getTileAt(Header->x, Header->y, Header->layer))
				{
					if (const dtTileRef PreExistingTileRef = DetourNavMesh->getTileRef(PreExistingTile))
					{
						NavMesh.LogRecastTile(ANSI_TO_TCHAR(__FUNCTION__), FName("   "), FName("removing"), *DetourNavMesh, Header->x, Header->y, Header->layer, PreExistingTileRef);
						
						DetourNavMesh->removeTile(PreExistingTileRef, nullptr, nullptr);	
					}
				}

				// Attach mesh tile to target nav mesh 
				dtTileRef TileRef = 0;
				const dtMeshTile* MeshTile = nullptr;

				dtStatus status = DetourNavMesh->addTile(TileData.TileRawData->RawData, TileData.TileDataSize, DT_TILE_FREE_DATA, 0, &TileRef);

				if (dtStatusFailed(status))
				{
					if (dtStatusDetail(status, DT_OUT_OF_MEMORY))
					{
						UE_LOGF(LogNavigation, Warning, 
							"%ls> Failed to add tile (%d,%d:%d), %d tile limit reached! (from: %s). Increase AverageLayersPerTile or, if using FixedTilePoolSize, increase the TilePoolSize or use bigger tiles.",
							*NavMesh.GetName(), Header->x, Header->y, Header->layer, DetourNavMesh->getMaxTiles(), __FUNCTION__);
					}
					
					continue;
				}
				else
				{
					MeshTile = DetourNavMesh->getTileByRef(TileRef);
					check(MeshTile);
					
					TileData.X = MeshTile->header->x;
					TileData.Y = MeshTile->header->y;
					TileData.Layer = MeshTile->header->layer;
					TileData.bAttached = true;
				}

				NavMesh.LogRecastTile(ANSI_TO_TCHAR(__FUNCTION__), FName("   "), FName("added"), *DetourNavMesh, TileData.X, TileData.Y, TileData.Layer, TileRef);
				
				if (ActiveTiles)
				{
					ActiveTiles->FindOrAdd(FIntPoint(TileData.X, TileData.Y));
				}
				
				if (bKeepCopyOfData == false)
				{
					// We don't own tile data anymore it will be released by recast navmesh 
					TileData.TileDataSize = 0;
					TileData.TileRawData->RawData = nullptr;
				}
				else
				{
					// In the editor we still need to own data, so make a copy of it
					TileData.TileRawData->RawData = DuplicateRecastRawData(TileData.TileRawData->RawData, TileData.TileDataSize);
				}

				// Attach tile cache layer to target nav mesh
				if (TileData.TileCacheDataSize > 0)
				{
					FBox TileBBox = Recast2UnrealBox(MeshTile->header->bmin, MeshTile->header->bmax);

					FNavMeshTileData LayerData(TileData.TileCacheRawData->RawData, TileData.TileCacheDataSize, TileData.Layer, TileBBox);
					NavMesh.GetRecastNavMeshImpl()->AddTileCacheLayer(TileData.X, TileData.Y, TileData.Layer, LayerData);

					if (bKeepCopyOfCacheData == false)
					{
						// We don't own tile cache data anymore it will be released by navmesh
						TileData.TileCacheDataSize = 0;
						TileData.TileCacheRawData->RawData = nullptr;
					}
					else
					{
						// In the editor we still need to own data, so make a copy of it
						TileData.TileCacheRawData->RawData = DuplicateRecastRawData(TileData.TileCacheRawData->RawData, TileData.TileCacheDataSize);
					}
				}

				Result.Add(FNavTileRef(TileRef));
			}
		}

#if WITH_NAVMESH_SEGMENT_LINKS
		// Create segment link connections now that all the tiles have been loaded.
		if (const FPImplRecastNavMesh* const NavMeshImpl = NavMesh.GetRecastNavMeshImpl())
		{
			for (int32 Index = 0; Index < Result.Num(); ++Index)
			{
				NavMeshImpl->ProcessSegmentLinksForTile(Result[Index]);
			}
		}
#endif // WITH_NAVMESH_SEGMENT_LINKS		
	}

	UE_LOGF(LogNavigation, Verbose, "Attached %d tiles to NavMesh - %ls", Result.Num(), *NavigationDataName.ToString());
	return Result;
}

TArray<FNavTileRef> URecastNavMeshDataChunk::DetachTiles(ARecastNavMesh& NavMesh)
{
	check(NavMesh.GetWorld());
	const bool bIsGameWorld = NavMesh.GetWorld()->IsGameWorld();

	// Keep data in game worlds (in editor we have a copy of the data so we don't keep it).
	const bool bTakeDataOwnership = bIsGameWorld;
	const bool bTakeCacheDataOwnership = bIsGameWorld;

	return DetachTiles(NavMesh, bTakeDataOwnership, bTakeCacheDataOwnership);
}

TArray<FNavTileRef> URecastNavMeshDataChunk::DetachTiles(ARecastNavMesh& NavMesh, const bool bTakeDataOwnership, const bool bTakeCacheDataOwnership)
{
	UE_LOGF(LogNavigation, Verbose, "%ls Detaching from %ls", ANSI_TO_TCHAR(__FUNCTION__), *NavigationDataName.ToString());

	TArray<FNavTileRef> Result;
	Result.Reserve(Tiles.Num());

	dtNavMesh* DetourNavMesh = NavMesh.GetRecastMesh();

	if (DetourNavMesh != nullptr)
	{
		TSet<FIntPoint>* ActiveTiles = nullptr;
		if (UE::NavMesh::Private::IsUsingActiveTileGeneration(NavMesh))
		{
			ActiveTiles = &NavMesh.GetActiveTileSet();
		}

		TArray<const dtMeshTile*> ExtraMeshTiles;
		check(NavMesh.GetWorld());
		const bool bIsGameWorld = NavMesh.GetWorld()->IsGameWorld();
		// Whether the navmesh is fully dynamic and supports rebuild from geometry. This allows for Dynamic Modifiers Only navmesh to take ownership of tiles on every layer when at the same XY location
		const bool bIsDynamic = bIsGameWorld && NavMesh.GetRuntimeGenerationMode() == ERuntimeGenerationType::Dynamic;
		
		for (FRecastTileData& TileData : Tiles)
		{
			if (TileData.bAttached)
			{
				// Detach tile cache layer and take ownership over compressed data
				dtTileRef TileRef = 0;
				const dtMeshTile* MeshTile = DetourNavMesh->getTileAt(TileData.X, TileData.Y, TileData.Layer);
				if (MeshTile)
				{
					TileRef = DetourNavMesh->getTileRef(MeshTile);

					if (bTakeCacheDataOwnership)
					{
						FNavMeshTileData TileCacheData = NavMesh.GetRecastNavMeshImpl()->GetTileCacheLayer(TileData.X, TileData.Y, TileData.Layer);
						if (TileCacheData.IsValid())
						{
							TileData.TileCacheDataSize = TileCacheData.DataSize;
							TileData.TileCacheRawData->RawData = TileCacheData.Release();
						}
					}

					NavMesh.LogRecastTile(ANSI_TO_TCHAR(__FUNCTION__), FName("   "), FName("removing"), *DetourNavMesh, TileData.X, TileData.Y, TileData.Layer, TileRef);
				
					NavMesh.GetRecastNavMeshImpl()->RemoveTileCacheLayer(TileData.X, TileData.Y, TileData.Layer);

					if (bTakeDataOwnership)
					{
						// Remove tile from navmesh and take ownership of tile raw data
						DetourNavMesh->removeTile(TileRef, &TileData.TileRawData->RawData, &TileData.TileDataSize);
					}
					else
					{
						// In the editor we have a copy of tile data so just release tile in navmesh
						DetourNavMesh->removeTile(TileRef, nullptr, nullptr);
					}

					if (ActiveTiles)
					{
						ActiveTiles->Remove(FIntPoint(TileData.X, TileData.Y));
					}
						
					Result.Add(FNavTileRef(TileRef));
				}

				if (bIsDynamic)
				{
					// Remove any tile remaining
					const int32 MaxTiles = DetourNavMesh->getTileCountAt(TileData.X, TileData.Y);
					if (MaxTiles > 0)
					{
						ExtraMeshTiles.SetNumZeroed(MaxTiles, EAllowShrinking::No);
						const int32 MeshTilesCount = DetourNavMesh->getTilesAt(TileData.X, TileData.Y, ExtraMeshTiles.GetData(), MaxTiles);
						for (int32 i = 0; i < MeshTilesCount; ++i)
						{
							const dtMeshTile* ExtraMeshTile = ExtraMeshTiles[i];
							dtTileRef ExtraTileRef = DetourNavMesh->getTileRef(ExtraMeshTile);
							if (ExtraTileRef)
							{
								DetourNavMesh->removeTile(ExtraTileRef, nullptr, nullptr);
								Result.Add(FNavTileRef(ExtraTileRef));
							}
						}
					}
				}
				
			}

			TileData.bAttached = false;
			TileData.X = 0;
			TileData.Y = 0;
			TileData.Layer = 0;
		}
	}

	UE_LOGF(LogNavigation, Verbose, "Detached %d tiles from NavMesh - %ls", Result.Num(), *NavigationDataName.ToString());
	return Result;
}
#endif // WITH_RECAST

void URecastNavMeshDataChunk::MoveTiles(FPImplRecastNavMesh& NavMeshImpl, const FIntPoint& Offset, const FVector::FReal RotationDeg, const FVector2D& RotationCenter)
{
#if WITH_RECAST	
	UE_LOGF(LogNavigation, Verbose, "%ls Moving %i tiles on navmesh %ls.", ANSI_TO_TCHAR(__FUNCTION__), Tiles.Num(), *NavigationDataName.ToString());

	dtNavMesh* NavMesh = NavMeshImpl.DetourNavMesh;
	if (NavMesh != nullptr)
	{
		for (FRecastTileData& TileData : Tiles)
		{
			if (TileData.TileCacheDataSize != 0)
			{
				UE_LOGF(LogNavigation, Error, "   TileCacheRawData is expected to be empty. No support for moving the cache data yet.");
				continue;
			}

			if ((TileData.bAttached == false) && TileData.TileRawData.IsValid())
			{
				const FVector RcRotationCenter = Unreal2RecastPoint(FVector(RotationCenter.X, RotationCenter.Y, 0.f));

				const FVector::FReal TileWidth = NavMesh->getParams()->tileWidth;
				const FVector::FReal TileHeight = NavMesh->getParams()->tileHeight;

				const dtMeshHeader* Header = (dtMeshHeader*)TileData.TileRawData->RawData;
				if (Header->version != DT_NAVMESH_VERSION)
				{
					continue;
				}

				// Apply rotation to tile coordinates
				int DeltaX = 0;
				int DeltaY = 0;
				FBox TileBox(Recast2UnrealPoint(Header->bmin), Recast2UnrealPoint(Header->bmax));
				FVector RcTileCenter = Unreal2RecastPoint(TileBox.GetCenter());
				dtComputeTileOffsetFromRotation(&RcTileCenter.X, &RcRotationCenter.X, RotationDeg, TileWidth, TileHeight, DeltaX, DeltaY);

				const int OffsetWithRotX = Offset.X + DeltaX;
				const int OffsetWithRotY = Offset.Y + DeltaY;

				const bool bSuccess = dtTransformTileData(TileData.TileRawData->RawData, TileData.TileDataSize, OffsetWithRotX, OffsetWithRotY, TileWidth, TileHeight, RotationDeg, NavMesh->getBVQuantFactor(Header->resolution));
				UE_CLOGF(bSuccess, LogNavigation, Verbose, "   Moved tile from (%i,%i) to (%i,%i).", TileData.OriginalX, TileData.OriginalY, (TileData.OriginalX + OffsetWithRotX), (TileData.OriginalY + OffsetWithRotY));
			}
		}
	}

	UE_LOGF(LogNavigation, Verbose, "%ls Moving done.", ANSI_TO_TCHAR(__FUNCTION__));
#endif// WITH_RECAST
}

int32 URecastNavMeshDataChunk::GetNumTiles() const
{
	return Tiles.Num();
}

void URecastNavMeshDataChunk::ReleaseTiles()
{
	Tiles.Reset();
}

// Deprecated
void URecastNavMeshDataChunk::GetTiles(const FPImplRecastNavMesh* NavMeshImpl, const TArray<int32>& TileIndices, const EGatherTilesCopyMode CopyMode, const bool bMarkAsAttached /*= true*/)
{
	Tiles.Empty(TileIndices.Num());

#if WITH_RECAST
	if (NavMeshImpl)
	{
		TArray<uint32> TileUnsignedIndices;
		TileUnsignedIndices.Append(TileIndices);

		TArray<FNavTileRef> TileRefs;
		FNavTileRef::DeprecatedMakeTileRefsFromTileIds(NavMeshImpl, TileUnsignedIndices, TileRefs);
		GetTiles(NavMeshImpl, TileRefs, CopyMode, bMarkAsAttached);
	}
#endif // WITH_RECAST
}

#if WITH_RECAST
void URecastNavMeshDataChunk::GetTiles(const FPImplRecastNavMesh* NavMeshImpl, const TArray<FNavTileRef>& TileRefs, const EGatherTilesCopyMode CopyMode, const bool bMarkAsAttached /*= true*/)
{
	Tiles.Empty(TileRefs.Num());

	const dtNavMesh* NavMesh = NavMeshImpl->DetourNavMesh;
	
	for (const FNavTileRef TileRef : TileRefs)
	{
		const dtMeshTile* Tile = NavMesh->getTileByRef(static_cast<dtTileRef>(TileRef));
		if (Tile && Tile->header)
		{
			// Make our own copy of tile data
			uint8* RawTileData = nullptr;
			if (CopyMode & EGatherTilesCopyMode::CopyData)
			{
				RawTileData = DuplicateRecastRawData(Tile->data, Tile->dataSize);
			}

			// We need tile cache data only if navmesh supports any kind of runtime generation
			FNavMeshTileData TileCacheData;
			uint8* RawTileCacheData = nullptr;
			if (CopyMode & EGatherTilesCopyMode::CopyCacheData)
			{
				TileCacheData = NavMeshImpl->GetTileCacheLayer(Tile->header->x, Tile->header->y, Tile->header->layer);
				if (TileCacheData.IsValid())
				{
					// Make our own copy of tile cache data
					RawTileCacheData = DuplicateRecastRawData(TileCacheData.GetData(), TileCacheData.DataSize);
				}
			}

			FRecastTileData RecastTileData(Tile->dataSize, RawTileData, TileCacheData.DataSize, RawTileCacheData);
			RecastTileData.OriginalX = Tile->header->x;
			RecastTileData.OriginalY = Tile->header->y;
			RecastTileData.X = Tile->header->x;
			RecastTileData.Y = Tile->header->y;
			RecastTileData.Layer = Tile->header->layer;
			RecastTileData.bAttached = bMarkAsAttached;

			Tiles.Add(RecastTileData);
		}
	}
}
#endif // WITH_RECAST

#if WITH_RECAST
void URecastNavMeshDataChunk::GetTilesBounds(const FPImplRecastNavMesh& NavMeshImpl, const TArray<FNavTileRef>& TileRefs, FBox& OutBounds) const
{
	OutBounds.Init();

	const dtNavMesh* NavMesh = NavMeshImpl.DetourNavMesh;

	for (const FNavTileRef TileRef : TileRefs)
	{
		const dtMeshTile* Tile = NavMesh->getTileByRef(static_cast<dtTileRef>(TileRef));
		if (Tile && Tile->header)
		{
			OutBounds += Recast2UnrealBox(Tile->header->bmin, Tile->header->bmax);
		}
	}
}
#endif // WITH_RECAST
