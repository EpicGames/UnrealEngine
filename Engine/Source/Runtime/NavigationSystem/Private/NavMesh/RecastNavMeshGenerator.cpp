// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavMesh/RecastNavMeshGenerator.h"

#include "DynamicMeshBuilder.h"
#include "AI/Navigation/NavRelevantInterface.h"
#include "Compression/OodleDataCompression.h"
#include "Engine/Level.h"
#include "GameFramework/Pawn.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Serialization/MemoryWriter.h"
#include "GameFramework/PlayerController.h"
#include "Engine/Engine.h"
#include "Logging/LogScopedCategoryAndVerbosityOverride.h"
#include "NavigationSystem.h"
#include "FramePro/FrameProProfiler.h"
#include "NavMesh/RecastVersion.h"
#include "UObject/GarbageCollection.h"

#if WITH_RECAST

#include "Chaos/HeightField.h"
#include "Chaos/TriangleMeshImplicitObject.h"
#include "NavMesh/PImplRecastNavMesh.h"
#include "NavMesh/RecastGeometryExport.h"
#include "VisualLogger/VisualLogger.h"

// recast includes
#include "Detour/DetourNavMeshBuilder.h"
#include "Detour/DetourNavLinkBuilder.h"
#include "DetourTileCache/DetourTileCacheBuilder.h"
#include "NavMesh/RecastHelpers.h"
#include "NavAreas/NavArea_LowHeight.h"
#include "AI/NavigationSystemHelpers.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/BodySetup.h"

#if RECAST_INTERNAL_DEBUG_DATA
#include "DebugUtils/DebugDraw.h"
#include "DebugUtils/RecastDebugDraw.h"
#include "DebugUtils/DetourDebugDraw.h"
#include "DebugUtils/DetourNavLinkDebugDraw.h"
#endif //RECAST_INTERNAL_DEBUG_DATA

#ifndef OUTPUT_NAV_TILE_LAYER_COMPRESSION_DATA
	#define OUTPUT_NAV_TILE_LAYER_COMPRESSION_DATA 0
#endif

#ifndef FAVOR_NAV_COMPRESSION_SPEED
	#define FAVOR_NAV_COMPRESSION_SPEED 1
#endif

#define SEAMLESS_REBUILDING_ENABLED 1

#define SHOW_NAV_EXPORT_PREVIEW 0

CSV_DEFINE_CATEGORY(NAVREGEN, false);

struct dtTileCacheAlloc;

//Experimental debug tools
static int32 GNavmeshSynchronousTileGeneration = 0;
static FAutoConsoleVariableRef NavmeshVarSynchronous(TEXT("ai.nav.GNavmeshSynchronousTileGeneration"), GNavmeshSynchronousTileGeneration, TEXT(""), ECVF_Default);

#if RECAST_INTERNAL_DEBUG_DATA
static int32 GNavmeshDebugTileX = MAX_int32;
static int32 GNavmeshDebugTileY = MAX_int32;
static bool GNavmeshGenerateDebugTileOnly = false;
static FAutoConsoleVariableRef NavmeshVarDebugTileX(TEXT("ai.nav.GNavmeshDebugTileX"), GNavmeshDebugTileX, TEXT(""), ECVF_Default);
static FAutoConsoleVariableRef NavmeshVarDebugTileY(TEXT("ai.nav.GNavmeshDebugTileY"), GNavmeshDebugTileY, TEXT(""), ECVF_Default);
static FAutoConsoleVariableRef NavmeshVarGenerateDebugTileOnly(TEXT("ai.nav.GNavmeshGenerateDebugTileOnly"), GNavmeshGenerateDebugTileOnly, TEXT(""), ECVF_Default);
#endif //RECAST_INTERNAL_DEBUG_DATA

// Hotfixing this flag without rebuilding the data will cause decompression errors, equivalent to not having prebuilt navmesh data at all.
static bool GNavmeshUseOodleCompression = true;
static FAutoConsoleVariableRef NavmeshVarOodleCompression(TEXT("ai.nav.NavmeshUseOodleCompression"), GNavmeshUseOodleCompression, TEXT("Use Oodle for run-time tile cache compression/decompression. Optimized for size in editor, optimized for speed in standalone."), ECVF_Default);

namespace UE::NavMesh::Private
{
	static float RecentlyBuildTileDisplayTime = 0.2f;
	static FAutoConsoleVariableRef CVarRecentlyBuildTileDisplayTime(TEXT("ai.nav.RecentlyBuildTileDisplayTime"), RecentlyBuildTileDisplayTime, TEXT("Time (in seconds) to display tiles that have recently been built."), ECVF_Default);

	static bool bUseTightBoundExpansion = true;
	static FAutoConsoleVariableRef CVarUseTightBoundExpansion(TEXT("ai.nav.UseTightBoundExpansion"), bUseTightBoundExpansion, TEXT("Active by default. Use an expansion of one AgentRadius. Set to false to revert to the previous behavior (2 AgentRadius)."), ECVF_Default);

	static bool bUseAsymetricBorderSizes = false;
	static FAutoConsoleVariableRef CVarUseAsymetricBorderSizes(TEXT("ai.nav.UseAsymetricBorderSizes"), bUseAsymetricBorderSizes, TEXT("Disabled by default. When generating links, use asymetric tile border sizes to improve generation speed."), ECVF_Default);

	/** Hard upper bound on heightfield width/height (in voxels) used as a last-resort guard in CreateHeightField.
	 * Sized so that sizeof(rcTempSpan)*(size+2)*(size+2) fits within int32 (the binding constraint,
	 * since rcAlloc takes int rather than size_t): floor(sqrt(INT32_MAX / sizeof(rcTempSpan))) - 2
	 * = floor(sqrt(INT32_MAX / 8)) - 2 = 16381. The spans allocation (sizeof(rcSpan*)*size*size)
	 * is less constraining. */
	static constexpr int32 MaxHeightfieldSize = 16381;
}

static FOodleDataCompression::ECompressor GNavmeshTileCacheCompressor = FOodleDataCompression::ECompressor::Mermaid;
static FOodleDataCompression::ECompressionLevel GNavmeshTileCacheCompressionLevel = FOodleDataCompression::ECompressionLevel::HyperFast1;

FORCEINLINE bool DoesBoxContainOrOverlapVector(const FBox& BigBox, const FVector& In)
{
	return (In.X >= BigBox.Min.X) && (In.X <= BigBox.Max.X) 
		&& (In.Y >= BigBox.Min.Y) && (In.Y <= BigBox.Max.Y) 
		&& (In.Z >= BigBox.Min.Z) && (In.Z <= BigBox.Max.Z);
}
/** main difference between this and FBox::ContainsBox is that this returns true also when edges overlap */
FORCEINLINE bool DoesBoxContainBox(const FBox& BigBox, const FBox& SmallBox)
{
	return DoesBoxContainOrOverlapVector(BigBox, SmallBox.Min) && DoesBoxContainOrOverlapVector(BigBox, SmallBox.Max);
}

int32 GetTilesCountHelper(const dtNavMesh* DetourMesh)
{
	int32 NumTiles = 0;
	if (DetourMesh)
	{
		for (int32 i = 0; i < DetourMesh->getMaxTiles(); i++)
		{
			const dtMeshTile* TileData = DetourMesh->getTile(i);
			if (TileData && TileData->header && TileData->dataSize > 0)
			{
				NumTiles++;
			}
		}
	}

	return NumTiles;
}

/**
 * Exports geometry to OBJ file. Can be used to verify NavMesh generation in RecastDemo app
 * @param InFileName - full name of OBJ file with extension
 * @param GeomCoords - list of vertices
 * @param GeomFaces - list of triangles (3 vert indices for each)
 */
static void ExportGeomToOBJFile(const FString& InFileName, const TNavStatArray<FVector::FReal>& GeomCoords, const TNavStatArray<int32>& GeomFaces, const FString& AdditionalData)
{
#define USE_COMPRESSION 0

#if ALLOW_DEBUG_FILES
	SCOPE_CYCLE_COUNTER(STAT_Navigation_TileGeometryExportToObjAsync);

	FString FileName = InFileName;

#if USE_COMPRESSION
	FileName += TEXT("z");
	struct FDataChunk
	{
		TArray<uint8> UncompressedBuffer;
		TArray<uint8> CompressedBuffer;
		void CompressBuffer()
		{
			const int32 HeaderSize = sizeof(int32);
			const int32 UncompressedSize = UncompressedBuffer.Num();
			CompressedBuffer.Init(0, HeaderSize + FMath::Trunc(1.1f * UncompressedSize));

			int32 CompressedSize = CompressedBuffer.Num() - HeaderSize;
			uint8* DestBuffer = CompressedBuffer.GetData();
			FMemory::Memcpy(DestBuffer, &UncompressedSize, HeaderSize);
			DestBuffer += HeaderSize;

			FCompression::CompressMemory(NAME_Zlib, (void*)DestBuffer, CompressedSize, (void*)UncompressedBuffer.GetData(), UncompressedSize, COMPRESS_BiasMemory);
			CompressedBuffer.SetNum(CompressedSize + HeaderSize, EAllowShrinking::No);
		}
	};
	FDataChunk AllDataChunks[3];
	const int32 NumberOfChunks = sizeof(AllDataChunks) / sizeof(FDataChunk);
	{
		FMemoryWriter ArWriter(AllDataChunks[0].UncompressedBuffer);
		for (int32 i = 0; i < GeomCoords.Num(); i += 3)
		{
			FVector Vertex(GeomCoords[i + 0], GeomCoords[i + 1], GeomCoords[i + 2]);
			ArWriter << Vertex;
		}
	}

	{
		FMemoryWriter ArWriter(AllDataChunks[1].UncompressedBuffer);
		for (int32 i = 0; i < GeomFaces.Num(); i += 3)
		{
			FVector Face(GeomFaces[i + 0] + 1, GeomFaces[i + 1] + 1, GeomFaces[i + 2] + 1);
			ArWriter << Face;
		}
	}

	{
		auto AnsiAdditionalData = StringCast<ANSICHAR>(*AdditionalData);
		FMemoryWriter ArWriter(AllDataChunks[2].UncompressedBuffer);
		ArWriter.Serialize((ANSICHAR*)AnsiAdditionalData.Get(), AnsiAdditionalData.Length());
	}

	FArchive* FileAr = IFileManager::Get().CreateDebugFileWriter(*FileName);
	if (FileAr != NULL)
	{
		for (int32 Index = 0; Index < NumberOfChunks; ++Index)
		{
			AllDataChunks[Index].CompressBuffer();
			int32 BufferSize = AllDataChunks[Index].CompressedBuffer.Num();
			FileAr->Serialize(&BufferSize, sizeof(int32));
			FileAr->Serialize((void*)AllDataChunks[Index].CompressedBuffer.GetData(), AllDataChunks[Index].CompressedBuffer.Num());
		}
		UE_LOGF(LogNavigation, Error, "UncompressedBuffer size:: %d ", AllDataChunks[0].UncompressedBuffer.Num() + AllDataChunks[1].UncompressedBuffer.Num() + AllDataChunks[2].UncompressedBuffer.Num());
		FileAr->Close();
	}

#else
	FArchive* FileAr = IFileManager::Get().CreateDebugFileWriter(*FileName);
	if (FileAr != NULL)
	{
		for (int32 Index = 0; Index < GeomCoords.Num(); Index += 3)
		{
			FString LineToSave = FString::Printf(TEXT("v %f %f %f\n"), GeomCoords[Index + 0], GeomCoords[Index + 1], GeomCoords[Index + 2]);
			auto AnsiLineToSave = StringCast<ANSICHAR>(*LineToSave);
			FileAr->Serialize((ANSICHAR*)AnsiLineToSave.Get(), AnsiLineToSave.Length());
		}

		for (int32 Index = 0; Index < GeomFaces.Num(); Index += 3)
		{
			FString LineToSave = FString::Printf(TEXT("f %d %d %d\n"), GeomFaces[Index + 0] + 1, GeomFaces[Index + 1] + 1, GeomFaces[Index + 2] + 1);
			auto AnsiLineToSave = StringCast<ANSICHAR>(*LineToSave);
			FileAr->Serialize((ANSICHAR*)AnsiLineToSave.Get(), AnsiLineToSave.Length());
		}

		auto AnsiAdditionalData = StringCast<ANSICHAR>(*AdditionalData);
		FileAr->Serialize((ANSICHAR*)AnsiAdditionalData.Get(), AnsiAdditionalData.Length());
		FileAr->Close();
	}
#endif

#undef USE_COMPRESSION
#endif
}

FRecastVoxelCache::FRecastVoxelCache(const uint8* Memory)
{
	uint8* BytesArr = (uint8*)Memory;
	if (Memory)
	{
		NumTiles = *((int32*)BytesArr);	BytesArr += sizeof(int32);
		if (NumTiles > 0)
		{
			Tiles = (FTileInfo*)BytesArr;

			FTileInfo* iTile = Tiles;
			for (int i = 0; i < NumTiles; i++)
			{
				iTile = (FTileInfo*)BytesArr; BytesArr += sizeof(FTileInfo);
				if (iTile->NumSpans)
				{
					iTile->SpanData = (rcSpanCache*)BytesArr; BytesArr += sizeof(rcSpanCache) * iTile->NumSpans;
				}
				else
				{
					iTile->SpanData = 0;
				}

				iTile->NextTile = (FTileInfo*)BytesArr;
			}

			iTile->NextTile = 0;
		}
		else
		{
			Tiles = 0;
		}
	}
	else
	{
		NumTiles = 0;
		Tiles = 0;
	}
}

FRecastGeometryCache::FRecastGeometryCache(const uint8* Memory)
{
	Header = *((FHeader*)Memory);
	Verts = (FVector::FReal*)(Memory + sizeof(FRecastGeometryCache));
	Indices = (int32*)(Memory + sizeof(FRecastGeometryCache) + (sizeof(FVector::FReal) * Header.NumVerts * 3));
}

rcRasterizationFlags FRecastRawGeometryElement::GetRasterizationFlags(const FCompositeNavModifier& InModifier)
{
	int32 Flags = 0;
	if (InModifier.GetFillCollisionUnderneathForNavmesh())
	{
		// To prevent navmesh generation under the geometry, set the RC_PROJECT_TO_BOTTOM flag to true.
		// This rasterize triangles as filled columns down to the HF lower bound.
		Flags |= RC_PROJECT_TO_BOTTOM;
	}
	if (InModifier.GetRasterizeAsFilledConvexVolume())
	{
		// To rasterized as filled convex volume, set RC_RASTERIZE_AS_FILLED_CONVEX flag to true.
		// It will rasterize all the triangles of an ElementData into a 2d grid of 1 span per cell thus making sure that the interior is full
		Flags |= RC_RASTERIZE_AS_FILLED_CONVEX;
	}

	return rcRasterizationFlags(Flags);
}

TSubclassOf<UNavAreaBase> FRecastRawGeometryElement::GetCollisionAreaClass(const FCompositeNavModifier& InModifier)
{
	const TArray<FAreaNavModifier>& Areas = InModifier.GetAreas();
	ensureMsgf(Areas.Num() <= 1, TEXT("GetCollisionAreaClass expects to find at most one area modifier"));
	return Areas.Num() > 0 ? Areas[0].GetAreaClass() : nullptr;
}

namespace RecastGeometryExport {

#if SHOW_NAV_EXPORT_PREVIEW
static UWorld* FindEditorWorld()
{
	if (GEngine)
	{
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			if (Context.WorldType == EWorldType::Editor)
			{
				return Context.World();
			}
		}
	}

	return NULL;
}
#endif //SHOW_NAV_EXPORT_PREVIEW

static void StoreCollisionCache(FRecastGeometryExport& GeomExport)
{
	const int32 NumFaces = GeomExport.IndexBuffer.Num() / 3;
	const int32 NumVerts = GeomExport.VertexBuffer.Num() / 3;

	if (NumFaces == 0 || NumVerts == 0)
	{
		GeomExport.Data->CollisionData.Empty();
		return;
	}

	FRecastGeometryCache::FHeader HeaderInfo;
	HeaderInfo.NumFaces = NumFaces;
	HeaderInfo.NumVerts = NumVerts;
	HeaderInfo.SlopeOverride = GeomExport.SlopeOverride;

	// allocate memory
	constexpr int32 HeaderSize = sizeof(FRecastGeometryCache);
	const int64 CoordsSize = sizeof(FVector::FReal) * 3 * NumVerts;
	const int64 IndicesSize = sizeof(int32) * 3 * NumFaces;
	const int64 CacheSize64 = HeaderSize + CoordsSize + IndicesSize;

	if (!IntFitsIn<int32>(CacheSize64))
	{
		UE_LOGF(LogNavigation, Warning, "%s Can't cache collision data (expected size would be %lld). "
			"See other warnings, like logs about collision geometry triangle count.",
			__FUNCTION__, CacheSize64);

		// Fill with a empty header
		HeaderInfo.Validation.DataSize = 0;
		HeaderInfo.NumFaces = 0;
		HeaderInfo.NumVerts = 0;
		HeaderInfo.SlopeOverride = GeomExport.SlopeOverride;

		// empty + add combo to allocate exact amount (without any overhead/slack)
		GeomExport.Data->CollisionData.Empty(HeaderSize);
		GeomExport.Data->CollisionData.AddUninitialized(HeaderSize);
		
		uint8* RawMemory = GeomExport.Data->CollisionData.GetData();
		FRecastGeometryCache* CacheMemory = (FRecastGeometryCache*)RawMemory;
		
		CacheMemory->Header = HeaderInfo;
		CacheMemory->Verts = nullptr;
		CacheMemory->Indices = nullptr;
		return;
	}

	const int32 CacheSize = (int32)CacheSize64;
	HeaderInfo.Validation.DataSize = CacheSize;

	// empty + add combo to allocate exact amount (without any overhead/slack)
	GeomExport.Data->CollisionData.Empty(CacheSize);
	GeomExport.Data->CollisionData.AddUninitialized(CacheSize);

	// store collisions
	uint8* RawMemory = GeomExport.Data->CollisionData.GetData();
	FRecastGeometryCache* CacheMemory = (FRecastGeometryCache*)RawMemory;
	CacheMemory->Header = HeaderInfo;
	CacheMemory->Verts = nullptr;
	CacheMemory->Indices = nullptr;

	FMemory::Memcpy(RawMemory + HeaderSize, GeomExport.VertexBuffer.GetData(), CoordsSize);
	FMemory::Memcpy(RawMemory + HeaderSize + CoordsSize, GeomExport.IndexBuffer.GetData(), IndicesSize);
}

void ExportChaosTriMesh(const Chaos::FTriangleMeshImplicitObject* const TriMesh, const FTransform& LocalToWorld
	, TNavStatArray<FVector::FReal>& VertexBuffer, TNavStatArray<int32>& IndexBuffer
	, FBox& UnrealBounds)
{
	if (TriMesh == nullptr)
	{
		return;
	}

	using namespace Chaos;

	int32 VertOffset = VertexBuffer.Num() / 3;

	auto LambdaHelper = [&](const auto& Triangles)
	{
		int32 NumTris = Triangles.Num();
		const Chaos::FTriangleMeshImplicitObject::ParticlesType& Vertices = TriMesh->Particles();
	
		VertexBuffer.Reserve(VertexBuffer.Num() + NumTris * 9);
		IndexBuffer.Reserve(IndexBuffer.Num() + NumTris * 3);

		const bool bFlipCullMode = (LocalToWorld.GetDeterminant() < 0.f);

		const int32 IndexOrder[3] = { bFlipCullMode ? 0 : 2, 1, bFlipCullMode ? 2 : 0 };

	#if SHOW_NAV_EXPORT_PREVIEW
		UWorld* DebugWorld = FindEditorWorld();
	#endif // SHOW_NAV_EXPORT_PREVIEW

		for (int32 TriIdx = 0; TriIdx < NumTris; ++TriIdx)
		{
			for (int32 i = 0; i < 3; i++)
			{
				const FVector UnrealCoords = LocalToWorld.TransformPosition((FVector)Vertices.GetX(Triangles[TriIdx][i]));
				UnrealBounds += UnrealCoords;

				VertexBuffer.Add(UnrealCoords.X);
				VertexBuffer.Add(UnrealCoords.Y);
				VertexBuffer.Add(UnrealCoords.Z);
			}

			IndexBuffer.Add(VertOffset + IndexOrder[0]);
			IndexBuffer.Add(VertOffset + IndexOrder[1]);
			IndexBuffer.Add(VertOffset + IndexOrder[2]);

	#if SHOW_NAV_EXPORT_PREVIEW
			if (DebugWorld)
			{
				FVector V0(VertexBuffer[(VertOffset + IndexOrder[0]) * 3 + 0], VertexBuffer[(VertOffset + IndexOrder[0]) * 3 + 1], VertexBuffer[(VertOffset + IndexOrder[0]) * 3 + 2]);
				FVector V1(VertexBuffer[(VertOffset + IndexOrder[1]) * 3 + 0], VertexBuffer[(VertOffset + IndexOrder[1]) * 3 + 1], VertexBuffer[(VertOffset + IndexOrder[1]) * 3 + 2]);
				FVector V2(VertexBuffer[(VertOffset + IndexOrder[2]) * 3 + 0], VertexBuffer[(VertOffset + IndexOrder[2]) * 3 + 1], VertexBuffer[(VertOffset + IndexOrder[2]) * 3 + 2]);

				DrawDebugLine(DebugWorld, V0, V1, bFlipCullMode ? FColor::Red : FColor::Blue, true);
				DrawDebugLine(DebugWorld, V1, V2, bFlipCullMode ? FColor::Red : FColor::Blue, true);
				DrawDebugLine(DebugWorld, V2, V0, bFlipCullMode ? FColor::Red : FColor::Blue, true);
			}
	#endif // SHOW_NAV_EXPORT_PREVIEW

			VertOffset += 3;
		}
	};


	const FTrimeshIndexBuffer& IdxBuffer = TriMesh->Elements();
	if(IdxBuffer.RequiresLargeIndices())
	{
		LambdaHelper(IdxBuffer.GetLargeIndexBuffer());
	}
	else
	{
		LambdaHelper(IdxBuffer.GetSmallIndexBuffer());
	}
}



void ExportChaosConvexMesh(const FKConvexElem* const Convex, const FTransform& LocalToWorld
	, TNavStatArray<FVector::FReal>& VertexBuffer, TNavStatArray<int32>& IndexBuffer
	, FBox& UnrealBounds)
{
	using namespace Chaos;

	if (Convex == nullptr)
	{
		return;
	}


	QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_ExportChaosConvexMesh);

	int32 VertOffset = VertexBuffer.Num() / 3;

	VertexBuffer.Reserve(VertexBuffer.Num() + Convex->VertexData.Num() * 3);
	IndexBuffer.Reserve(IndexBuffer.Num() + Convex->IndexData.Num());

#if SHOW_NAV_EXPORT_PREVIEW
	UWorld* DebugWorld = FindEditorWorld();
#endif // SHOW_NAV_EXPORT_PREVIEW

	if (Convex->VertexData.Num())
	{
		if(Convex->IndexData.Num() == 0)
		{
			UE_LOGF(LogNavigation, Verbose, "Zero indices in convex.");
			return;
		}

		if(Convex->IndexData.Num() % 3 != 0)
		{
			UE_LOGF(LogNavigation, Verbose, "Invalid indices in convex.");
			return;
		}
	}

	for (const FVector& Vertex : Convex->VertexData)
	{
		const FVector UnrealCoord = LocalToWorld.TransformPosition(Vertex);
		UnrealBounds += UnrealCoord;

		VertexBuffer.Add(UnrealCoord.X);
		VertexBuffer.Add(UnrealCoord.Y);
		VertexBuffer.Add(UnrealCoord.Z);
	}

	if (Convex->IndexData.Num() % 3 == 0)
	{
		for (int32 i = 0; i < Convex->IndexData.Num(); i += 3)
		{
			IndexBuffer.Add(VertOffset + Convex->IndexData[i]);
			IndexBuffer.Add(VertOffset + Convex->IndexData[i + 2]);
			IndexBuffer.Add(VertOffset + Convex->IndexData[i + 1]);
		}
	}

#if SHOW_NAV_EXPORT_PREVIEW
	if (DebugWorld)
	{
		for (int32 Index = VertOffset; Index < VertexBuffer.Num(); Index += 3)
		{
			FVector V0(VertexBuffer[IndexBuffer[Index] * 3], VertexBuffer[IndexBuffer[Index] * 3 + 1], VertexBuffer[IndexBuffer[Index] * 3] + 2);
			FVector V1(VertexBuffer[IndexBuffer[Index + 1] * 3], VertexBuffer[IndexBuffer[Index + 1] * 3 + 1], VertexBuffer[IndexBuffer[Index + 1] * 3] + 2);
			FVector V2(VertexBuffer[IndexBuffer[Index + 2] * 3], VertexBuffer[IndexBuffer[Index + 2] * 3 + 1], VertexBuffer[IndexBuffer[Index + 2] * 3] + 2);

			DrawDebugLine(DebugWorld, V0, V1, FColor::Blue, true);
			DrawDebugLine(DebugWorld, V1, V2, FColor::Blue, true);
			DrawDebugLine(DebugWorld, V2, V0, FColor::Blue, true);
		}
	}
#endif // SHOW_NAV_EXPORT_PREVIEW
}

void ExportChaosHeightField(const Chaos::FHeightField* const HeightField, const FTransform& LocalToWorld
	, TNavStatArray<FVector::FReal>& VertexBuffer, TNavStatArray<int32>& IndexBuffer
	, FBox& UnrealBounds)
{
	using namespace Chaos;

	if(HeightField == nullptr)
	{
		return;
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_ExportChaosHeightField);

	const int32 NumRows = HeightField->GetNumRows();
	const int32 NumCols = HeightField->GetNumCols();
	const int32 VertexCount = NumRows * NumCols;

	const int32 VertOffset = VertexBuffer.Num() / 3;
	const int32 NumQuads = (NumRows - 1) * (NumCols - 1);

	VertexBuffer.Reserve(VertexBuffer.Num() + VertexCount * 3);
	IndexBuffer.Reserve(IndexBuffer.Num() + NumQuads * 6);

	const bool bMirrored = (LocalToWorld.GetDeterminant() < 0.f);

	for(int32 Y = 0; Y < NumRows; Y++)
	{
		for(int32 X = 0; X < NumCols; X++)
		{
			const int32 SampleIdx = Y * NumCols + X;  // #PHYSTODO bMirrored support

			const FVector UnrealCoords = LocalToWorld.TransformPosition(FVector(X, Y, HeightField->GetHeight(SampleIdx)));
			UnrealBounds += UnrealCoords;

			VertexBuffer.Add(UnrealCoords.X);
			VertexBuffer.Add(UnrealCoords.Y);
			VertexBuffer.Add(UnrealCoords.Z);
		}
	}

	for(int32 Y = 0; Y < NumRows - 1; Y++)
	{
		for(int32 X = 0; X < NumCols - 1; X++)
		{
			if(HeightField->IsHole(X, Y))
			{
				continue;
			}

			const int32 I0 = Y * NumCols + X;  // #PHYSTODO bMirrored support
			int32 I1 = I0 + 1;
			int32 I2 = I0 + NumCols;
			const int32 I3 = I2 + 1;

			if(bMirrored)
			{
				// Flip the winding so the triangles face the right way after scaling
				Swap(I1, I2);
			}

			IndexBuffer.Add(VertOffset + I0);
			IndexBuffer.Add(VertOffset + I3);
			IndexBuffer.Add(VertOffset + I1);

			IndexBuffer.Add(VertOffset + I0);
			IndexBuffer.Add(VertOffset + I2);
			IndexBuffer.Add(VertOffset + I3);
		}
	}
}

void ExportChaosHeightFieldSlice(const FNavHeightfieldSamples& PrefetchedHeightfieldSamples, const int32 NumRows, const int32 NumCols, const FTransform& LocalToWorld
	, TNavStatArray<FVector::FReal>& VertexBuffer, TNavStatArray<int32>& IndexBuffer, const FBox& SliceBox
	, FBox& UnrealBounds)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_ExportChaosHeightFieldSlice);

	// calculate the actual start and number of columns we want
	const FBox LocalBox = SliceBox.TransformBy(LocalToWorld.Inverse());
	const bool bMirrored = (LocalToWorld.GetDeterminant() < 0.f);

	const int32 MinX = FMath::Clamp(FMath::FloorToInt32(LocalBox.Min.X) - 1, 0, NumCols);
	const int32 MinY = FMath::Clamp(FMath::FloorToInt32(LocalBox.Min.Y) - 1, 0, NumRows);
	const int32 MaxX = FMath::Clamp(FMath::CeilToInt32(LocalBox.Max.X) + 1, 0, NumCols);
	const int32 MaxY = FMath::Clamp(FMath::CeilToInt32(LocalBox.Max.Y) + 1, 0, NumRows);
	const int32 SizeX = MaxX - MinX;
	const int32 SizeY = MaxY - MinY;

	if (SizeX <= 0 || SizeY <= 0)
	{
		// slice is outside bounds, skip
		return;
	}

	const int32 VertOffset = VertexBuffer.Num() / 3;
	const int32 NumVerts = SizeX * SizeY;
	const int32 NumQuads = (SizeX - 1) * (SizeY - 1);
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastGeometryExport_AllocatingMemory);
		VertexBuffer.Reserve(VertexBuffer.Num() + NumVerts * 3);
		IndexBuffer.Reserve(IndexBuffer.Num() + NumQuads * 3 * 2);
	}

	for (int32 IdxY = 0; IdxY < SizeY; IdxY++)
	{
		for (int32 IdxX = 0; IdxX < SizeX; IdxX++)
		{
			const int32 CoordX = IdxX + MinX;
			const int32 CoordY = IdxY + MinY;
			const int32 SampleIdx = CoordY * NumCols + CoordX; // #PHYSTODO bMirrored support


			const FVector UnrealCoords = LocalToWorld.TransformPosition(FVector(CoordX, CoordY, PrefetchedHeightfieldSamples.Heights[SampleIdx]));
			VertexBuffer.Add(UnrealCoords.X);
			VertexBuffer.Add(UnrealCoords.Y);
			VertexBuffer.Add(UnrealCoords.Z);
		}
	}

	for (int32 IdxY = 0; IdxY < SizeY - 1; IdxY++)
	{
		for (int32 IdxX = 0; IdxX < SizeX - 1; IdxX++)
		{
			const int32 CoordX = IdxX + MinX;
			const int32 CoordY = IdxY + MinY;
			const int32 SampleIdx = CoordY * (NumCols-1) + CoordX;  // #PHYSTODO bMirrored support

			const bool bIsHole = PrefetchedHeightfieldSamples.Holes[SampleIdx];
			if (bIsHole)
			{
				continue;
			}

			const int32 I0 = IdxY * SizeX + IdxX;
			int32 I1 = I0 + 1;
			int32 I2 = I0 + SizeX;
			const int32 I3 = I2 + 1;
			if (bMirrored)
			{
				Swap(I1, I2);
			}

			IndexBuffer.Add(VertOffset + I0);
			IndexBuffer.Add(VertOffset + I3);
			IndexBuffer.Add(VertOffset + I1);

			IndexBuffer.Add(VertOffset + I0);
			IndexBuffer.Add(VertOffset + I2);
			IndexBuffer.Add(VertOffset + I3);
		}
	}
}


void ExportCustomMesh(const FVector* InVertices, int32 NumVerts, const int32* InIndices, int32 NumIndices, const FTransform& LocalToWorld,
					  TNavStatArray<FVector::FReal>& VertexBuffer, TNavStatArray<int32>& IndexBuffer, FBox& UnrealBounds)
{
	if (NumVerts <= 0 || NumIndices <= 0)
	{
		return;
	}

	if (InVertices == nullptr || InIndices == nullptr)
	{
		return;
	}

	int32 VertOffset = VertexBuffer.Num() / 3;
	VertexBuffer.Reserve(VertexBuffer.Num() + NumVerts*3);
	IndexBuffer.Reserve(IndexBuffer.Num() + NumIndices);

	const bool bFlipCullMode = (LocalToWorld.GetDeterminant() < 0.f);
	const int32 IndexOrder[3] = { bFlipCullMode ? 2 : 0, 1, bFlipCullMode ? 0 : 2 };

#if SHOW_NAV_EXPORT_PREVIEW
	UWorld* DebugWorld = FindEditorWorld();
#endif // SHOW_NAV_EXPORT_PREVIEW

	// Add vertices
	for (int32 i = 0; i < NumVerts; ++i)
	{
		const FVector UnrealCoords = LocalToWorld.TransformPosition(InVertices[i]);
		UnrealBounds += UnrealCoords;

		VertexBuffer.Add(UnrealCoords.X);
		VertexBuffer.Add(UnrealCoords.Y);
		VertexBuffer.Add(UnrealCoords.Z);
	}

	// Add indices
	for (int32 i = 0; i < NumIndices; i += 3)
	{
		IndexBuffer.Add(InIndices[i + IndexOrder[0]] + VertOffset);
		IndexBuffer.Add(InIndices[i + IndexOrder[1]] + VertOffset);
		IndexBuffer.Add(InIndices[i + IndexOrder[2]] + VertOffset);

#if SHOW_NAV_EXPORT_PREVIEW
		if (DebugWorld)
		{
			FVector V0(VertexBuffer[(VertOffset + InIndices[i + IndexOrder[0]]) * 3+0], VertexBuffer[(VertOffset + InIndices[i + IndexOrder[0]]) * 3+1], VertexBuffer[(VertOffset + InIndices[i + IndexOrder[0]]) * 3+2]);
			FVector V1(VertexBuffer[(VertOffset + InIndices[i + IndexOrder[1]]) * 3+0], VertexBuffer[(VertOffset + InIndices[i + IndexOrder[1]]) * 3+1], VertexBuffer[(VertOffset + InIndices[i + IndexOrder[1]]) * 3+2]);
			FVector V2(VertexBuffer[(VertOffset + InIndices[i + IndexOrder[2]]) * 3+0], VertexBuffer[(VertOffset + InIndices[i + IndexOrder[2]]) * 3+1], VertexBuffer[(VertOffset + InIndices[i + IndexOrder[2]]) * 3+2]);

			DrawDebugLine(DebugWorld, V0, V1, bFlipCullMode ? FColor::Red : FColor::Blue, true);
			DrawDebugLine(DebugWorld, V1, V2, bFlipCullMode ? FColor::Red : FColor::Blue, true);
			DrawDebugLine(DebugWorld, V2, V0, bFlipCullMode ? FColor::Red : FColor::Blue, true);
		}
#endif // SHOW_NAV_EXPORT_PREVIEW
	}
}

template<typename OtherAllocator>
FORCEINLINE_DEBUGGABLE void AddFacesToRecast(TArray<FVector, OtherAllocator>& InVerts, TArray<int32, OtherAllocator>& InFaces,
											 TNavStatArray<FVector::FReal>& OutVerts, TNavStatArray<int32>& OutIndices, FBox& UnrealBounds)
{
	// Add indices
	int32 StartVertOffset = OutVerts.Num();
	if (StartVertOffset > 0)
	{
		const int32 FirstIndex = OutIndices.AddUninitialized(InFaces.Num());
		for (int32 Idx=0; Idx < InFaces.Num(); ++Idx)
		{
			OutIndices[FirstIndex + Idx] = InFaces[Idx]+StartVertOffset;
		}
	}
	else
	{
		OutIndices.Append(InFaces);
	}

	// Add vertices
	for (int32 i = 0; i < InVerts.Num(); i++)
	{
		const FVector& RecastCoords = InVerts[i];
		OutVerts.Add(RecastCoords.X);
		OutVerts.Add(RecastCoords.Y);
		OutVerts.Add(RecastCoords.Z);

		UnrealBounds += Recast2UnrealPoint(RecastCoords);
	}
}

FORCEINLINE_DEBUGGABLE void ExportRigidBodyConvexElements(UBodySetup& BodySetup, TNavStatArray<FVector::FReal>& VertexBuffer, TNavStatArray<int32>& IndexBuffer,
	TNavStatArray<int32>& ShapeBuffer, FBox& UnrealBounds, const FTransform& LocalToWorld)
{
	const int32 ConvexCount = BodySetup.AggGeom.ConvexElems.Num();
	FKConvexElem const* ConvexElem = BodySetup.AggGeom.ConvexElems.GetData();

	for (int32 i = 0; i < ConvexCount; ++i, ++ConvexElem)
	{
		// Store index of first vertex in shape buffer
		ShapeBuffer.Add(VertexBuffer.Num() / 3);

		if (ConvexElem->GetChaosConvexMesh())
		{
			ExportChaosConvexMesh(ConvexElem, ConvexElem->GetTransform() * LocalToWorld, VertexBuffer, IndexBuffer, UnrealBounds);
		}
	}
}

FORCEINLINE_DEBUGGABLE void ExportRigidBodyTriMesh(UBodySetup& BodySetup, TNavStatArray<FVector::FReal>& VertexBuffer, TNavStatArray<int32>& IndexBuffer,
	FBox& UnrealBounds, const FTransform& LocalToWorld)
{
	if (BodySetup.GetCollisionTraceFlag() == CTF_UseComplexAsSimple)
	{
		for(const auto& TriMesh : BodySetup.TriMeshGeometries)
		{
			ExportChaosTriMesh(TriMesh.GetReference(), LocalToWorld, VertexBuffer, IndexBuffer, UnrealBounds);
		}
	}
}

void ExportRigidBodyBoxElements(const FKAggregateGeom& AggGeom, TNavStatArray<FVector::FReal>& VertexBuffer, TNavStatArray<int32>& IndexBuffer,
								TNavStatArray<int32>& ShapeBuffer, FBox& UnrealBounds, const FTransform& LocalToWorld, const int32 NumExistingVerts = 0)
{
	for (int32 i = 0; i < AggGeom.BoxElems.Num(); i++)
	{
		const FKBoxElem& BoxInfo = AggGeom.BoxElems[i];
		const FMatrix ElemTM = BoxInfo.GetTransform().ToMatrixWithScale() * LocalToWorld.ToMatrixWithScale();
		const FVector Extent(BoxInfo.X * 0.5f, BoxInfo.Y * 0.5f, BoxInfo.Z * 0.5f);

		const int32 VertBase = NumExistingVerts + (VertexBuffer.Num() / 3);
		
		// Store index of first vertex in shape buffer
		ShapeBuffer.Add(VertBase);

		// add box vertices
		FVector UnrealVerts[] = {
			ElemTM.TransformPosition(FVector(-Extent.X, -Extent.Y,  Extent.Z)),
			ElemTM.TransformPosition(FVector( Extent.X, -Extent.Y,  Extent.Z)),
			ElemTM.TransformPosition(FVector(-Extent.X, -Extent.Y, -Extent.Z)),
			ElemTM.TransformPosition(FVector( Extent.X, -Extent.Y, -Extent.Z)),
			ElemTM.TransformPosition(FVector(-Extent.X,  Extent.Y,  Extent.Z)),
			ElemTM.TransformPosition(FVector( Extent.X,  Extent.Y,  Extent.Z)),
			ElemTM.TransformPosition(FVector(-Extent.X,  Extent.Y, -Extent.Z)),
			ElemTM.TransformPosition(FVector( Extent.X,  Extent.Y, -Extent.Z))
		};

		for (int32 iv = 0; iv < UE_ARRAY_COUNT(UnrealVerts); iv++)
		{
			UnrealBounds += UnrealVerts[iv];

			VertexBuffer.Add(UnrealVerts[iv].X);
			VertexBuffer.Add(UnrealVerts[iv].Y);
			VertexBuffer.Add(UnrealVerts[iv].Z);
		}
		
		IndexBuffer.Add(VertBase + 3); IndexBuffer.Add(VertBase + 2); IndexBuffer.Add(VertBase + 0);
		IndexBuffer.Add(VertBase + 3); IndexBuffer.Add(VertBase + 0); IndexBuffer.Add(VertBase + 1);
		IndexBuffer.Add(VertBase + 7); IndexBuffer.Add(VertBase + 3); IndexBuffer.Add(VertBase + 1);
		IndexBuffer.Add(VertBase + 7); IndexBuffer.Add(VertBase + 1); IndexBuffer.Add(VertBase + 5);
		IndexBuffer.Add(VertBase + 6); IndexBuffer.Add(VertBase + 7); IndexBuffer.Add(VertBase + 5);
		IndexBuffer.Add(VertBase + 6); IndexBuffer.Add(VertBase + 5); IndexBuffer.Add(VertBase + 4);
		IndexBuffer.Add(VertBase + 2); IndexBuffer.Add(VertBase + 6); IndexBuffer.Add(VertBase + 4);
		IndexBuffer.Add(VertBase + 2); IndexBuffer.Add(VertBase + 4); IndexBuffer.Add(VertBase + 0);
		IndexBuffer.Add(VertBase + 1); IndexBuffer.Add(VertBase + 0); IndexBuffer.Add(VertBase + 4);
		IndexBuffer.Add(VertBase + 1); IndexBuffer.Add(VertBase + 4); IndexBuffer.Add(VertBase + 5);
		IndexBuffer.Add(VertBase + 7); IndexBuffer.Add(VertBase + 6); IndexBuffer.Add(VertBase + 2);
		IndexBuffer.Add(VertBase + 7); IndexBuffer.Add(VertBase + 2); IndexBuffer.Add(VertBase + 3);
	}
}

void ExportRigidBodySphylElements(const FKAggregateGeom& AggGeom, TNavStatArray<FVector::FReal>& VertexBuffer, TNavStatArray<int32>& IndexBuffer,
								  TNavStatArray<int32>& ShapeBuffer, FBox& UnrealBounds, const FTransform& LocalToWorld, const int32 NumExistingVerts = 0)
{
	TArray<FVector> ArcVerts;

	for (int32 i = 0; i < AggGeom.SphylElems.Num(); i++)
	{
		const FKSphylElem& SphylInfo = AggGeom.SphylElems[i];
		const FMatrix ElemTM = SphylInfo.GetTransform().ToMatrixWithScale() * LocalToWorld.ToMatrixWithScale();

		const int32 VertBase = NumExistingVerts + (VertexBuffer.Num() / 3);

		// Store index of first vertex in shape buffer
		ShapeBuffer.Add(VertBase);

		const int32 NumSides = 16;
		const int32 NumRings = (NumSides/2) + 1;

		ArcVerts.Reset();
		ArcVerts.AddZeroed(NumRings+1);
		for (int32 RingIdx=0; RingIdx<NumRings+1; RingIdx++)
		{
			FVector::FReal Angle;
			FVector::FReal ZOffset;
			if (RingIdx <= NumSides/4)
			{
				Angle = ((FVector::FReal)RingIdx/(NumRings-1)) * (FVector::FReal)DOUBLE_PI;
				ZOffset = 0.5f * SphylInfo.Length;
			}
			else
			{
				Angle = ((FVector::FReal)(RingIdx-1)/(NumRings-1)) * (FVector::FReal)DOUBLE_PI;
				ZOffset = -0.5f * SphylInfo.Length;
			}

			// Note- unit sphere, so position always has mag of one. We can just use it for normal!		
			FVector SpherePos;
			SpherePos.X = 0.0f;
			SpherePos.Y = SphylInfo.Radius * FMath::Sin(Angle);
			SpherePos.Z = SphylInfo.Radius * FMath::Cos(Angle);

			ArcVerts[RingIdx] = SpherePos + FVector(0,0,ZOffset);
		}

		// Then rotate this arc NumSides+1 times.
		for (int32 SideIdx=0; SideIdx<NumSides+1; SideIdx++)
		{
			const FRotator ArcRotator(0, 360.f * ((float)SideIdx/NumSides), 0);
			const FRotationMatrix ArcRot( ArcRotator );
			const FMatrix ArcTM = ArcRot * ElemTM;

			for(int32 VertIdx=0; VertIdx<NumRings+1; VertIdx++)
			{
				const FVector UnrealVert = ArcTM.TransformPosition(ArcVerts[VertIdx]);
				UnrealBounds += UnrealVert;

				VertexBuffer.Add(UnrealVert.X);
				VertexBuffer.Add(UnrealVert.Y);
				VertexBuffer.Add(UnrealVert.Z);
			}
		}

		// Add all of the triangles to the mesh.
		for (int32 SideIdx=0; SideIdx<NumSides; SideIdx++)
		{
			const int32 a0start = VertBase + ((SideIdx+0) * (NumRings+1));
			const int32 a1start = VertBase + ((SideIdx+1) * (NumRings+1));

			for (int32 RingIdx=0; RingIdx<NumRings; RingIdx++)
			{
				IndexBuffer.Add(a0start + RingIdx + 0); IndexBuffer.Add(a1start + RingIdx + 0); IndexBuffer.Add(a0start + RingIdx + 1);
				IndexBuffer.Add(a1start + RingIdx + 0); IndexBuffer.Add(a1start + RingIdx + 1); IndexBuffer.Add(a0start + RingIdx + 1);
			}
		}
	}
}

void ExportRigidBodySphereElements(const FKAggregateGeom& AggGeom, TNavStatArray<FVector::FReal>& VertexBuffer, TNavStatArray<int32>& IndexBuffer,
								   TNavStatArray<int32>& ShapeBuffer, FBox& UnrealBounds, const FTransform& LocalToWorld, const int32 NumExistingVerts = 0)
{
	TArray<FVector> ArcVerts;

	for (int32 i = 0; i < AggGeom.SphereElems.Num(); i++)
	{
		const FKSphereElem& SphereInfo = AggGeom.SphereElems[i];
		const FMatrix ElemTM = SphereInfo.GetTransform().ToMatrixWithScale() * LocalToWorld.ToMatrixWithScale();

		const int32 VertBase = NumExistingVerts + (VertexBuffer.Num() / 3);

		// Store index of first vertex in shape buffer
		ShapeBuffer.Add(VertBase);

		const int32 NumSides = 16;
		const int32 NumRings = (NumSides/2) + 1;

		ArcVerts.Reset();
		ArcVerts.AddZeroed(NumRings+1);
		for (int32 RingIdx=0; RingIdx<NumRings+1; RingIdx++)
		{
			FVector::FReal Angle = ((FVector::FReal)RingIdx/NumRings) * (FVector::FReal)DOUBLE_PI;

			// Note- unit sphere, so position always has mag of one. We can just use it for normal!			
			FVector& ArcVert = ArcVerts[RingIdx];
			ArcVert.X = 0.0f;
			ArcVert.Y = SphereInfo.Radius * FMath::Sin(Angle);
			ArcVert.Z = SphereInfo.Radius * FMath::Cos(Angle);
		}

		// Then rotate this arc NumSides+1 times.
		for (int32 SideIdx=0; SideIdx<NumSides+1; SideIdx++)
		{
			const FRotator ArcRotator(0, 360.f * ((float)SideIdx/NumSides), 0);
			const FRotationMatrix ArcRot( ArcRotator );
			const FMatrix ArcTM = ArcRot * ElemTM;

			for(int32 VertIdx=0; VertIdx<NumRings+1; VertIdx++)
			{
				const FVector UnrealVert = ArcTM.TransformPosition(ArcVerts[VertIdx]);
				UnrealBounds += UnrealVert;

				VertexBuffer.Add(UnrealVert.X);
				VertexBuffer.Add(UnrealVert.Y);
				VertexBuffer.Add(UnrealVert.Z);
			}
		}

		// Add all of the triangles to the mesh.
		for (int32 SideIdx=0; SideIdx<NumSides; SideIdx++)
		{
			const int32 a0start = VertBase + ((SideIdx+0) * (NumRings+1));
			const int32 a1start = VertBase + ((SideIdx+1) * (NumRings+1));

			for (int32 RingIdx=0; RingIdx<NumRings; RingIdx++)
			{
				IndexBuffer.Add(a0start + RingIdx + 0); IndexBuffer.Add(a1start + RingIdx + 0); IndexBuffer.Add(a0start + RingIdx + 1);
				IndexBuffer.Add(a1start + RingIdx + 0); IndexBuffer.Add(a1start + RingIdx + 1); IndexBuffer.Add(a0start + RingIdx + 1);
			}
		}
	}
}

FORCEINLINE_DEBUGGABLE void ExportRigidBodySetup(UBodySetup& BodySetup, TNavStatArray<FVector::FReal>& VertexBuffer, TNavStatArray<int32>& IndexBuffer,
												 FBox& UnrealBounds, const FTransform& LocalToWorld)
{
	// Make sure meshes are created before we try and export them
	BodySetup.CreatePhysicsMeshes();

	static TNavStatArray<int32> TemporaryShapeBuffer;

	ExportRigidBodyTriMesh(BodySetup, VertexBuffer, IndexBuffer, UnrealBounds, LocalToWorld);
	ExportRigidBodyConvexElements(BodySetup, VertexBuffer, IndexBuffer, TemporaryShapeBuffer, UnrealBounds, LocalToWorld);
	ExportRigidBodyBoxElements(BodySetup.AggGeom, VertexBuffer, IndexBuffer, TemporaryShapeBuffer, UnrealBounds, LocalToWorld);
	ExportRigidBodySphylElements(BodySetup.AggGeom, VertexBuffer, IndexBuffer, TemporaryShapeBuffer, UnrealBounds, LocalToWorld);
	ExportRigidBodySphereElements(BodySetup.AggGeom, VertexBuffer, IndexBuffer, TemporaryShapeBuffer, UnrealBounds, LocalToWorld);

	TemporaryShapeBuffer.Reset();
}

void ExportObject(const FNavigationElement& Element, FRecastGeometryExport& GeomExport)
{
	const EHasCustomNavigableGeometry::Type GeometryExportType = Element.GetGeometryExportType();
	if (GeometryExportType == EHasCustomNavigableGeometry::DontExport)
	{
		return;
	}

	bool bDefaultGeometryExportRequired = true;
	if (GeometryExportType != EHasCustomNavigableGeometry::No)
	{
		Element.CustomGeometryExportDelegate.ExecuteIfBound(Element, GeomExport, bDefaultGeometryExportRequired);
	}

	if (UBodySetup* BodySetup = Element.GetBodySetup())
	{
		if (bDefaultGeometryExportRequired)
		{
			ExportRigidBodySetup(*BodySetup, GeomExport.VertexBuffer, GeomExport.IndexBuffer, GeomExport.Data->Bounds, Element.GetTransform());
		}
		GeomExport.SlopeOverride = BodySetup->WalkableSlopeOverride;
	}
}

#if !UE_BUILD_SHIPPING
FORCEINLINE_DEBUGGABLE void ValidateGeometryExport(const FRecastGeometryExport& GeomExport)
{
	if (const UObject* Owner = GeomExport.Data->SourceElement->GetWeakUObject().Get())
	{
		if (const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(Owner->GetWorld()))
		{
			constexpr int32 CoordinatePerTriangle = 9;
			if (NavSys->GeometryExportTriangleCountWarningThreshold > 0
				&& (GeomExport.VertexBuffer.Num() / CoordinatePerTriangle) > NavSys->GeometryExportTriangleCountWarningThreshold)
			{
				static uint32 LastNameHash = 0;
				const FString FullName = GeomExport.Data->SourceElement.Get().GetFullName();
				const uint32 NameHash = GetTypeHash(FullName);
				if (NameHash != LastNameHash)
				{
					UE_LOGF(LogNavigation,
						Warning,
						"Exporting collision geometry with too many triangles (%i). This might cause performance and memory issues."
						" Add a simple collision or change GeometryExportVertexCountWarningThreshold. See '%ls'.",
						GeomExport.VertexBuffer.Num() / CoordinatePerTriangle, *FullName);
				}
				LastNameHash = NameHash;
			}
		}
	}
}
#endif //!UE_BUILD_SHIPPING
	
FORCEINLINE void TransformVertexSoupToRecast(const TArray<FVector>& VertexSoup, TNavStatArray<FVector>& Verts, TNavStatArray<int32>& Faces)
{
	if (VertexSoup.Num() == 0)
	{
		return;
	}

	check(VertexSoup.Num() % 3 == 0);

	const int32 StaticFacesCount = VertexSoup.Num() / 3;
	int32 VertsCount = Verts.Num();
	const FVector* Vertex = VertexSoup.GetData();

	for (int32 k = 0; k < StaticFacesCount; ++k, Vertex += 3)
	{
		Verts.Add(Unreal2RecastPoint(Vertex[0]));
		Verts.Add(Unreal2RecastPoint(Vertex[1]));
		Verts.Add(Unreal2RecastPoint(Vertex[2]));
		Faces.Add(VertsCount + 2);
		Faces.Add(VertsCount + 1);
		Faces.Add(VertsCount + 0);
			
		VertsCount += 3;
	}
}

FORCEINLINE void ConvertCoordDataToRecast(TNavStatArray<FVector::FReal>& Coords)
{
	if (Coords.Num() == 0)
	{
		return;
	}

	FVector::FReal* CoordPtr = Coords.GetData();
	const int32 MaxIt = Coords.Num() / 3;
	for (int32 i = 0; i < MaxIt; i++)
	{
		CoordPtr[0] = -CoordPtr[0];

		const FVector::FReal TmpV = -CoordPtr[1];
		CoordPtr[1] = CoordPtr[2];
		CoordPtr[2] = TmpV;

		CoordPtr += 3;
	}
}

void ExportVertexSoup(const TArray<FVector>& VertexSoup, TNavStatArray<FVector::FReal>& VertexBuffer, TNavStatArray<int32>& IndexBuffer, FBox& UnrealBounds)
{
	if (VertexSoup.Num())
	{
		check(VertexSoup.Num() % 3 == 0);
		
		int32 VertBase = VertexBuffer.Num() / 3;
		VertexBuffer.Reserve(VertexSoup.Num() * 3);
		IndexBuffer.Reserve(VertexSoup.Num() / 3);

		const int32 NumVerts = VertexSoup.Num();
		for (int32 i = 0; i < NumVerts; i++)
		{
			const FVector& UnrealCoords = VertexSoup[i];
			UnrealBounds += UnrealCoords;

			const FVector RecastCoords = Unreal2RecastPoint(UnrealCoords);
			VertexBuffer.Add(RecastCoords.X);
			VertexBuffer.Add(RecastCoords.Y);
			VertexBuffer.Add(RecastCoords.Z);
		}

		const int32 NumFaces = VertexSoup.Num() / 3;
		for (int32 i = 0; i < NumFaces; i++)
		{
			IndexBuffer.Add(VertBase + 2);
			IndexBuffer.Add(VertBase + 1);
			IndexBuffer.Add(VertBase + 0);
			VertBase += 3;
		}
	}
}

} // namespace RecastGeometryExport

FRecastGeometryExport::FRecastGeometryExport(FNavigationRelevantData& InData) 
	: Data(&InData) 
{
	Data->Bounds = FBox(ForceInit);
}

void FRecastGeometryExport::ExportChaosTriMesh(const Chaos::FTriangleMeshImplicitObject* const TriMesh, const FTransform& LocalToWorld)
{
	RecastGeometryExport::ExportChaosTriMesh(TriMesh, LocalToWorld, VertexBuffer, IndexBuffer, Data->Bounds);
}

void FRecastGeometryExport::ExportChaosConvexMesh(const FKConvexElem* const Convex, const FTransform& LocalToWorld)
{
	RecastGeometryExport::ExportChaosConvexMesh(Convex, LocalToWorld, VertexBuffer, IndexBuffer, Data->Bounds);
}

void FRecastGeometryExport::ExportChaosHeightField(const Chaos::FHeightField* const Heightfield, const FTransform& LocalToWorld)
{
	RecastGeometryExport::ExportChaosHeightField(Heightfield, LocalToWorld, VertexBuffer, IndexBuffer, Data->Bounds);
}

void FRecastGeometryExport::ExportChaosHeightFieldSlice(const FNavHeightfieldSamples& PrefetchedHeightfieldSamples, const int32 NumRows, const int32 NumCols, const FTransform& LocalToWorld, const FBox& SliceBox)
{
	RecastGeometryExport::ExportChaosHeightFieldSlice(PrefetchedHeightfieldSamples, NumRows, NumCols, LocalToWorld, VertexBuffer, IndexBuffer, SliceBox, Data->Bounds);
}

void FRecastGeometryExport::ExportCustomMesh(const FVector* InVertices, int32 NumVerts, const int32* InIndices, int32 NumIndices, const FTransform& LocalToWorld)
{
	if (NumIndices % 3 != 0)
	{
		UE_LOGF(LogNavigation, Warning, "%s: InIndices doesn't represent a list of triangles. Skipping it [Data Owner: %ls]", __FUNCTION__, *GetDataOwnerName());
		return;
	}

	RecastGeometryExport::ExportCustomMesh(InVertices, NumVerts, InIndices, NumIndices, LocalToWorld, VertexBuffer, IndexBuffer, Data->Bounds);
}

void FRecastGeometryExport::ExportRigidBodySetup(UBodySetup& BodySetup, const FTransform& LocalToWorld)
{
	RecastGeometryExport::ExportRigidBodySetup(BodySetup, VertexBuffer, IndexBuffer, Data->Bounds, LocalToWorld);
}

void FRecastGeometryExport::AddNavModifiers(const FCompositeNavModifier& Modifiers)
{
	Data->Modifiers.Add(Modifiers);
}

void FRecastGeometryExport::SetNavDataPerInstanceTransformDelegate(const FNavDataPerInstanceTransformDelegate& InDelegate)
{
	Data->NavDataPerInstanceTransformDelegate = InDelegate;
}

void FRecastGeometryExport::ConvertVertexBufferToRecast()
{
	if (VertexBuffer.Num() % 3 != 0)
	{
		UE_LOGF(LogNavigation, Warning, "%s: try to convert a vertex buffer that doesn't contain a list of vertex triplets. Skipping it [Data Owner: %ls]", __FUNCTION__, *GetDataOwnerName());
		return;
	}

	RecastGeometryExport::ConvertCoordDataToRecast(VertexBuffer);
}

void FRecastGeometryExport::StoreCollisionCache()
{
	RecastGeometryExport::StoreCollisionCache(*this);
}

void FRecastGeometryExport::TransformVertexSoupToRecast(const TArray<FVector>& VertexSoup, TNavStatArray<FVector>& Verts, TNavStatArray<int32>& Faces)
{
	RecastGeometryExport::TransformVertexSoupToRecast(VertexSoup, Verts, Faces);
}

FString FRecastGeometryExport::GetDataOwnerName() const
{
	return Data ? Data->SourceElement.Get().GetName() : TEXT("No Data");
}

FORCEINLINE void GrowConvexHull(const FVector::FReal ExpandBy, const TArray<FVector>& Verts, TArray<FVector>& OutResult)
{
	if (Verts.Num() < 3)
	{
		return;
	}

	struct FSimpleLine
	{
		FVector P1, P2;

		FSimpleLine() {}

		FSimpleLine(FVector Point1, FVector Point2) 
			: P1(Point1), P2(Point2) 
		{

		}
		static FVector Intersection(const FSimpleLine& Line1, const FSimpleLine& Line2)
		{
			const FVector::FReal A1 = Line1.P2.X - Line1.P1.X;
			const FVector::FReal B1 = Line2.P1.X - Line2.P2.X;
			const FVector::FReal C1 = Line2.P1.X - Line1.P1.X;

			const FVector::FReal A2 = Line1.P2.Y - Line1.P1.Y;
			const FVector::FReal B2 = Line2.P1.Y - Line2.P2.Y;
			const FVector::FReal C2 = Line2.P1.Y - Line1.P1.Y;

			const FVector::FReal Denominator = A2*B1 - A1*B2;
			if (Denominator != 0)
			{
				const FVector::FReal t = (B1*C2 - B2*C1) / Denominator;
				return Line1.P1 + t * (Line1.P2 - Line1.P1);
			}

			return FVector::ZeroVector;
		}
	};

	TArray<FVector> AllVerts(Verts);
	AllVerts.Add(Verts[0]);
	AllVerts.Add(Verts[1]);

	const int32 VertsCount = AllVerts.Num();
	const FQuat Rotation90(FVector(0, 0, 1), FMath::DegreesToRadians(90.0));

	FVector::FReal RotationAngle = TNumericLimits<FVector::FReal>::Max();
	for (int32 Index = 0; Index < VertsCount - 2; ++Index)
	{
		const FVector& V1 = AllVerts[Index + 0];
		const FVector& V2 = AllVerts[Index + 1];
		const FVector& V3 = AllVerts[Index + 2];

		const FVector V01 = (V1 - V2).GetSafeNormal();
		const FVector V12 = (V2 - V3).GetSafeNormal();
		const FVector NV1 = Rotation90.RotateVector(V01);
		const FVector::FReal d = FVector::DotProduct(NV1, V12);

		if (d < 0)
		{
			// CW
			RotationAngle = -90;
			break;
		}
		else if (d > 0)
		{
			//CCW
			RotationAngle = 90;
			break;
		}
	}

	// check if we detected CW or CCW direction
	if (RotationAngle >= BIG_NUMBER)
	{
		return;
	}

	const FVector::FReal ExpansionThreshold = 2 * ExpandBy;
	const FVector::FReal ExpansionThresholdSQ = ExpansionThreshold * ExpansionThreshold;
	const FQuat Rotation(FVector(0, 0, 1), FMath::DegreesToRadians(RotationAngle));
	FSimpleLine PreviousLine;
	OutResult.Reserve(Verts.Num());
	for (int32 Index = 0; Index < VertsCount-2; ++Index)
	{
		const FVector& V1 = AllVerts[Index + 0];
		const FVector& V2 = AllVerts[Index + 1];
		const FVector& V3 = AllVerts[Index + 2];

		FSimpleLine Line1;
		if (Index > 0)
		{
			Line1 = PreviousLine;
		}
		else
		{
			const FVector V01 = (V1 - V2).GetSafeNormal();
			const FVector N1 = Rotation.RotateVector(V01).GetSafeNormal();
			const FVector MoveDir1 = N1 * ExpandBy;
			Line1 = FSimpleLine(V1 + MoveDir1, V2 + MoveDir1);
		}

		const FVector V12 = (V2 - V3).GetSafeNormal();
		const FVector N2 = Rotation.RotateVector(V12).GetSafeNormal();
		const FVector MoveDir2 = N2 * ExpandBy;
		const FSimpleLine Line2(V2 + MoveDir2, V3 + MoveDir2);

		const FVector NewPoint = FSimpleLine::Intersection(Line1, Line2);
		if (NewPoint == FVector::ZeroVector)
		{
			// both lines are parallel so just move our point by expansion distance
			OutResult.Add(V2 + MoveDir2);
		}
		else
		{
			const FVector VectorToNewPoint = NewPoint - V2;
			const FVector::FReal DistToNewVector = VectorToNewPoint.SizeSquared2D();
			if (DistToNewVector > ExpansionThresholdSQ)
			{
				//clamp our point to not move to far from original location
				const FVector HelpPos = V2 + VectorToNewPoint.GetSafeNormal2D() * ExpandBy * 1.4142;
				OutResult.Add(HelpPos);
			}
			else
			{
				OutResult.Add(NewPoint);
			}
		}

		PreviousLine = Line2;
	}
}

//----------------------------------------------------------------------//

struct FOffMeshData
{
	TArray<dtOffMeshLinkCreateParams> LinkParams;
	const TMap<const UClass*, int32>* AreaClassToIdMap;
	const ARecastNavMesh::FNavPolyFlags* FlagsPerArea;

	FOffMeshData() : AreaClassToIdMap(NULL), FlagsPerArea(NULL) {}

	FORCEINLINE void Reserve(const uint32 ElementsCount)
	{
		LinkParams.Reserve(ElementsCount);
	}

	void AddLink(const FNavigationLink& Link, const FTransform& LocalToWorld, float DefaultSnapHeight, TFunctionRef<void(dtOffMeshLinkCreateParams&)> AddAreaID)
	{
		dtOffMeshLinkCreateParams NewInfo;
		FMemory::Memzero(NewInfo);

		// not doing anything to link's points order - should be already ordered properly by link processor
		StoreUnrealPoint(NewInfo.vertsA0, LocalToWorld.TransformPosition(Link.Left));
		StoreUnrealPoint(NewInfo.vertsB0, LocalToWorld.TransformPosition(Link.Right));

		NewInfo.type = DT_OFFMESH_CON_POINT | 
			(Link.Direction == ENavLinkDirection::BothWays ? DT_OFFMESH_CON_BIDIR : 0) |
			(Link.bSnapToCheapestArea ? DT_OFFMESH_CON_CHEAPAREA : 0) |
			(Link.bIsGenerated ? DT_OFFMESH_CON_GENERATED : 0) |
			(Link.Direction == ENavLinkDirection::RightToLeft ? DT_OFFMESH_CON_REVERSED : 0);

		NewInfo.snapRadius = Link.SnapRadius;
		NewInfo.snapHeight = Link.bUseSnapHeight ? Link.SnapHeight : DefaultSnapHeight;
		NewInfo.userID = Link.NavLinkId.GetId();

		AddAreaID(NewInfo);

		// snap area is currently not supported for regular (point-point) offmesh links

		LinkParams.Add(NewInfo);
	}
	
	void AddLinks(const TArray<FNavigationLink>& Links, const FTransform& LocalToWorld, int32 AgentIndex, float DefaultSnapHeight)
	{
		for (int32 LinkIndex = 0; LinkIndex < Links.Num(); ++LinkIndex)
		{
			const FNavigationLink& Link = Links[LinkIndex];
			if (!Link.SupportedAgents.Contains(AgentIndex))
			{
				continue;
			}
			
			AddLink(Link, LocalToWorld, DefaultSnapHeight, [&] (dtOffMeshLinkCreateParams& NewInfo)
			{
				UClass* AreaClass = Link.GetAreaClass();
				const int32* AreaID = AreaClassToIdMap->Find(AreaClass);
				if (AreaID != nullptr)
				{
					NewInfo.area = IntCastChecked<unsigned char>(*AreaID);
					NewInfo.polyFlag = FlagsPerArea[*AreaID];
				}
				else
				{
					UE_LOGF(LogNavigation, Warning, "FRecastTileGenerator: Trying to use undefined area class while defining Off-Mesh links! (%ls)", *GetNameSafe(AreaClass));
				}
			});
		}
	}
	
	void AddLinks(const TInlineGeneratedNavLinkArray& Links, const FTransform& LocalToWorld, int32 AgentIndex, float DefaultSnapHeight)
	{
		for (int32 LinkIndex = 0; LinkIndex < Links.Num(); ++LinkIndex)
		{
			const FGeneratedNavigationLink& Link = Links[LinkIndex];
			if (!Link.SupportedAgents.Contains(AgentIndex))
			{
				continue;
			}

			AddLink(Link, LocalToWorld, DefaultSnapHeight, [&] (dtOffMeshLinkCreateParams& NewInfo)
			{
				// area and polyFlag have already been resolved for generated links, jut copy them
				NewInfo.area = Link.generatedLinkArea;
				NewInfo.polyFlag = Link.generatedLinkPolyFlag;
			});
		}
	}

#if WITH_NAVMESH_SEGMENT_LINKS	
	void AddSegmentLinks(const TArray<FNavigationSegmentLink>& Links, const FTransform& LocalToWorld, int32 AgentIndex, float DefaultSnapHeight)
	{
		for (int32 LinkIndex = 0; LinkIndex < Links.Num(); ++LinkIndex)
		{
			const FNavigationSegmentLink& Link = Links[LinkIndex];
			if (!Link.SupportedAgents.Contains(AgentIndex))
			{
				continue;
			}

			dtOffMeshLinkCreateParams NewInfo;
			FMemory::Memzero(NewInfo);

			// not doing anything to link's points order - should be already ordered properly by link processor
			StoreUnrealPoint(NewInfo.vertsA0, LocalToWorld.TransformPosition(Link.LeftStart));
			StoreUnrealPoint(NewInfo.vertsA1, LocalToWorld.TransformPosition(Link.LeftEnd));
			StoreUnrealPoint(NewInfo.vertsB0, LocalToWorld.TransformPosition(Link.RightStart));
			StoreUnrealPoint(NewInfo.vertsB1, LocalToWorld.TransformPosition(Link.RightEnd));

			NewInfo.type = DT_OFFMESH_CON_SEGMENT | (Link.Direction == ENavLinkDirection::BothWays ? DT_OFFMESH_CON_BIDIR : 0);
			NewInfo.snapRadius = Link.SnapRadius;
			NewInfo.snapHeight = Link.bUseSnapHeight ? Link.SnapHeight : DefaultSnapHeight;
			NewInfo.userID = Link.NavLinkId.GetId();

			UClass* AreaClass = Link.GetAreaClass();
			const int32* AreaID = AreaClassToIdMap->Find(AreaClass);
			if (AreaID != NULL)
			{
				NewInfo.area = IntCastChecked<unsigned char>(*AreaID);
				NewInfo.polyFlag = FlagsPerArea[*AreaID];
			}
			else
			{
				UE_LOGF(LogNavigation, Warning, "FRecastTileGenerator: Trying to use undefined area class while defining Off-Mesh links! (%ls)", *GetNameSafe(AreaClass));
			}

			LinkParams.Add(NewInfo);
		}
	}
#endif // WITH_NAVMESH_SEGMENT_LINKS

protected:

	void StoreUnrealPoint(FVector::FReal* dest, const FVector& UnrealPt)
	{
		const FVector RecastPt = Unreal2RecastPoint(UnrealPt);
		dest[0] = RecastPt.X;
		dest[1] = RecastPt.Y;
		dest[2] = RecastPt.Z;
	}
};

//----------------------------------------------------------------------//
// FNavMeshBuildContext
// A navmesh building reporting helper
//----------------------------------------------------------------------//
class FNavMeshBuildContext : public rcContext, public dtTileCacheLogContext
{
public:
	FNavMeshBuildContext(FRecastTileGenerator& InTileGenerator)
		: rcContext(true)
#if RECAST_INTERNAL_DEBUG_DATA
		, InternalDebugData(InTileGenerator.GetMutableDebugData())
#endif
	{
	}

#if RECAST_INTERNAL_DEBUG_DATA
	FRecastInternalDebugData& InternalDebugData;
#endif

protected:
	/// Logs a message.
	///  @param[in]		category	The category of the message.
	///  @param[in]		Msg			The formatted message.
	///  @param[in]		len			The length of the formatted message.
	virtual void doLog(const rcLogCategory category, const char* Msg, const int32 /*len*/) 
	{
		switch (category) 
		{
		case RC_LOG_ERROR:
			UE_LOGF(LogNavigation, Error, "Recast: %s",  Msg);
			break;
		case RC_LOG_WARNING:
			UE_LOGF(LogNavigation, Log, "Recast: %s",  Msg);
			break;
		default:
			UE_LOGF(LogNavigation, VeryVerbose, "Recast: %s", Msg);
			break;
		}
	}

	virtual void doDtLog(const char* Msg, const int32 /*len*/)
	{
		UE_LOGF(LogNavigation, Error, "Recast: %s", Msg);
	}
};

//----------------------------------------------------------------------//
struct FTileCacheCompressor : public dtTileCacheCompressor
{
	struct FCompressedCacheHeader
	{
		int32 UncompressedSize;
	};

	virtual int32 maxCompressedSize(const int32 bufferSize)
	{
		if (GNavmeshUseOodleCompression)
		{
			return FOodleDataCompression::CompressedBufferSizeNeeded(bufferSize) + sizeof(FCompressedCacheHeader);
		}
		else
		{
			return FMath::TruncToInt(static_cast<float>(bufferSize) * 1.1f) + sizeof(FCompressedCacheHeader);
		}
	}

	virtual dtStatus compress(const uint8* buffer, const int32 bufferSize,
		uint8* compressed, const int32 maxCompressedSize, int32* outCompressedSize)
	{
		const int32 HeaderSize = sizeof(FCompressedCacheHeader);

		FCompressedCacheHeader DataHeader;
		DataHeader.UncompressedSize = bufferSize;
		FMemory::Memcpy((void*)compressed, &DataHeader, HeaderSize);

		uint8* DataPtr = compressed + HeaderSize;		
		int32 DataSize = maxCompressedSize - HeaderSize;

		if (GNavmeshUseOodleCompression)
		{
			const int64 CompressedSize = FOodleDataCompression::CompressParallel((void*)DataPtr, DataSize, (const void*)buffer, bufferSize, GNavmeshTileCacheCompressor, GNavmeshTileCacheCompressionLevel);
			if (CompressedSize > 0)
			{
				*outCompressedSize = IntCastChecked<int32>(CompressedSize + HeaderSize);
				return DT_SUCCESS;
			}
			else
			{
				return DT_FAILURE;
			}
		}
		else
		{
			if (FCompression::CompressMemory(NAME_Zlib, (void*)DataPtr, DataSize, (const void*)buffer, bufferSize, COMPRESS_BiasMemory))
			{
				*outCompressedSize = DataSize + HeaderSize;
				return DT_SUCCESS;
			}
			else
			{
				return DT_FAILURE;
			}

		}
	}

	virtual dtStatus decompress(const uint8* compressed, const int32 compressedSize,
		uint8* buffer, const int32 maxBufferSize, int32* outDecompressedSize)
	{
		const int32 HeaderSize = sizeof(FCompressedCacheHeader);
		
		FCompressedCacheHeader DataHeader;
		FMemory::Memcpy(&DataHeader, (void*)compressed, HeaderSize);

		const uint8* DataPtr = compressed + HeaderSize;		
		const int32 DataSize = compressedSize - HeaderSize;

		if (GNavmeshUseOodleCompression)
		{
			if (FOodleDataCompression::Decompress((void*)buffer, DataHeader.UncompressedSize, (const void*)DataPtr, DataSize))
			{
				*outDecompressedSize = DataHeader.UncompressedSize;
				return DT_SUCCESS;
			}
			else
			{
				return DT_FAILURE;
			}
		}
		else
		{
			if (FCompression::UncompressMemory(NAME_Zlib, (void*)buffer, DataHeader.UncompressedSize, (const void*)DataPtr, DataSize))
			{
				*outDecompressedSize = DataHeader.UncompressedSize;
				return DT_SUCCESS;
			}
			else
			{
				return DT_FAILURE;
			}
		}
	}
};

struct FTileCacheAllocator : public dtTileCacheAlloc
{
	virtual void reset()
	{
		 check(0 && "dtTileCacheAlloc.reset() is not supported!");
	}

	virtual void* alloc(const int32 Size)
	{
		return dtAlloc(Size, DT_ALLOC_TEMP);
	}

	virtual void free(void* Data)
	{
		dtFree(Data, DT_ALLOC_TEMP);
	}
};

//----------------------------------------------------------------------//
// FVoxelCacheRasterizeContext
//----------------------------------------------------------------------//

struct FVoxelCacheRasterizeContext
{
	FVoxelCacheRasterizeContext()
	{
		RasterizeHF = NULL;
	}

	~FVoxelCacheRasterizeContext()
	{
		rcFreeHeightField(RasterizeHF);
		RasterizeHF = 0;
	}

	void Create(int32 FieldSize, FVector::FReal CellSize, FVector::FReal CellHeight)
	{
		if (RasterizeHF == NULL)
		{
			const FVector::FReal DummyBounds[3] = { 0 };
			const bool bAllocateTempSpanColumns = true;	// we can't know here if at some point a NavElementData will need it, so we need to allocate it

			RasterizeHF = rcAllocHeightfield();
			rcCreateHeightfield(NULL, *RasterizeHF, FieldSize, FieldSize, DummyBounds, DummyBounds, CellSize, CellHeight, bAllocateTempSpanColumns);
		}
	}

	void Reset()
	{
		rcResetHeightfield(*RasterizeHF);
	}

	void SetupForTile(const FVector::FReal* TileBMin, const FVector::FReal* TileBMax, const FVector::FReal RasterizationLowPadding, const FVector::FReal RasterizationHighPadding)
	{
		Reset();

		rcVcopy(RasterizeHF->bmin, TileBMin);
		rcVcopy(RasterizeHF->bmax, TileBMax);

		RasterizeHF->bmin[0] -= RasterizationLowPadding;
		RasterizeHF->bmin[2] -= RasterizationLowPadding;
		RasterizeHF->bmax[0] += RasterizationHighPadding;
		RasterizeHF->bmax[2] += RasterizationHighPadding;
	}

	rcHeightfield* RasterizeHF;
};

static FVoxelCacheRasterizeContext VoxelCacheContext;

uint32 GetTileCacheSizeHelper(TArray<FNavMeshTileData>& CompressedTiles)
{
	uint32 TotalMemory = 0;
	for (int32 i = 0; i < CompressedTiles.Num(); i++)
	{
		TotalMemory += CompressedTiles[i].DataSize;
	}

	return TotalMemory;
}

//----------------------------------------------------------------------//
// FRecastTileGenerator
//----------------------------------------------------------------------//

FRecastTileGenerator::FRecastTileGenerator(FRecastNavMeshGenerator& ParentGenerator, const FIntPoint& Location, const double PendingTileCreationTime)
	: TimeSliceManager(ParentGenerator.GetTimeSliceManager())
	, RasterizeGeometryWorldRecastCoordsBounds(ForceInit)
{
	bUpdateGeometry = true;
	bHasLowAreaModifiers = false;
	bHasGeometryToRasterizeAsFilledConvexVolume = false;

	TileX = Location.X;
	TileY = Location.Y;

	TileCreationTime = PendingTileCreationTime;

	// Copy tile config from parent generator
	TileConfig = ParentGenerator.GetConfig();
	
	TileDebugSettings = ParentGenerator.GetTileDebugSettings();
	Version = ParentGenerator.GetVersion();
	AdditionalCachedData = ParentGenerator.GetAdditionalCachedData();

	ParentGeneratorWeakPtr = ((FNavDataGenerator&)ParentGenerator).AsShared();

	RasterizeGeomRecastState = ERasterizeGeomRecastTimeSlicedState::MarkWalkableTriangles;
	RasterizeGeomState = ERasterizeGeomTimeSlicedState::RasterizeGeometryTransformCoordsAndFlipIndices;
	GenerateRecastFilterState = EGenerateRecastFilterTimeSlicedState::FilterLowHangingWalkableObstacles;
	GenRecastFilterLedgeSpansYStart = 0;
	DoWorkTimeSlicedState = EDoWorkTimeSlicedState::GatherGeometryFromSources;
	GenerateTileTimeSlicedState = EGenerateTileTimeSlicedState::GenerateCompressedLayers;

	GenerateNavDataTimeSlicedState = EGenerateNavDataTimeSlicedState::Init;
	GenNavDataLayerTimeSlicedIdx = 0;
	GenCompressedLayersTimeSlicedState = EGenerateCompressedLayersTimeSliced::Init;
	RasterizeTrianglesTimeSlicedRawGeomIdx = 0;
	RasterizeTrianglesTimeSlicedInstTransformIdx = 0;

	check(ParentGenerator.GetOwner());
	TileTimeSliceSettings.FilterLedgeSpansMaxYProcess = ParentGenerator.GetOwner()->TimeSliceFilterLedgeSpansMaxYProcess;

	SolidHF = nullptr;
	CompactHF = nullptr;
}

FRecastTileGenerator::~FRecastTileGenerator()
{
	rcFreeHeightField(SolidHF);
	rcFreeCompactHeightfield(CompactHF);

	GenNavDataTimeSlicedGenerationContext.Reset();
	GenNavDataTimeSlicedAllocator.Reset();
	GenCompressedlayersTimeSlicedRasterContext.Reset();
}

void FRecastTileGenerator::Setup(const FRecastNavMeshGenerator& ParentGenerator, const TArray<FBox>& DirtyAreas)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FRecastTileGenerator_Setup);
	
	const FVector RcNavMeshOrigin = ParentGenerator.GetRcNavMeshOrigin();
	const FBox NavTotalBounds = ParentGenerator.GetTotalBounds();
	const FVector::FReal TileSizeUU = TileConfig.GetTileSizeUU();

	NavDataConfig = ParentGenerator.GetOwner()->GetConfig();

	TileBB = CalculateTileBounds(TileX, TileY, RcNavMeshOrigin, NavTotalBounds, TileSizeUU);

	if (UE::NavMesh::Private::bUseTightBoundExpansion)
	{
		TileBBExpandedForAgent = ParentGenerator.GrowBoundingBox(TileBB, /*bIncludeAgentHeight*/ false);
	}
	else
	{
		// Deprecated
		TileBBExpandedForAgent = TileBB.ExpandBy(NavDataConfig.AgentRadius * 2 + TileConfig.cs);
	}
	
	const FBox RCBox = Unreal2RecastBox(TileBB);
	const FVector Min32(RCBox.Min);
	const FVector Max32(RCBox.Max);
	rcVcopy(TileConfig.bmin, &Min32.X);
	rcVcopy(TileConfig.bmax, &Max32.X);
			
	// from passed in boxes pick the ones overlapping with tile bounds
	bFullyEncapsulatedByInclusionBounds = true;
	const TNavStatArray<FBox>& ParentBounds = ParentGenerator.GetInclusionBounds();
	if (ParentBounds.Num() > 0)
	{
		bFullyEncapsulatedByInclusionBounds = false;
		InclusionBounds.Reserve(ParentBounds.Num());
		for (const FBox& Bounds : ParentBounds)
		{
			if (Bounds.Intersect(TileBB))
			{
				InclusionBounds.Add(Bounds);
				bFullyEncapsulatedByInclusionBounds = DoesBoxContainBox(Bounds, TileBB);
			}
		}
	}

	// If there are no DirtyAreas, we expect there is a geometry change (also see usage of bRegenerateCompressedLayers).
	// Else it's a modifier change.
	const bool bGeometryChanged = (DirtyAreas.Num() == 0);
	if (!bGeometryChanged)
	{
		// Get compressed tile cache layers if they exist for this location
		CompressedLayers = ParentGenerator.GetOwner()->GetTileCacheLayers(TileX, TileY);
		for (FNavMeshTileData& LayerData : CompressedLayers)
		{
			// we don't want to modify shared state inside async task, so make sure we are unique owner
			LayerData.MakeUnique();
		}
	}

	// We have to regenerate layers data in case geometry is changed or tile cache is missing
	bRegenerateCompressedLayers = (bGeometryChanged || TileConfig.bGenerateLinks || CompressedLayers.Num() == 0);
	
	// Gather geometry for tile if it's inside navigable bounds
	// DirtyLayers might not be initialized if InclusionBounds are empty
	if (InclusionBounds.Num())
	{
		if (!bRegenerateCompressedLayers)
		{
			// Mark layers that needs to be updated
			DirtyLayers.Init(false, CompressedLayers.Num());
			for (const FNavMeshTileData& LayerData : CompressedLayers)
			{
				for (FBox DirtyBox : DirtyAreas)
				{
					if (DirtyBox.Intersect(LayerData.LayerBBox))
					{
						DirtyLayers[LayerData.LayerIndex] = true;
					}
				}
			}
		}
		
		bool bGatherGeometryNow = ParentGenerator.GatherGeometryOnGameThread();
#if !RECAST_ASYNC_REBUILDING
		if (ParentGenerator.IsTimeSliceRegenActive())
		{
			bGatherGeometryNow = false;
		}
#endif // !RECAST_ASYNC_REBUILDING

		if (bGatherGeometryNow)
		{
			GatherGeometry(ParentGenerator, bRegenerateCompressedLayers);
		}
		else
		{
			PrepareGeometrySources(ParentGenerator, bRegenerateCompressedLayers);
		}
	}
	
	UsedMemoryOnStartup = GetUsedMemCount() + sizeof(FRecastTileGenerator);
}

bool FRecastTileGenerator::HasDataToBuild() const
{
	return
		CompressedLayers.Num()
		|| Modifiers.Num()
		|| OffmeshLinks.Num()
		|| RawGeometry.Num()
		|| (InclusionBounds.Num() && NavigationRelevantData.Num() > 0);
}

FBox FRecastTileGenerator::CalculateTileBounds(int32 X, int32 Y, const FVector& RcNavMeshOrigin, const FBox& TotalNavBounds, FVector::FReal TileSizeInWorldUnits)
{
	FBox TileBox(
		RcNavMeshOrigin + (FVector(X + 0, 0, Y + 0) * TileSizeInWorldUnits),
		RcNavMeshOrigin + (FVector(X + 1, 0, Y + 1) * TileSizeInWorldUnits)
		);

	TileBox = Recast2UnrealBox(TileBox);
	TileBox.Min.Z = TotalNavBounds.Min.Z;
	TileBox.Max.Z = TotalNavBounds.Max.Z;

	// unreal coord space
	return TileBox;
}

ETimeSliceWorkResult FRecastTileGenerator::DoWorkTimeSliced()
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_DoWork);

	TSharedPtr<FNavDataGenerator, ESPMode::ThreadSafe> ParentGenerator = ParentGeneratorWeakPtr.Pin();
	ETimeSliceWorkResult WorkResult = ETimeSliceWorkResult::Succeeded;

	check(TimeSliceManager);

	if (ParentGenerator.IsValid())
	{
		switch (DoWorkTimeSlicedState)
		{
		case EDoWorkTimeSlicedState::Invalid:
		{
			ensureMsgf(false, TEXT("Invalid EDoWorkTimeSlicedState, has this function been called when its already finished processing?"));
			return ETimeSliceWorkResult::Failed;
		}
		case EDoWorkTimeSlicedState::GatherGeometryFromSources:
		{
			if (InclusionBounds.Num())
			{
				WorkResult = GatherGeometryFromSourcesTimeSliced();

				// Needs to occur after DemandLazyDataGathering
				TSharedPtr<FRecastNavMeshGenerator> RecastParentGenerator = StaticCastSharedPtr<FRecastNavMeshGenerator>(ParentGenerator);
				SetupTileConfigFromHighestResolution(*RecastParentGenerator);	

				if (WorkResult == ETimeSliceWorkResult::CallAgainNextTimeSlice)
				{
					break;
				}
			}
			DoWorkTimeSlicedState = EDoWorkTimeSlicedState::GenerateTile;
		} //fall through to next state
		case EDoWorkTimeSlicedState::GenerateTile:
		{
			WorkResult = GenerateTileTimeSliced();

			if (WorkResult != ETimeSliceWorkResult::CallAgainNextTimeSlice)
			{
				DumpAsyncData();
				DumpSyncData();		// Currently, TIME_SLICE_NAV_REGEN can only be active if not async.

				DoWorkTimeSlicedState = EDoWorkTimeSlicedState::Invalid;//Set to Invalid as we never want to call this again on this instance
			}
		}
		break;

		default:
		{
			ensureMsgf(false, TEXT("unhandled EDoWorkTimeSlicedState"));
			return ETimeSliceWorkResult::Failed;
		}
		}
	}

	return WorkResult;
}

bool FRecastTileGenerator::DoWork()
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_DoWork);

	TSharedPtr<FNavDataGenerator, ESPMode::ThreadSafe> ParentGenerator = ParentGeneratorWeakPtr.Pin();
	bool bSuccess = true;

	if (ParentGenerator.IsValid())
	{
		if (InclusionBounds.Num())
		{
			GatherGeometryFromSources();
			
			// Needs to occur after DemandLazyDataGathering
			TSharedPtr<FRecastNavMeshGenerator> RecastParentGenerator = StaticCastSharedPtr<FRecastNavMeshGenerator>(ParentGenerator);
			SetupTileConfigFromHighestResolution(*RecastParentGenerator);	
		}

		bSuccess = GenerateTile();

		DumpAsyncData();
	}

	return bSuccess;
}

void FRecastTileGenerator::DumpAsyncData()
{
	RawGeometry.Empty();
	Modifiers.Empty();
	OffmeshLinks.Empty();

	NavSystem = nullptr;
}

void FRecastTileGenerator::DumpSyncData()
{
	ensure(IsInGameThread());
	NavigationRelevantData.Empty();
}

void FRecastTileGenerator::SetupTileConfigFromHighestResolution(const FRecastNavMeshGenerator& ParentGenerator)
{
	ENavigationDataResolution HighestResolution = ENavigationDataResolution::Low;
	bool bNewResolutionFound = false;

	for (const FRecastAreaNavModifierElement& Element : Modifiers)
	{
		if (Element.NavMeshResolution != ENavigationDataResolution::Invalid)
		{
			HighestResolution = FMath::Max(HighestResolution, Element.NavMeshResolution);
			bNewResolutionFound = true;
		}	
	}
	
	check(HighestResolution != ENavigationDataResolution::Invalid);
	if (bNewResolutionFound && ParentGenerator.GetOwner()->NavMeshResolutionParams[(uint8)HighestResolution].IsValid())
	{
		// Update the TileConfig
		ParentGenerator.SetupTileConfig(HighestResolution, TileConfig);
	}
}
	
void FRecastTileGenerator::GatherGeometryFromSources()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_GatherGeometryFromSources);

	UNavigationSystemV1* NavSys = NavSystem.Get();
	if (NavSys == nullptr)
	{
		return;
	}

	for (TSharedRef<FNavigationRelevantData, ESPMode::ThreadSafe>& ElementData : NavigationRelevantData)
	{
		GatherNavigationDataGeometry(ElementData, *NavSys, NavDataConfig, bUpdateGeometry);
	}
}

ETimeSliceWorkResult FRecastTileGenerator::GatherGeometryFromSourcesTimeSliced()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_GatherGeometryFromSources);

	UNavigationSystemV1* NavSys = NavSystem.Get();
	if (NavSys == nullptr)
	{
		return ETimeSliceWorkResult::Failed;
	}

	while(NavigationRelevantData.Num())
	{
		GatherNavigationDataGeometry(NavigationRelevantData.Pop(EAllowShrinking::No), *NavSys, NavDataConfig, bUpdateGeometry);

		MARK_TIMESLICE_SECTION_DEBUG(TimeSliceManager->GetTimeSlicer(), GatherGeometryFromSources);

		if (TimeSliceManager->GetTimeSlicer().TestTimeSliceFinished())
		{
			return ETimeSliceWorkResult::CallAgainNextTimeSlice;
		}
	}

	return ETimeSliceWorkResult::Succeeded;
}

void FRecastTileGenerator::PrepareGeometrySources(const FRecastNavMeshGenerator& ParentGenerator, bool bGeometryChanged)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_PrepareGeometrySources);

	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(ParentGenerator.GetWorld());
	const FNavigationOctree* NavOctreeInstance = NavSys ? NavSys->GetNavOctree() : nullptr;
	check(NavOctreeInstance);
	NavigationRelevantData.Reset();
	NavSystem = NavSys;
	bUpdateGeometry = bGeometryChanged;
	
	const ARecastNavMesh* const OwnerNav = ParentGenerator.GetOwner();
	const bool bUseVirtualGeometryFilteringAndDirtying = OwnerNav->bUseVirtualGeometryFilteringAndDirtying;
	
	NavOctreeInstance->FindElementsWithBoundsTest(ParentGenerator.GrowBoundingBox(TileBB, /*bIncludeAgentHeight*/ false),
		[&ParentGenerator, this, bGeometryChanged, bUseVirtualGeometryFilteringAndDirtying](const FNavigationOctreeElement& Element)
	{
		const bool bShouldUse = bUseVirtualGeometryFilteringAndDirtying ?
			ParentGenerator.ShouldGenerateGeometryForOctreeElement(Element, NavDataConfig) :
			Element.ShouldUseGeometry(NavDataConfig);
		if (bShouldUse)
		{
			const bool bExportGeometry = bGeometryChanged && (Element.Data->HasGeometry() || Element.Data->IsPendingLazyGeometryGathering());
			if (bExportGeometry || 
				Element.Data->NeedAnyPendingLazyModifiersGathering() ||
				Element.Data->Modifiers.HasMetaAreas() == true || 
				Element.Data->Modifiers.IsEmpty() == false)
			{
				NavigationRelevantData.Add(Element.Data);
			}
		}
	});
}

void FRecastTileGenerator::GatherGeometry(const FRecastNavMeshGenerator& ParentGenerator, bool bGeometryChanged)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_GatherGeometry);

	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(ParentGenerator.GetWorld());
	const FNavigationOctree* NavigationOctree = NavSys ? NavSys->GetNavOctree() : nullptr;
	if (NavigationOctree == nullptr)
	{
		return;
	}
	const ARecastNavMesh* const OwnerNav = ParentGenerator.GetOwner();
	const bool bUseVirtualGeometryFilteringAndDirtying = OwnerNav->bUseVirtualGeometryFilteringAndDirtying;
	const FNavDataConfig& OwnerNavDataConfig = OwnerNav->GetConfig();

	TArray<TSharedRef<FNavigationRelevantData, ESPMode::ThreadSafe>> RelevantDataArray;

	UE_SUPPRESS(LogNavigation, VeryVerbose, UE_VLOG_BOX(OwnerNav, LogNavigation, VeryVerbose, TileBB, FColor::White, TEXT("Tile (%i, %i)"), TileX, TileY));	

	const FBox NewBounds = ParentGenerator.GrowBoundingBox(TileBB, /*bIncludeAgentHeight*/ false);
	
	NavigationOctree->FindElementsWithBoundsTest(NewBounds,
		[&RelevantDataArray, &OwnerNavDataConfig, &ParentGenerator, this, bUseVirtualGeometryFilteringAndDirtying](const FNavigationOctreeElement& Element)
	{
		const bool bShouldUse = bUseVirtualGeometryFilteringAndDirtying ?
			ParentGenerator.ShouldGenerateGeometryForOctreeElement(Element, OwnerNavDataConfig) :
			Element.ShouldUseGeometry(OwnerNavDataConfig);
		if (bShouldUse)
		{
			RelevantDataArray.Add(Element.Data);
		}
	});

	ENavigationDataResolution HighestResolution = ENavigationDataResolution::Low;
	bool bNewResolutionFound = false;
	
	for (TSharedRef<FNavigationRelevantData, ESPMode::ThreadSafe>& ElementData : RelevantDataArray)
	{
		GatherNavigationDataGeometry(ElementData, *NavSys, OwnerNavDataConfig, bGeometryChanged);

		// Keep highest resolution that is not the default.
		const ENavigationDataResolution Resolution = ElementData->Modifiers.GetNavMeshResolution();
		if (Resolution != ENavigationDataResolution::Invalid)
		{
			HighestResolution = FMath::Max(HighestResolution, Resolution);
			bNewResolutionFound = true;
		}
	}

	check(HighestResolution != ENavigationDataResolution::Invalid);
	if (bNewResolutionFound && ParentGenerator.GetOwner()->NavMeshResolutionParams[(uint8)HighestResolution].IsValid())
	{
		// Update the TileConfig
		ParentGenerator.SetupTileConfig(HighestResolution, TileConfig);
	}
}

void FRecastTileGenerator::GatherNavigationDataGeometry(const TSharedRef<FNavigationRelevantData, ESPMode::ThreadSafe>& ElementDataRef, UNavigationSystemV1& NavSys, const FNavDataConfig& OwnerNavDataConfig, const bool bGeometryChanged)
{
	bool bDumpGeometryData = false;
	FNavigationRelevantData& ElementData = ElementDataRef.Get();

#if RECAST_INTERNAL_DEBUG_DATA
	if (!IsTileDebugAllowingGeneration())
	{
		return;
	}

	if (IsTileDebugActive())
	{
		UE_LOGF(LogNavigation, Log, "Gathering geometry for tile (%i,%i): %ls.\n"
										"                       Bounds: %ls\n"
										"                       Geometry: Has=%ls Pending=%ls Slice=%ls\n"
										"                       Modifier: Has=%ls Pending=%ls",
			TileX, TileY, *ElementData.SourceElement.Get().GetFullName(),
			*ElementData.Bounds.ToString(),
			*LexToString(ElementData.HasGeometry()), *LexToString(ElementData.IsPendingLazyGeometryGathering()), *LexToString(ElementData.SupportsGatheringGeometrySlices()),
			*LexToString(ElementData.HasModifiers()), *LexToString(ElementData.NeedAnyPendingLazyModifiersGathering()));
	}
#endif // RECAST_INTERNAL_DEBUG_DATA

	if (ElementData.IsPendingLazyGeometryGathering() || ElementData.NeedAnyPendingLazyModifiersGathering())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_LazyGeometryExport);
		NavSys.DemandLazyDataGathering(ElementData);
	}
				
	if (ElementData.IsPendingLazyGeometryGathering() && ElementData.SupportsGatheringGeometrySlices())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_LandscapeSlicesExporting);

		FRecastGeometryExport GeomExport(ElementData);

		// adding a small bump to avoid special case of zero-expansion when tile bounds
		// overlap landscape's tile bounds
		ElementData.SourceElement->GeometrySliceExportDelegate.ExecuteIfBound(ElementData.SourceElement.Get(), GeomExport, TileBBExpandedForAgent);
		RecastGeometryExport::ConvertCoordDataToRecast(GeomExport.VertexBuffer);
		RecastGeometryExport::StoreCollisionCache(GeomExport);
		bDumpGeometryData = true;
	}

	// Not using the ternary operator to avoid a lvalue-to-rvalue conversion which causes copies of FCompositeNavModifier even when no meta area is used.
	// This is to avoid performance hit when using FCompositeNavModifier with a lot of internal content.
	TOptional<FCompositeNavModifier> Storage;
	const FCompositeNavModifier* ModifierInstancePtr;
	if (ElementData.Modifiers.HasMetaAreas())
	{
		Storage = ElementData.Modifiers.GetInstantiatedMetaModifier(&OwnerNavDataConfig, ElementData.SourceElement->GetWeakUObject());
		ModifierInstancePtr = &Storage.GetValue();
	}
	else
	{
		ModifierInstancePtr = &ElementData.Modifiers;
	}
	const FCompositeNavModifier& ModifierInstance = *ModifierInstancePtr;

	const bool bExportGeometry = bGeometryChanged && ElementData.HasGeometry();
	if (bExportGeometry)
	{
		if (ARecastNavMesh::IsVoxelCacheEnabled())
		{
			TNavStatArray<rcSpanCache> SpanData;
			rcSpanCache* CachedVoxels = 0;
			int32 NumCachedVoxels = 0;

			DECLARE_SCOPE_CYCLE_COUNTER(TEXT("Rasterization: prepare voxel cache"), Stat_RecastRasterCachePrep, STATGROUP_Navigation);

			if (!HasVoxelCache(ElementData.VoxelData, CachedVoxels, NumCachedVoxels))
			{

				// rasterize
				PrepareVoxelCache(ElementData.CollisionData, Unreal2RecastBox(ElementData.Bounds), ModifierInstance, SpanData);
				CachedVoxels = SpanData.GetData();
				NumCachedVoxels = SpanData.Num();

				// encode
				{
					LLM_SCOPE_BYTAG(NavigationOctree);

					const SIZE_T PrevElementMemory = ElementData.GetAllocatedSize();
					AddVoxelCache(ElementData.VoxelData, CachedVoxels, NumCachedVoxels);

					const SIZE_T NewElementMemory = ElementData.GetAllocatedSize();
					const SIZE_T ElementMemoryDelta = NewElementMemory - PrevElementMemory;
					INC_MEMORY_STAT_BY(STAT_Navigation_CollisionTreeMemory, ElementMemoryDelta);
				}
			}
		}
		else
		{
			ValidateAndAppendGeometry(ElementData, ModifierInstance);
		}

		if (bDumpGeometryData)
		{
			ElementData.CollisionData.Empty();
		}
	}
				
	if (ModifierInstance.IsEmpty() == false)
	{
		AppendModifier(ModifierInstance, ElementData.NavDataPerInstanceTransformDelegate);
	}
}

void FRecastTileGenerator::ApplyVoxelFilter(rcHeightfield* HF, FVector::FReal WalkableRadius)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_TileVoxelFilteringAsync);

	if (HF != NULL)
	{
		const int32 Width = HF->width;
		const int32 Height = HF->height;
		const FVector::FReal CellSize = HF->cs;
		const FVector::FReal CellHeight = HF->ch;
		const FVector::FReal BottomX = HF->bmin[0];
		const FVector::FReal BottomZ = HF->bmin[1];
		const FVector::FReal BottomY = HF->bmin[2];
		const int32 SpansCount = Width*Height;
		// we need to expand considered bounding boxes so that
		// it doesn't create "fake cliffs"
		const FVector::FReal ExpandBBBy = WalkableRadius*CellSize;

		const FBox* BBox = InclusionBounds.GetData();
		// optimized common case of single box
		if (InclusionBounds.Num() == 1)
		{
			const FBox BB = BBox->ExpandBy(ExpandBBBy);

			rcSpan** Span = HF->spans;

			for (int32 y = 0; y < Height; ++y)
			{
				for (int32 x = 0; x < Width; ++x)
				{
					const FVector::FReal SpanX = -(BottomX + x * CellSize);
					const FVector::FReal SpanY = -(BottomY + y * CellSize);

					// mark all spans outside of InclusionBounds as unwalkable
					for (rcSpan* s = *Span; s; s = s->next)
					{
						if (s->data.area == RC_WALKABLE_AREA)
						{
							const FVector::FReal SpanMin = CellHeight * s->data.smin + BottomZ;
							const FVector::FReal SpanMax = CellHeight * s->data.smax + BottomZ;

							const FVector SpanMinV(SpanX-CellSize, SpanY-CellSize, SpanMin);
							const FVector SpanMaxV(SpanX, SpanY, SpanMax);

							if (BB.IsInside(SpanMinV) == false && BB.IsInside(SpanMaxV) == false)
							{
								s->data.area = RC_NULL_AREA;
							}
						}
					}
					++Span;
				}
			}
		}
		else
		{
			TArray<FBox> Bounds;
			Bounds.Reserve(InclusionBounds.Num());

			for (int32 i = 0; i < InclusionBounds.Num(); ++i, ++BBox)
			{	
				Bounds.Add(BBox->ExpandBy(ExpandBBBy));
			}
			const int32 BoundsCount = Bounds.Num();

			rcSpan** Span = HF->spans;

			for (int32 y = 0; y < Height; ++y)
			{
				for (int32 x = 0; x < Width; ++x)
				{
					const FVector::FReal SpanX = -(BottomX + x * CellSize);
					const FVector::FReal SpanY = -(BottomY + y * CellSize);

					// mark all spans outside of InclusionBounds as unwalkable
					for (rcSpan* s = *Span; s; s = s->next)
					{
						if (s->data.area == RC_WALKABLE_AREA)
						{
							const FVector::FReal SpanMin = CellHeight * s->data.smin + BottomZ;
							const FVector::FReal SpanMax = CellHeight * s->data.smax + BottomZ;

							const FVector SpanMinV(SpanX-CellSize, SpanY-CellSize, SpanMin);
							const FVector SpanMaxV(SpanX, SpanY, SpanMax);

							bool bIsInsideAnyBB = false;
							const FBox* BB = Bounds.GetData();
							for (int32 BoundIndex = 0; BoundIndex < BoundsCount; ++BoundIndex, ++BB)
							{
								if (BB->IsInside(SpanMinV) || BB->IsInside(SpanMaxV))
								{
									bIsInsideAnyBB = true;
									break;
								}
							}

							if (bIsInsideAnyBB == false)
							{
								s->data.area = RC_NULL_AREA;
							}
						}
					}
					++Span;
				}
			}
		}
	}
}

void FRecastTileGenerator::InitRasterizationMaskArray(const rcHeightfield* InSolidHF, TInlineMaskArray& OutRasterizationMasks)
{
	const int CellCount = InSolidHF->width * InSolidHF->height;
	OutRasterizationMasks.SetNumUninitialized(CellCount);
	const uint8 AllowAllFlags = 0xFF;
	FMemory::Memset(OutRasterizationMasks.GetData(), AllowAllFlags, CellCount*sizeof(TInlineMaskArray::ElementType));
}

// Deprecated
void FRecastTileGenerator::MarkDynamicAreas(dtTileCacheLayer& Layer)
{
	FNavMeshBuildContext BuildContext(*this);
	return MarkDynamicAreas(BuildContext, Layer);
}

// Deprecated
void FRecastTileGenerator::MarkDynamicArea(const FAreaNavModifier& Modifier, const FTransform& LocalToWorld, dtTileCacheLayer& Layer)
{
	FNavMeshBuildContext BuildContext(*this);
	return MarkDynamicArea(BuildContext, Modifier, LocalToWorld, Layer);
}

// Deprecated
void FRecastTileGenerator::MarkDynamicArea(const FAreaNavModifier& Modifier, const FTransform& LocalToWorld, dtTileCacheLayer& Layer, const int32 AreaId, const int32* ReplaceAreaIdPtr)
{
	FNavMeshBuildContext BuildContext(*this);
	return MarkDynamicArea(BuildContext, Modifier, LocalToWorld, Layer, AreaId, ReplaceAreaIdPtr);
}

void FRecastTileGenerator::PrepareVoxelCache(const TNavStatArray<uint8>& RawCollisionCache, const FBox& RecastBounds, const FCompositeNavModifier& InModifier, TNavStatArray<rcSpanCache>& SpanData)
{
	// tile's geometry: voxel cache (only for synchronous rebuilds)
	const int32 WalkableClimbVX = TileConfig.walkableClimb;
	const FVector::FReal WalkableSlopeCos = FMath::Cos(FMath::DegreesToRadians(TileConfig.walkableSlopeAngle));
	const FVector::FReal RasterizationLowPadding = TileConfig.borderSize.low * TileConfig.cs;
	const FVector::FReal RasterizationHighPadding = TileConfig.borderSize.high * TileConfig.cs;

	FRecastGeometryCache CachedCollisions(RawCollisionCache.GetData());

	VoxelCacheContext.SetupForTile(TileConfig.bmin, TileConfig.bmax, RasterizationLowPadding, RasterizationHighPadding);

	float SlopeCosPerActor = UE_REAL_TO_FLOAT(WalkableSlopeCos);
	CachedCollisions.Header.SlopeOverride.ModifyWalkableFloorZ(SlopeCosPerActor);

	// rasterize triangle soup
	TNavStatArray<uint8> TriAreas;
	TriAreas.AddZeroed(CachedCollisions.Header.NumFaces);

	rcMarkWalkableTrianglesCos(0, SlopeCosPerActor,
		CachedCollisions.Verts, CachedCollisions.Header.NumVerts,
		CachedCollisions.Indices, CachedCollisions.Header.NumFaces,
		TriAreas.GetData());

	// Apply surface area if specified
	const TSubclassOf<UNavAreaBase> AreaClass = FRecastRawGeometryElement::GetCollisionAreaClass(InModifier);
	if (AreaClass)
	{
		const int32* AreaIDPtr = AdditionalCachedData.AreaClassToIdMap.Find(AreaClass);
		if (AreaIDPtr)
		{
			rcApplyAreaToTriangles(TriAreas.GetData(), CachedCollisions.Header.NumFaces, IntCastChecked<uint8>(*AreaIDPtr));
		}
	}

	TInlineMaskArray RasterizationMasks;
	if (InModifier.GetMaskFillCollisionUnderneathForNavmesh())
	{
		const int32 Mask = ~RC_PROJECT_TO_BOTTOM;
		for (const FAreaNavModifier& ModifierArea : InModifier.GetAreas())
		{
			MarkRasterizationMask(0 /*ctx*/, VoxelCacheContext.RasterizeHF, ModifierArea, FTransform::Identity, Mask, RasterizationMasks);
		}
	}

	const rcRasterizationFlags Flags = FRecastRawGeometryElement::GetRasterizationFlags(InModifier);

	const FVector::FReal* BoundsMin = RecastBounds.IsValid ? &RecastBounds.Min.X : nullptr;
	const FVector::FReal* BoundsMax = RecastBounds.IsValid ? &RecastBounds.Max.X : nullptr;
	TInlineMaskArray::ElementType* MaskArray = RasterizationMasks.Num() > 0 ? RasterizationMasks.GetData() : nullptr;
	rcRasterizeTriangles(0, CachedCollisions.Verts, CachedCollisions.Header.NumVerts,
		CachedCollisions.Indices, TriAreas.GetData(), CachedCollisions.Header.NumFaces,
		*VoxelCacheContext.RasterizeHF, WalkableClimbVX, Flags, MaskArray, BoundsMin, BoundsMax);

	const int32 NumSpans = rcCountSpans(0, *VoxelCacheContext.RasterizeHF);
	if (NumSpans > 0)
	{
		SpanData.AddZeroed(NumSpans);
		rcCacheSpans(0, *VoxelCacheContext.RasterizeHF, SpanData.GetData());
	}
}

bool FRecastTileGenerator::HasVoxelCache(const TNavStatArray<uint8>& RawVoxelCache, rcSpanCache*& CachedVoxels, int32& NumCachedVoxels) const
{
	FRecastVoxelCache VoxelCache(RawVoxelCache.GetData());
	for (FRecastVoxelCache::FTileInfo* iTile = VoxelCache.Tiles; iTile; iTile = iTile->NextTile)
	{
		if (iTile->TileX == TileX && iTile->TileY == TileY)
		{
			CachedVoxels = iTile->SpanData;
			NumCachedVoxels = iTile->NumSpans;
			return true;
		}
	}
	
	return false;
}

void FRecastTileGenerator::AddVoxelCache(TNavStatArray<uint8>& RawVoxelCache, const rcSpanCache* CachedVoxels, const int32 NumCachedVoxels) const
{
	if (RawVoxelCache.Num() == 0)
	{
		RawVoxelCache.AddZeroed(sizeof(int32));
	}

	int32* NumTiles = (int32*)RawVoxelCache.GetData();
	*NumTiles = *NumTiles + 1;

	const int32 NewCacheIdx = RawVoxelCache.Num();
	const int32 HeaderSize = sizeof(FRecastVoxelCache::FTileInfo);
	const int32 VoxelsSize = sizeof(rcSpanCache) * NumCachedVoxels;
	const int32 EntrySize = HeaderSize + VoxelsSize;
	RawVoxelCache.AddZeroed(EntrySize);

	FRecastVoxelCache::FTileInfo* TileInfo = (FRecastVoxelCache::FTileInfo*)(RawVoxelCache.GetData() + NewCacheIdx);
	TileInfo->TileX = TileX;
	TileInfo->TileY = TileY;
	TileInfo->NumSpans = NumCachedVoxels;

	FMemory::Memcpy(RawVoxelCache.GetData() + NewCacheIdx + HeaderSize, CachedVoxels, VoxelsSize);
}

void FRecastTileGenerator::AppendModifier(const FCompositeNavModifier& Modifier, const FNavDataPerInstanceTransformDelegate& InTransformsDelegate)
{
	// append all offmesh links (not included in compress layers)
	OffmeshLinks.Append(Modifier.GetSimpleLinks());

	// evaluate custom links
	const FCustomLinkNavModifier* LinkModifier = Modifier.GetCustomLinks().GetData();
	for (int32 i = 0; i < Modifier.GetCustomLinks().Num(); i++, LinkModifier++)
	{
		FSimpleLinkNavModifier SimpleLinkCollection(UNavLinkDefinition::GetLinksDefinition(LinkModifier->GetNavLinkClass()), LinkModifier->LocalToWorld);
		OffmeshLinks.Add(SimpleLinkCollection);
	}

	bHasGeometryToRasterizeAsFilledConvexVolume = bHasGeometryToRasterizeAsFilledConvexVolume || Modifier.GetRasterizeAsFilledConvexVolume();

	// Navmesh resolutions is a modifier without area, if present, it must not be skipped.
	if (Modifier.GetAreas().Num() == 0 && Modifier.GetNavMeshResolution() == ENavigationDataResolution::Invalid)
	{
		return;
	}

	bHasLowAreaModifiers = bHasLowAreaModifiers || Modifier.HasLowAreaModifiers();
	
	FRecastAreaNavModifierElement ModifierElement;

	// Gather per instance transforms if any
	if (InTransformsDelegate.IsBound())
	{
		InTransformsDelegate.Execute(TileBBExpandedForAgent, ModifierElement.PerInstanceTransform);
		// skip this modifier in case there is no instances for this tile
		if (ModifierElement.PerInstanceTransform.Num() == 0)
		{
			return;
		}
	}

	ModifierElement.Areas = Modifier.GetAreas();
	ModifierElement.bMaskFillCollisionUnderneathForNavmesh = Modifier.GetMaskFillCollisionUnderneathForNavmesh();
	ModifierElement.NavMeshResolution = Modifier.GetNavMeshResolution();

#if RECAST_INTERNAL_DEBUG_DATA
	if (IsTileDebugActive())
	{
		TStringBuilder<256> StrBuilder;
		for (const FTransform& Transform : ModifierElement.PerInstanceTransform)
		{
			StrBuilder.AppendChar(' ').Append(Transform.ToString());
		}
		UE_LOGF(LogNavigation, Log, "    Modifier transforms [%d]:%ls", ModifierElement.PerInstanceTransform.Num(), *StrBuilder);

		for (const FAreaNavModifier& AreaNavModifier : ModifierElement.Areas)
		{
			UE_LOGF(LogNavigation, Log, "    Adding area modifier:\n"
										"                           Type: %ls\n"
										"                           ApplyMode: %ls\n"
										"                           AreaClass: %ls\n"
										"                           Bounds: %ls",
			*LexToString(AreaNavModifier.GetShapeType()),
			*LexToString(AreaNavModifier.GetApplyMode()),
			*AreaNavModifier.GetAreaClass()->GetName(),
			*AreaNavModifier.GetBounds().ToString());
		}
	}
#endif

	Modifiers.Add(MoveTemp(ModifierElement));
}

void FRecastTileGenerator::ValidateAndAppendGeometry(const FNavigationRelevantData& ElementData, const FCompositeNavModifier& InModifier)
{
	if (ElementData.IsCollisionDataValid())
	{
		AppendGeometry(ElementData, InModifier, ElementData.NavDataPerInstanceTransformDelegate);
	}
}

void FRecastTileGenerator::AppendGeometry(const FNavigationRelevantData& ElementData, const FCompositeNavModifier& InModifier, const FNavDataPerInstanceTransformDelegate& InTransformsDelegate)
{	
	const TNavStatArray<uint8>& RawCollisionCache = ElementData.CollisionData;
	if (RawCollisionCache.Num() == 0)
	{
		return;
	}
	
	FRecastRawGeometryElement GeometryElement;
	GeometryElement.RasterizationFlags = FRecastRawGeometryElement::GetRasterizationFlags(InModifier);
	GeometryElement.AreaClass = FRecastRawGeometryElement::GetCollisionAreaClass(InModifier);

	FRecastGeometryCache CollisionCache(RawCollisionCache.GetData());
	
	// Gather per instance transforms
	if (InTransformsDelegate.IsBound())
	{
		InTransformsDelegate.Execute(TileBBExpandedForAgent, GeometryElement.PerInstanceTransform);
		if (GeometryElement.PerInstanceTransform.Num() == 0)
		{
			return;
		}
	}
	
	const int32 NumCoords = CollisionCache.Header.NumVerts * 3;
	const int32 NumIndices = CollisionCache.Header.NumFaces * 3;
	if (NumIndices > 0)
	{
		UE_LOGF(LogNavigationDataBuild, VeryVerbose, "%s adding %i vertices from %ls.",
			__FUNCTION__, CollisionCache.Header.NumVerts, *ElementData.SourceElement.Get().GetFullName());

		GeometryElement.GeomCoords.SetNumUninitialized(NumCoords);
		GeometryElement.GeomIndices.SetNumUninitialized(NumIndices);

		FMemory::Memcpy(GeometryElement.GeomCoords.GetData(), CollisionCache.Verts, sizeof(FVector::FReal) * NumCoords);
		FMemory::Memcpy(GeometryElement.GeomIndices.GetData(), CollisionCache.Indices, sizeof(int32) * NumIndices);

		GeometryElement.GeomCoordsBounds = Unreal2RecastBox(ElementData.Bounds); 

		RawGeometry.Add(MoveTemp(GeometryElement));
	}	
}

ETimeSliceWorkResult FRecastTileGenerator::GenerateTileTimeSliced()
{
	UE_LOGF(LogNavigation, Verbose, "Building tile (time sliced): (%i,%i)", TileX, TileY);

	FNavMeshBuildContext BuildContext(*this);
	ETimeSliceWorkResult WorkResult = ETimeSliceWorkResult::Succeeded;

	dtLinkBuilderData linkBuiderData;
	linkBuiderData.generatingLinks = TileConfig.bGenerateLinks;
	
	check(TimeSliceManager);

	switch (GenerateTileTimeSlicedState)
	{
	case EGenerateTileTimeSlicedState::Invalid:
	{
		ensureMsgf(false, TEXT("Invalid EGenerateTileTimeSlicedState, has this function been called when its already finished time processong?"));
		return ETimeSliceWorkResult::Failed;
	}
	case EGenerateTileTimeSlicedState::GenerateCompressedLayers:
	{
		if (bRegenerateCompressedLayers)
		{
			const ETimeSliceWorkResult WorkResultCompressed = GenerateCompressedLayersTimeSliced(BuildContext);

			if (WorkResultCompressed == ETimeSliceWorkResult::Succeeded)
			{
				GenerateTileTimeSlicedState = EGenerateTileTimeSlicedState::GenerateNavigationData;
				// Mark all layers as dirty
				DirtyLayers.Init(true, CompressedLayers.Num());
			}
			else if (WorkResultCompressed == ETimeSliceWorkResult::Failed)
			{
				GenerateTileTimeSlicedState = EGenerateTileTimeSlicedState::Invalid;
				return ETimeSliceWorkResult::Failed;
			}

			if (TimeSliceManager->GetTimeSlicer().IsTimeSliceFinishedCached())
			{
				return ETimeSliceWorkResult::CallAgainNextTimeSlice;
			}
		}
		else
		{
			GenerateTileTimeSlicedState = EGenerateTileTimeSlicedState::GenerateNavigationData;
		}
	} //fall through to next state
	case EGenerateTileTimeSlicedState::GenerateNavigationData:
	{
		WorkResult = GenerateNavigationDataTimeSliced(BuildContext, linkBuiderData);

		if (WorkResult != ETimeSliceWorkResult::CallAgainNextTimeSlice)
		{
			GenerateTileTimeSlicedState = EGenerateTileTimeSlicedState::Invalid;
		}
	}
	break;
	default:
	{
		ensureMsgf(false, TEXT("unhandled EGenerateTileTimeSlicedState"));
		return ETimeSliceWorkResult::Failed;
	}
	};

	// it's possible to have valid generation with empty resulting tile (no navigable geometry in tile)
	return WorkResult;
}

bool FRecastTileGenerator::GenerateTile()
{
#if RECAST_INTERNAL_DEBUG_DATA
	const double StartStamp = FPlatformTime::Seconds();
	double PostCompressLayerStamp = StartStamp;
#endif // RECAST_INTERNAL_DEBUG_DATA

	UE_LOGF(LogNavigation, Verbose, "Building tile: (%i,%i)", TileX, TileY);
	
	FNavMeshBuildContext BuildContext(*this);
	bool bSuccess = true;

	dtLinkBuilderData LinkBuiderData;
	LinkBuiderData.generatingLinks = TileConfig.bGenerateLinks;

	if (bRegenerateCompressedLayers)
	{
		CompressedLayers.Reset();
		bSuccess = GenerateCompressedLayers(BuildContext, LinkBuiderData);

#if RECAST_INTERNAL_DEBUG_DATA
		PostCompressLayerStamp = FPlatformTime::Seconds();
#endif // RECAST_INTERNAL_DEBUG_DATA

		if (bSuccess)
		{
			// Mark all layers as dirty
			DirtyLayers.Init(true, CompressedLayers.Num());
		}
	}

	if (bSuccess)
	{
		bSuccess = GenerateNavigationData(BuildContext, LinkBuiderData);
	}
	
#if RECAST_INTERNAL_DEBUG_DATA	
	const double EndStamp = FPlatformTime::Seconds();
	BuildContext.InternalDebugData.BuildTime = EndStamp - StartStamp;
	BuildContext.InternalDebugData.BuildCompressedLayerTime = PostCompressLayerStamp - StartStamp;
	BuildContext.InternalDebugData.BuildNavigationDataTime = EndStamp - PostCompressLayerStamp;
	BuildContext.InternalDebugData.Resolution = (unsigned char)TileConfig.TileResolution;
#endif // RECAST_INTERNAL_DEBUG_DATA
	
	// it's possible to have valid generation with empty resulting tile (no navigable geometry in tile)
	return bSuccess;
}

struct FTileRasterizationContext
{
	FTileRasterizationContext() : LayerSet(nullptr), RasterizationFlags(rcRasterizationFlags(0))
	{
	}

	~FTileRasterizationContext()
	{
		rcFreeHeightfieldLayerSet(LayerSet);
	}

	rcRasterizationFlags GetRasterizationFlags() const { return RasterizationFlags; }
	void SetRasterizationFlags(rcRasterizationFlags Value) { RasterizationFlags = Value; }

	struct rcHeightfieldLayerSet* LayerSet;
	TArray<FNavMeshTileData> Layers;
	FRecastTileGenerator::TInlineMaskArray RasterizationMasks;

private:
	rcRasterizationFlags RasterizationFlags;
};

bool FRecastTileGenerator::CreateHeightField(FNavMeshBuildContext& BuildContext)
{
#if RECAST_INTERNAL_DEBUG_DATA
	if (!IsTileDebugAllowingGeneration())
	{
		return false;
	}
#endif // RECAST_INTERNAL_DEBUG_DATA

	SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastCreateHeightField);

	// Guard against pathological borderSize values that would cause int32 overflow in width*height,
	// or an unrecoverable OOM allocation. This is a last-resort check; ComputeConfigBorderSizes
	// should have already warned if LinkSpillDistance was extreme.
	// Use int64 for the addition to avoid signed overflow (UB) before the check executes.
	const int64 size64 = (int64)TileConfig.tileSize + TileConfig.borderSize.low + TileConfig.borderSize.high;
	if (size64 <= 0 || size64 > UE::NavMesh::Private::MaxHeightfieldSize)
	{
		BuildContext.log(RC_LOG_ERROR, "CreateHeightField: Heightfield size %lld is out of range [1, %d]. "
			"Skipping tile.", size64, UE::NavMesh::Private::MaxHeightfieldSize);
		return false;
	}
	const int size = (int)size64;
	TileConfig.width = size;
	TileConfig.height = size;

	const FVector::FReal BBoxPaddingLow = TileConfig.borderSize.low * TileConfig.cs;
	const FVector::FReal BBoxPaddingHigh = TileConfig.borderSize.high * TileConfig.cs;
	TileConfig.bmin[0] -= BBoxPaddingLow;
	TileConfig.bmin[2] -= BBoxPaddingLow;
	TileConfig.bmax[0] += BBoxPaddingHigh;
	TileConfig.bmax[2] += BBoxPaddingHigh;

	BuildContext.log(RC_LOG_PROGRESS, "CreateHeightField:");
	BuildContext.log(RC_LOG_PROGRESS, " - %d x %d cells", TileConfig.width, TileConfig.height);

	const bool bHasGeometry = RawGeometry.Num() > 0;

	// Allocate voxel heightfield where we rasterize our input data to.
	if (bHasGeometry)
	{
		SolidHF = rcAllocHeightfield();
		if (SolidHF == nullptr)
		{
			BuildContext.log(RC_LOG_ERROR, "CreateHeightField: Out of memory 'SolidHF'.");
			return false;
		}
		if (!rcCreateHeightfield(&BuildContext, *SolidHF, TileConfig.width, TileConfig.height, TileConfig.bmin, TileConfig.bmax, TileConfig.cs, TileConfig.ch, bHasGeometryToRasterizeAsFilledConvexVolume))
		{
			BuildContext.log(RC_LOG_ERROR, "CreateHeightField: Could not create solid heightfield.");
			return false;
		}
	}
	return true;
}

ETimeSliceWorkResult FRecastTileGenerator::RasterizeGeometryRecastTimeSliced(FNavMeshBuildContext& BuildContext, 
	const TArray<FVector::FReal>& Coords, const TArray<int32>& Indices, const FBox& CoordsBounds, 
	const rcRasterizationFlags RasterizationFlags, FTileRasterizationContext& RasterContext, TSubclassOf<UNavAreaBase> AreaClass)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Navigation_RasterizeGeometryRecast);

	check(TimeSliceManager);

	const int32 NumFaces = Indices.Num() / 3;
	const int32 NumVerts = Coords.Num() / 3;

	switch (RasterizeGeomRecastState)
	{
	case ERasterizeGeomRecastTimeSlicedState::MarkWalkableTriangles:
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Navigation_MarkWalkableTriangles);

		RasterizeGeomRecastTriAreas.AddZeroed(NumFaces);

		rcMarkWalkableTriangles(&BuildContext, TileConfig.walkableSlopeAngle,
			Coords.GetData(), NumVerts, Indices.GetData(), NumFaces,
			RasterizeGeomRecastTriAreas.GetData());

		// Apply surface area if specified
		if (AreaClass)
		{
			const int32* AreaIDPtr = AdditionalCachedData.AreaClassToIdMap.Find(AreaClass);
			if (AreaIDPtr)
			{
				rcApplyAreaToTriangles(RasterizeGeomRecastTriAreas.GetData(), NumFaces, IntCastChecked<uint8>(*AreaIDPtr));
			}
		}

		RasterizeGeomRecastState = ERasterizeGeomRecastTimeSlicedState::RasterizeTriangles;

		MARK_TIMESLICE_SECTION_DEBUG(TimeSliceManager->GetTimeSlicer(), MarkWalkableTriangles);

		if (TimeSliceManager->GetTimeSlicer().TestTimeSliceFinished())
		{
			return ETimeSliceWorkResult::CallAgainNextTimeSlice;
		}
	}// fall through to next state
	case ERasterizeGeomRecastTimeSlicedState::RasterizeTriangles:
	{
		ComputeRasterizationMasks(BuildContext, RasterContext);

		QUICK_SCOPE_CYCLE_COUNTER(STAT_Navigation_RasterizeGeomRecastRasterizeTriangles);

		const FVector::FReal* BoundsMin = CoordsBounds.IsValid ? &CoordsBounds.Min.X : nullptr;
		const FVector::FReal* BoundsMax = CoordsBounds.IsValid ? &CoordsBounds.Max.X : nullptr;
		const TInlineMaskArray::ElementType* MaskArray = RasterContext.RasterizationMasks.Num() > 0 ? RasterContext.RasterizationMasks.GetData() : nullptr;
		rcRasterizeTriangles(&BuildContext,
			Coords.GetData(), NumVerts,
			Indices.GetData(), RasterizeGeomRecastTriAreas.GetData(), NumFaces,
			*SolidHF, TileConfig.walkableClimb, RasterizationFlags, MaskArray, BoundsMin, BoundsMax);

#if RECAST_INTERNAL_DEBUG_DATA	
		BuildContext.InternalDebugData.TriangleCount += NumFaces;
#endif // RECAST_INTERNAL_DEBUG_DATA
			
		RasterizeGeomRecastTriAreas.Reset();

		//reset this so next call we start by marking walkable triangles
		RasterizeGeomRecastState = ERasterizeGeomRecastTimeSlicedState::MarkWalkableTriangles;

		MARK_TIMESLICE_SECTION_DEBUG(TimeSliceManager->GetTimeSlicer(), RasterizeTriangles);

		TimeSliceManager->GetTimeSlicer().TestTimeSliceFinished();
	}
	break;
	default:
	{
		ensureMsgf(false, TEXT("unhandled ERasterizeGeomRecastTimeSlicedState"));
		return ETimeSliceWorkResult::Failed;
	}
	}
	return ETimeSliceWorkResult::Succeeded;
}

void FRecastTileGenerator::RasterizeGeometryRecast(FNavMeshBuildContext& BuildContext, 
	const TArray<FVector::FReal>& Coords, const TArray<int32>& Indices, const FBox& CoordsBounds, 
	const rcRasterizationFlags RasterizationFlags, FTileRasterizationContext& RasterContext, TSubclassOf<UNavAreaBase> AreaClass)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Navigation_RasterizeGeometryRecast);

	const int32 NumFaces = Indices.Num() / 3;
	const int32 NumVerts = Coords.Num() / 3;

	RasterizeGeomRecastTriAreas.AddZeroed(NumFaces);

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Navigation_MarkWalkableTriangles);

		rcMarkWalkableTriangles(&BuildContext, TileConfig.walkableSlopeAngle,
			Coords.GetData(), NumVerts, Indices.GetData(), NumFaces,
			RasterizeGeomRecastTriAreas.GetData());

		// Apply surface area if specified
		if (AreaClass)
		{
			const int32* AreaIDPtr = AdditionalCachedData.AreaClassToIdMap.Find(AreaClass);
			if (AreaIDPtr)
			{
				rcApplyAreaToTriangles(RasterizeGeomRecastTriAreas.GetData(), NumFaces, IntCastChecked<uint8>(*AreaIDPtr));
			}
		}
	}

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Navigation_RasterizeGeomRecastRasterizeTriangles);

		const FVector::FReal* BoundsMin = CoordsBounds.IsValid ? &CoordsBounds.Min.X : nullptr;
		const FVector::FReal* BoundsMax = CoordsBounds.IsValid ? &CoordsBounds.Max.X : nullptr;
		const TInlineMaskArray::ElementType* MaskArray = RasterContext.RasterizationMasks.Num() > 0 ? RasterContext.RasterizationMasks.GetData() : nullptr;
		rcRasterizeTriangles(&BuildContext,
			Coords.GetData(), NumVerts,
			Indices.GetData(), RasterizeGeomRecastTriAreas.GetData(), NumFaces,
			*SolidHF, TileConfig.walkableClimb, RasterizationFlags, MaskArray, BoundsMin, BoundsMax);
	}

#if RECAST_INTERNAL_DEBUG_DATA
	BuildContext.InternalDebugData.TriangleCount += NumFaces;

	if (IsTileDebugActive() && TileDebugSettings.bCollisionGeometry) 
	{
		TArray<FVector::FReal> Normals;
		Normals.AddZeroed(NumFaces*3);

		rcCalcTriNormals(Coords.GetData(), NumVerts, Indices.GetData(), NumFaces, Normals.GetData());

		constexpr FVector::FReal TextureScale = 1.;
		duDebugDrawTriMesh(&BuildContext.InternalDebugData, Coords.GetData(), NumVerts, Indices.GetData(), Normals.GetData(), NumFaces, RasterizeGeomRecastTriAreas.GetData(), TextureScale);
	}

	BuildContext.InternalDebugData.TriangleCount += NumFaces;
#endif // RECAST_INTERNAL_DEBUG_DATA

	RasterizeGeomRecastTriAreas.Reset();
}

void FRecastTileGenerator::RasterizeGeometryTransformCoordsAndFlipIndices(const TArray<FVector::FReal>& Coords, const TArray<int32>& Indices, const FBox& CoordsBounds, const FTransform& LocalToWorld)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Navigation_RasterizeGeometryTransformCoordsAndFlipIndices);

	RasterizeGeometryWorldRecastCoords.SetNumUninitialized(Coords.Num(), EAllowShrinking::No);
	RasterizeGeometryWorldRecastCoordsBounds = FBox(ForceInit);

	FMatrix LocalToRecastWorld = LocalToWorld.ToMatrixWithScale() * Unreal2RecastMatrix();

	// Convert geometry to recast world space
	for (int32 i = 0; i < Coords.Num(); i += 3)
	{
		// collision cache stores coordinates in recast space, convert them to unreal and transform to recast world space
		FVector WorldRecastCoord = LocalToRecastWorld.TransformPosition(Recast2UnrealPoint(&Coords[i]));

		RasterizeGeometryWorldRecastCoords[i + 0] = WorldRecastCoord.X;
		RasterizeGeometryWorldRecastCoords[i + 1] = WorldRecastCoord.Y;
		RasterizeGeometryWorldRecastCoords[i + 2] = WorldRecastCoord.Z;
	}

	//Flip the order of indices for each triangle if the source object is mirrored
	bRasterizeGeometryUseFlippedIndices = false;
	if (LocalToWorld.GetDeterminant() < 0.f)
	{
		bRasterizeGeometryUseFlippedIndices = true;
		RasterizeGeometryFlippedIndices.SetNumUninitialized(Indices.Num(), EAllowShrinking::No);

		for (int32 i = 0; i < Indices.Num(); i += 3)
		{
			RasterizeGeometryFlippedIndices[i] = Indices[i + 2];
			RasterizeGeometryFlippedIndices[i + 1] = Indices[i + 1];
			RasterizeGeometryFlippedIndices[i + 2] = Indices[i];
		}
	}

	if (CoordsBounds.IsValid)
	{
		RasterizeGeometryWorldRecastCoordsBounds = Recast2UnrealBox(CoordsBounds).TransformBy(LocalToRecastWorld);
	}
}

ETimeSliceWorkResult FRecastTileGenerator::RasterizeGeometryTimeSliced(
	FNavMeshBuildContext& BuildContext, 
	const TArray<FVector::FReal>& Coords, 
	const TArray<int32>& Indices, 
	const FBox& CoordsBounds, 
	const FTransform& LocalToWorld, 
	const rcRasterizationFlags RasterizationFlags, 
	FTileRasterizationContext& RasterContext, 
	TSubclassOf<UNavAreaBase> AreaClass)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Navigation_RasterizeGeometry);

	check(TimeSliceManager);

	ETimeSliceWorkResult WorkResult = ETimeSliceWorkResult::Succeeded;

	switch (RasterizeGeomState)
	{
	case ERasterizeGeomTimeSlicedState::RasterizeGeometryTransformCoordsAndFlipIndices:
	{
		RasterizeGeometryTransformCoordsAndFlipIndices(Coords, Indices, CoordsBounds, LocalToWorld);

		RasterizeGeomState = ERasterizeGeomTimeSlicedState::RasterizeGeometryRecast;

		MARK_TIMESLICE_SECTION_DEBUG(TimeSliceManager->GetTimeSlicer(), RasterizeGeometryTransformCoordsAndFlipIndices);

		if (TimeSliceManager->GetTimeSlicer().TestTimeSliceFinished())
		{
			return ETimeSliceWorkResult::CallAgainNextTimeSlice;
		}
	}// fall through to next state
	case ERasterizeGeomTimeSlicedState::RasterizeGeometryRecast:
	{
		const TArray<int32>& RasterizeGeometryIndices = bRasterizeGeometryUseFlippedIndices ? RasterizeGeometryFlippedIndices : Indices;
		WorkResult = RasterizeGeometryRecastTimeSliced(BuildContext, RasterizeGeometryWorldRecastCoords, RasterizeGeometryIndices, 
			RasterizeGeometryWorldRecastCoordsBounds, RasterizationFlags, RasterContext, AreaClass);

		if (WorkResult != ETimeSliceWorkResult::CallAgainNextTimeSlice)
		{
			//if we have finished rasterizing this geometry then reset RasterizeGeomTimeSlicedState so next time this function is called we go back to RasterizeGeometryTransformCoordsAndFlipIndices first
			RasterizeGeomState = ERasterizeGeomTimeSlicedState::RasterizeGeometryTransformCoordsAndFlipIndices;
		}
	}
	break;
	default:
	{
		ensureMsgf(false, TEXT("unhandled ERasterizeGeomTimeSlicedState"));
		return ETimeSliceWorkResult(ETimeSliceWorkResult::Failed);
	}
	}
	return WorkResult;
}

void FRecastTileGenerator::RasterizeGeometry(
	FNavMeshBuildContext& BuildContext, 
	const TArray<FVector::FReal>& Coords, 
	const TArray<int32>& Indices, 
	const FBox& CoordsBounds, 
	const FTransform& LocalToWorld, 
	const rcRasterizationFlags RasterizationFlags, 
	FTileRasterizationContext& RasterContext, 
	TSubclassOf<UNavAreaBase> AreaClass)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Navigation_RasterizeGeometry);

	RasterizeGeometryTransformCoordsAndFlipIndices(Coords, Indices, CoordsBounds, LocalToWorld);
	const TArray<int32>& RasterizeGeometryIndices = bRasterizeGeometryUseFlippedIndices ? RasterizeGeometryFlippedIndices : Indices;
	RasterizeGeometryRecast(BuildContext, RasterizeGeometryWorldRecastCoords, RasterizeGeometryIndices, 
		RasterizeGeometryWorldRecastCoordsBounds, RasterizationFlags, RasterContext, AreaClass);
}

ETimeSliceWorkResult FRecastTileGenerator::RasterizeTrianglesTimeSliced(FNavMeshBuildContext& BuildContext, FTileRasterizationContext& RasterContext)
{
	// Rasterize geometry
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastRasterizeTriangles)

	check(TimeSliceManager);

	while (RasterizeTrianglesTimeSlicedRawGeomIdx < RawGeometry.Num())
	{
		const FRecastRawGeometryElement& Element = RawGeometry[RasterizeTrianglesTimeSlicedRawGeomIdx];
		if (Element.PerInstanceTransform.Num() > 0)
		{
			while (RasterizeTrianglesTimeSlicedInstTransformIdx < Element.PerInstanceTransform.Num())
			{
				const FTransform& InstanceTransform = Element.PerInstanceTransform[RasterizeTrianglesTimeSlicedInstTransformIdx];
				const ETimeSliceWorkResult WorkResult = RasterizeGeometryTimeSliced(BuildContext, Element.GeomCoords, Element.GeomIndices, 
					Element.GeomCoordsBounds, InstanceTransform, Element.RasterizationFlags, RasterContext, Element.AreaClass);
			
				//the original code just kept calling the RasterizeGeometry() functions and had no return type, 
				//so we will process the next layer (if we are not needing to process this layer again next time slice) 
				if (TimeSliceManager->GetTimeSlicer().IsTimeSliceFinishedCached())
				{
					if (WorkResult != ETimeSliceWorkResult::CallAgainNextTimeSlice)
					{
						++RasterizeTrianglesTimeSlicedInstTransformIdx;
					}

					return ETimeSliceWorkResult::CallAgainNextTimeSlice;
				}

				++RasterizeTrianglesTimeSlicedInstTransformIdx;
			}
			//reset RasterizeTrianglesTimeSlicedIdx 
			RasterizeTrianglesTimeSlicedInstTransformIdx = 0;
		}
		else
		{
			const ETimeSliceWorkResult WorkResult = RasterizeGeometryRecastTimeSliced(BuildContext, Element.GeomCoords, Element.GeomIndices, 
				Element.GeomCoordsBounds, Element.RasterizationFlags, RasterContext, Element.AreaClass);
	
			if (TimeSliceManager->GetTimeSlicer().IsTimeSliceFinishedCached())
			{
				if (WorkResult != ETimeSliceWorkResult::CallAgainNextTimeSlice)
				{
					++RasterizeTrianglesTimeSlicedRawGeomIdx;
				}
				return ETimeSliceWorkResult::CallAgainNextTimeSlice;
			}
		}
		++RasterizeTrianglesTimeSlicedRawGeomIdx;
	}

	//return sucess as non timesliced functionality does not detect failure here
	return ETimeSliceWorkResult::Succeeded;
}

void FRecastTileGenerator::RasterizeTriangles(FNavMeshBuildContext& BuildContext, FTileRasterizationContext& RasterContext)
{
	// Rasterize geometry
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastRasterizeTriangles)

	for (int32 RawGeomIdx = 0; RawGeomIdx < RawGeometry.Num(); ++RawGeomIdx)
	{
		const FRecastRawGeometryElement& Element = RawGeometry[RawGeomIdx];
		if (Element.PerInstanceTransform.Num() > 0)
		{
			for (const FTransform& InstanceTransform : Element.PerInstanceTransform)
			{
				RasterizeGeometry(BuildContext, Element.GeomCoords, Element.GeomIndices, 
					Element.GeomCoordsBounds, InstanceTransform, Element.RasterizationFlags, RasterContext, Element.AreaClass);
			}
		}
		else
		{
			RasterizeGeometryRecast(BuildContext, Element.GeomCoords, Element.GeomIndices, 
				Element.GeomCoordsBounds, Element.RasterizationFlags, RasterContext, Element.AreaClass);
		}
	}
}

void FRecastTileGenerator::GenerateRecastFilter(FNavMeshBuildContext& BuildContext)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastFilter)

	// TileConfig.walkableHeight is set to 1 when marking low spans, calculate real value for filtering
	const int32 FilterWalkableHeight = FMath::CeilToInt(TileConfig.AgentHeight / static_cast<float>(TileConfig.ch));

	// Once all geometry is rasterized, we do initial pass of filtering to
	// remove unwanted overhangs caused by the conservative rasterization
	// as well as filter spans where the character cannot possibly stand.
	{
		rcFilterLowHangingWalkableObstacles(&BuildContext, TileConfig.walkableClimb, *SolidHF);
	}
	{
		SCOPE_CYCLE_COUNTER(STAT_Navigation_FilterLedgeSpans)
		
		rcFilterLedgeSpans(&BuildContext, TileConfig.walkableHeight, TileConfig.walkableClimb,
			(rcNeighborSlopeFilterMode)TileConfig.LedgeSlopeFilterMode, TileConfig.maxStepFromWalkableSlope, TileConfig.ch, *SolidHF);
	}
	if (!TileConfig.bMarkLowHeightAreas)
	{
		rcFilterWalkableLowHeightSpans(&BuildContext, TileConfig.walkableHeight, *SolidHF);
	}
	else if (TileConfig.bFilterLowSpanFromTileCache)
	{
		// TODO: investigate if creating detailed 2D map from active modifiers is cheap enough
		// for now, switch on presence of those modifiers, will save memory as long as they are sparse (should be)

		if (TileConfig.bFilterLowSpanSequences && bHasLowAreaModifiers)
		{
			rcFilterWalkableLowHeightSpansSequences(&BuildContext, FilterWalkableHeight, *SolidHF);
		}
		else
		{
			rcFilterWalkableLowHeightSpans(&BuildContext, FilterWalkableHeight, *SolidHF);
		}
	}
}

ETimeSliceWorkResult FRecastTileGenerator::GenerateRecastFilterTimeSliced(FNavMeshBuildContext& BuildContext)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastFilter)

	check(TimeSliceManager);

	ETimeSliceWorkResult WorkResult = ETimeSliceWorkResult::Succeeded;

	switch (GenerateRecastFilterState)
	{
	case EGenerateRecastFilterTimeSlicedState::FilterLowHangingWalkableObstacles:
	{
		// Once all geometry is rasterized, we do initial pass of filtering to
		// remove unwanted overhangs caused by the conservative rasterization
		// as well as filter spans where the character cannot possibly stand.
		rcFilterLowHangingWalkableObstacles(&BuildContext, TileConfig.walkableClimb, *SolidHF);
		GenerateRecastFilterState = EGenerateRecastFilterTimeSlicedState::FilterLedgeSpans;
	}// fall through to next state
	case EGenerateRecastFilterTimeSlicedState::FilterLedgeSpans:
	{
		SCOPE_CYCLE_COUNTER(STAT_Navigation_FilterLedgeSpans)

		bool DoIter = true;

		do
		{
			rcFilterLedgeSpans(&BuildContext, TileConfig.walkableHeight, TileConfig.walkableClimb,
				(rcNeighborSlopeFilterMode)TileConfig.LedgeSlopeFilterMode, TileConfig.maxStepFromWalkableSlope, TileConfig.ch, 
				GenRecastFilterLedgeSpansYStart, TileTimeSliceSettings.FilterLedgeSpansMaxYProcess, *SolidHF);

			GenRecastFilterLedgeSpansYStart += TileTimeSliceSettings.FilterLedgeSpansMaxYProcess;

			if (GenRecastFilterLedgeSpansYStart >= SolidHF->height)
			{
				GenRecastFilterLedgeSpansYStart = 0;
				DoIter = false;
				GenerateRecastFilterState = EGenerateRecastFilterTimeSlicedState::FilterWalkableLowHeightSpans;
			}

			MARK_TIMESLICE_SECTION_DEBUG(TimeSliceManager->GetTimeSlicer(), FilterLedgeSpans);

			// Only FilterLedge Spans has been found to be slow so we only actually test the timeslice here for this function
			if (TimeSliceManager->GetTimeSlicer().TestTimeSliceFinished())
			{
				return ETimeSliceWorkResult::CallAgainNextTimeSlice;
			}
		} while (DoIter);
	}// fall through to next state
	case EGenerateRecastFilterTimeSlicedState::FilterWalkableLowHeightSpans:
	{
		// TileConfig.walkableHeight is set to 1 when marking low spans, calculate real value for filtering
		const int32 FilterWalkableHeight = FMath::CeilToInt(TileConfig.AgentHeight / static_cast<float>(TileConfig.ch));

		if (!TileConfig.bMarkLowHeightAreas)
		{
			rcFilterWalkableLowHeightSpans(&BuildContext, TileConfig.walkableHeight, *SolidHF);
		}
		else if (TileConfig.bFilterLowSpanFromTileCache)
		{
			// TODO: investigate if creating detailed 2D map from active modifiers is cheap enough
			// for now, switch on presence of those modifiers, will save memory as long as they are sparse (should be)
			if (TileConfig.bFilterLowSpanSequences && bHasLowAreaModifiers)
			{
				rcFilterWalkableLowHeightSpansSequences(&BuildContext, FilterWalkableHeight, *SolidHF);
			}
			else
			{
				rcFilterWalkableLowHeightSpans(&BuildContext, FilterWalkableHeight, *SolidHF);
			}
		}
		GenerateRecastFilterState = EGenerateRecastFilterTimeSlicedState::FilterLowHangingWalkableObstacles;
	}
	break;
	default:
	{
		ensureMsgf(false, TEXT("unhandled EGenerateRecastFilterTimeSlicedState"));
		return ETimeSliceWorkResult::Failed;
	}
	}

	return ETimeSliceWorkResult::Succeeded;
}

bool FRecastTileGenerator::BuildCompactHeightField(FNavMeshBuildContext& BuildContext)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastBuildCompactHeightField);

	// Compact the heightfield so that it is faster to handle from now on.
	// This will result more cache coherent data as well as the neighbors
	// between walkable cells will be calculated.
	CompactHF = rcAllocCompactHeightfield();
	if (CompactHF == nullptr)
	{
		BuildContext.log(RC_LOG_ERROR, "BuildCompactHeightField: Out of memory 'CompactHF'.");
		return false;
	}
	if (!rcBuildCompactHeightfield(&BuildContext, TileConfig.walkableHeight, TileConfig.walkableClimb, *SolidHF, *CompactHF))
	{
		const int SpanCount = rcGetHeightFieldSpanCount(&BuildContext, *SolidHF);
		if (SpanCount > 0)
		{
			BuildContext.log(RC_LOG_ERROR, "BuildCompactHeightField: Could not build compact data.");
		}
		// else there's just no spans to walk on (no spans at all or too small/sparse)
		else
		{
			BuildContext.log(RC_LOG_WARNING, "BuildCompactHeightField: no walkable spans - aborting");
		}
		return false;
	}
	return true;
}

bool FRecastTileGenerator::RecastErodeWalkable(FNavMeshBuildContext& BuildContext)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastErodeWalkable);

	// TileConfig.walkableHeight is set to 1 when marking low spans, calculate real value for filtering
	const int32 FilterWalkableHeight = FMath::CeilToInt(TileConfig.AgentHeight / static_cast<float>(TileConfig.ch));

	if (static_cast<float>(TileConfig.walkableRadius) > RECAST_VERY_SMALL_AGENT_RADIUS)
	{
		uint8 FilterFlags = 0;
		if (TileConfig.bFilterLowSpanSequences)
		{
			FilterFlags = RC_LOW_FILTER_POST_PROCESS | (TileConfig.bFilterLowSpanFromTileCache ? 0 : RC_LOW_FILTER_SEED_SPANS);
		}

		const bool bEroded = TileConfig.bMarkLowHeightAreas ?
			rcErodeWalkableAndLowAreas(&BuildContext, TileConfig.walkableRadius, FilterWalkableHeight, RECAST_LOW_AREA, FilterFlags, *CompactHF) :
			rcErodeWalkableArea(&BuildContext, TileConfig.walkableRadius, *CompactHF);

		if (!bEroded)
		{
			BuildContext.log(RC_LOG_ERROR, "GenerateCompressedLayers: Could not erode.");
			return false;
		}

	}
	else if (TileConfig.bMarkLowHeightAreas)
	{
		rcMarkLowAreas(&BuildContext, FilterWalkableHeight, RECAST_LOW_AREA, *CompactHF);
	}

	return true;
}

bool FRecastTileGenerator::RecastBuildLayers(FNavMeshBuildContext& BuildContext, FTileRasterizationContext& RasterContext)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastBuildLayers);

	RasterContext.LayerSet = rcAllocHeightfieldLayerSet();
	if (RasterContext.LayerSet == nullptr)
	{
		BuildContext.log(RC_LOG_ERROR, "RecastBuildLayers: Out of memory 'LayerSet'.");
		return false;
	}

	if (TileConfig.regionPartitioning == RC_REGION_MONOTONE)
	{
		if (!rcBuildHeightfieldLayersMonotone(&BuildContext, *CompactHF, TileConfig.borderSize, TileConfig.walkableHeight, *RasterContext.LayerSet))
		{
			BuildContext.log(RC_LOG_ERROR, "RecastBuildLayers: Could not build heightfield layers.");
			return false;

		}
	}
	else if (TileConfig.regionPartitioning == RC_REGION_WATERSHED)
	{
		if (!rcBuildDistanceField(&BuildContext, *CompactHF))
		{
			BuildContext.log(RC_LOG_ERROR, "RecastBuildLayers: Could not build distance field.");
			return false;
		}

		if (!rcBuildHeightfieldLayers(&BuildContext, *CompactHF, TileConfig.borderSize, TileConfig.walkableHeight, *RasterContext.LayerSet))
		{
			BuildContext.log(RC_LOG_ERROR, "RecastBuildLayers: Could not build heightfield layers.");
			return false;
		}
	}
	else
	{
		if (!rcBuildHeightfieldLayersChunky(&BuildContext, *CompactHF, TileConfig.borderSize, TileConfig.walkableHeight, TileConfig.regionChunkSize, *RasterContext.LayerSet))
		{
			BuildContext.log(RC_LOG_ERROR, "RecastBuildLayers: Could not build heightfield layers.");
			return false;
		}
	}
	return true;
}

bool FRecastTileGenerator::RecastBuildTileCache(FNavMeshBuildContext& BuildContext, FTileRasterizationContext& RasterContext)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastBuildTileCache);

	const int32 NumLayers = RasterContext.LayerSet->nlayers;

	// use this to expand vertically layer's bounds
	// this is needed to allow off-mesh connections that are not quite
	// touching tile layer still connect with it.
	const FVector::FReal StepHeights = TileConfig.AgentMaxClimb;

	FTileCacheCompressor TileCompressor;
	for (int32 i = 0; i < NumLayers; i++)
	{
		const rcHeightfieldLayer* layer = &RasterContext.LayerSet->layers[i];

		// Store header
		dtTileCacheLayerHeader header;
		header.version = DT_TILECACHE_VERSION;

		// Tile layer location in the navmesh.
		header.tx = TileX;
		header.ty = TileY;
		header.tlayer = i;
		dtVcopy(header.bmin, layer->bmin);
		dtVcopy(header.bmax, layer->bmax);

		// Tile info.
		header.width = (unsigned short)layer->width;
		header.height = (unsigned short)layer->height;
		header.minx = (unsigned short)layer->minx;
		header.maxx = (unsigned short)layer->maxx;
		header.miny = (unsigned short)layer->miny;
		header.maxy = (unsigned short)layer->maxy;

		// Layer bounds in unreal coords
		FBox LayerBBox = Recast2UnrealBox(header.bmin, header.bmax);
		LayerBBox.Min.Z -= StepHeights;
		LayerBBox.Max.Z += StepHeights;

		// Compress tile layer
		uint8* TileData = nullptr;
		int32 TileDataSize = 0;
		const dtStatus status = dtBuildTileCacheLayer(&TileCompressor, &header, layer->heights, layer->areas, layer->cons, &TileData, &TileDataSize);
		if (dtStatusFailed(status))
		{
			dtFree(TileData, DT_ALLOC_PERM_TILE_DATA);
			BuildContext.log(RC_LOG_ERROR, "RecastBuildTileCache: failed to build layer.");
			return false;
		}
#if !UE_BUILD_SHIPPING && OUTPUT_NAV_TILE_LAYER_COMPRESSION_DATA
		else
		{
			const int gridSize = (int)header.width * (int)header.height;
			const int bufferSize = gridSize * 4;

			FPlatformMisc::CustomNamedStat("NavTileLayerUncompSize", static_cast<float>(bufferSize), "NavMesh", "Bytes");
			FPlatformMisc::CustomNamedStat("NavTileLayerCompSize", static_cast<float>(TileDataSize), "NavMesh", "Bytes");
		}
#endif

		// copy compressed data to new buffer in rasterization context
		// (TileData allocates a lots of space, but only first TileDataSize bytes hold compressed data)

		uint8* CompressedData = (uint8*)dtAlloc(TileDataSize * sizeof(uint8), DT_ALLOC_PERM_TILE_DATA);
		if (CompressedData == nullptr)
		{
			dtFree(TileData, DT_ALLOC_PERM_TILE_DATA);
			BuildContext.log(RC_LOG_ERROR, "RecastBuildTileCache: Out of memory 'CompressedData'.");
			return false;
		}

		FMemory::Memcpy(CompressedData, TileData, TileDataSize);
		RasterContext.Layers.Add(FNavMeshTileData(CompressedData, TileDataSize, i, LayerBBox));

		dtFree(TileData, DT_ALLOC_PERM_TILE_DATA);

		const int32 UncompressedSize = ((sizeof(dtTileCacheLayerHeader) + 3) & ~3) + (3 * header.width * header.height);
		const float Inv1kB = 1.0f / 1024.0f;
		BuildContext.log(RC_LOG_PROGRESS, ">> Cache[%d,%d:%d] = %.2fkB (full:%.2fkB rate:%.2f%%)",
			TileX,
			TileY,
			i,
			static_cast<float>(TileDataSize) * Inv1kB,
			static_cast<float>(UncompressedSize) * Inv1kB,
			static_cast<float>(TileDataSize) / static_cast<float>(UncompressedSize));
	}
	CompressedLayers = MoveTemp(RasterContext.Layers);
	return true;
}

ETimeSliceWorkResult FRecastTileGenerator::GenerateCompressedLayersTimeSliced(FNavMeshBuildContext& BuildContext)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastBuildCompressedLayers);

	check(TimeSliceManager);

	FTileRasterizationContext* RasterContext = GenCompressedlayersTimeSlicedRasterContext.Get();

	switch (GenCompressedLayersTimeSlicedState)
	{
	case EGenerateCompressedLayersTimeSliced::Invalid:
	{
		ensureMsgf(false, TEXT("Invalid EGenerateCompressedLayersTimeSliced, has this function been called when its already finished processing?"));
		return ETimeSliceWorkResult::Failed;
	}

	case EGenerateCompressedLayersTimeSliced::Init:
	{
		CompressedLayers.Reset();
		GenCompressedlayersTimeSlicedRasterContext = MakeUnique<FTileRasterizationContext>();
		RasterContext = GenCompressedlayersTimeSlicedRasterContext.Get();
		GenCompressedLayersTimeSlicedState = EGenerateCompressedLayersTimeSliced::CreateHeightField;
	} // fall through to next state
	case EGenerateCompressedLayersTimeSliced::CreateHeightField:
	{
		if (!CreateHeightField(BuildContext))
		{
			GenCompressedLayersTimeSlicedState = EGenerateCompressedLayersTimeSliced::Invalid;
			//no need to check time slice as not much work done
			return ETimeSliceWorkResult::Failed;
		}

		GenCompressedLayersTimeSlicedState = EGenerateCompressedLayersTimeSliced::RasterizeTriangles;

		MARK_TIMESLICE_SECTION_DEBUG(TimeSliceManager->GetTimeSlicer(), CreateHeightField);

		if (TimeSliceManager->GetTimeSlicer().TestTimeSliceFinished())
		{
			return ETimeSliceWorkResult::CallAgainNextTimeSlice;
		}
	} // fall through to next state
	case EGenerateCompressedLayersTimeSliced::RasterizeTriangles:
	{
		const ETimeSliceWorkResult WorkResult = RasterizeTrianglesTimeSliced(BuildContext, *RasterContext);

		//original code did not care about success or failure here
		if (WorkResult != ETimeSliceWorkResult::CallAgainNextTimeSlice)
		{
			GenCompressedLayersTimeSlicedState = EGenerateCompressedLayersTimeSliced::EmptyLayers;
		}

		if (TimeSliceManager->GetTimeSlicer().IsTimeSliceFinishedCached())
		{
			return ETimeSliceWorkResult::CallAgainNextTimeSlice;
		}
	} // fall through to next state
	case EGenerateCompressedLayersTimeSliced::EmptyLayers:
	{
		if (!SolidHF || SolidHF->pools == 0)
		{
			BuildContext.log(RC_LOG_WARNING, "GenerateCompressedLayersTimeSliced: empty tile - aborting");

			//no need to check time slice as not much work done
			GenCompressedLayersTimeSlicedState = EGenerateCompressedLayersTimeSliced::Invalid;
			return ETimeSliceWorkResult::Succeeded;
		}

		GenCompressedLayersTimeSlicedState = EGenerateCompressedLayersTimeSliced::VoxelFilter;
		//no need to check time slice as not much work done
	}// fall through to next state
	case EGenerateCompressedLayersTimeSliced::VoxelFilter:
	{
		GenCompressedLayersTimeSlicedState = EGenerateCompressedLayersTimeSliced::RecastFilter;
		// Reject voxels outside generation boundaries
		if (TileConfig.bPerformVoxelFiltering && !bFullyEncapsulatedByInclusionBounds)
		{
			ApplyVoxelFilter(SolidHF, TileConfig.walkableRadius);

			MARK_TIMESLICE_SECTION_DEBUG(TimeSliceManager->GetTimeSlicer(), VoxelFilter);

			if (TimeSliceManager->GetTimeSlicer().TestTimeSliceFinished())
			{
				return ETimeSliceWorkResult::CallAgainNextTimeSlice;
			}
		}
	}// fall through to next state
	case EGenerateCompressedLayersTimeSliced::RecastFilter:
	{
		const ETimeSliceWorkResult WorkResult = GenerateRecastFilterTimeSliced(BuildContext);

		// Non timesliced code this is based on did not care about success or failure here.
		if (WorkResult != ETimeSliceWorkResult::CallAgainNextTimeSlice)
		{
			GenCompressedLayersTimeSlicedState = EGenerateCompressedLayersTimeSliced::CompactHeightField;
		}

		if (TimeSliceManager->GetTimeSlicer().IsTimeSliceFinishedCached())
		{
			return ETimeSliceWorkResult::CallAgainNextTimeSlice;
		}
	}// fall through to next state
	case EGenerateCompressedLayersTimeSliced::CompactHeightField:
	{
		if (!BuildCompactHeightField(BuildContext))
		{
			//no need to check time slice as not much work done
			GenCompressedLayersTimeSlicedState = EGenerateCompressedLayersTimeSliced::Invalid;

			return ETimeSliceWorkResult::Failed;
		}

		GenCompressedLayersTimeSlicedState = EGenerateCompressedLayersTimeSliced::ErodeWalkable;

		MARK_TIMESLICE_SECTION_DEBUG(TimeSliceManager->GetTimeSlicer(), CompactHeightField);

		if (TimeSliceManager->GetTimeSlicer().TestTimeSliceFinished())
		{
			return ETimeSliceWorkResult::CallAgainNextTimeSlice;
		}
	}// fall through to next state
	case EGenerateCompressedLayersTimeSliced::ErodeWalkable:
	{
		if (!RecastErodeWalkable(BuildContext))
		{
			//no need to check time slice as not much work done
			GenCompressedLayersTimeSlicedState = EGenerateCompressedLayersTimeSliced::Invalid;

			return ETimeSliceWorkResult::Failed;
		}

		GenCompressedLayersTimeSlicedState = EGenerateCompressedLayersTimeSliced::BuildLayers;

		MARK_TIMESLICE_SECTION_DEBUG(TimeSliceManager->GetTimeSlicer(), ErodeWalkable);

		if (TimeSliceManager->GetTimeSlicer().TestTimeSliceFinished())
		{
			return ETimeSliceWorkResult::CallAgainNextTimeSlice;
		}
	}// fall through to next state
	case EGenerateCompressedLayersTimeSliced::BuildLayers:
	{
		const bool bRecastBuildLayers = RecastBuildLayers(BuildContext, *RasterContext);

		MARK_TIMESLICE_SECTION_DEBUG(TimeSliceManager->GetTimeSlicer(), BuildLayers);

		//this could have done a fair amount of work either way so check time slice
		TimeSliceManager->GetTimeSlicer().TestTimeSliceFinished();

		if (!bRecastBuildLayers)
		{
			GenCompressedLayersTimeSlicedState = EGenerateCompressedLayersTimeSliced::Invalid;

			return ETimeSliceWorkResult::Failed;
		}

		GenCompressedLayersTimeSlicedState = EGenerateCompressedLayersTimeSliced::BuildTileCache;

		if (TimeSliceManager->GetTimeSlicer().IsTimeSliceFinishedCached())
		{
			return ETimeSliceWorkResult::CallAgainNextTimeSlice;
		}
	}// fall through to next state
	case EGenerateCompressedLayersTimeSliced::BuildTileCache:
	{
		GenCompressedLayersTimeSlicedState = EGenerateCompressedLayersTimeSliced::Invalid;

		const bool bRecastBuildTileCache = RecastBuildTileCache(BuildContext, *RasterContext);
	
		MARK_TIMESLICE_SECTION_DEBUG(TimeSliceManager->GetTimeSlicer(), BuildTileCache);

		//this could have done a fair amount of work either way so check time slice
		TimeSliceManager->GetTimeSlicer().TestTimeSliceFinished();

		if (!bRecastBuildTileCache)
		{
			return ETimeSliceWorkResult::Failed;
		}
	}
	break;

	default:
	{
		ensureMsgf(false, TEXT("unknow EGenerateCompressedLayersTimeSliced state"));
		return ETimeSliceWorkResult::Failed;
	}
	}

	return ETimeSliceWorkResult::Succeeded;
}

namespace UE::NavMesh::Private
{
	// Make sure LinkSpillDistance and CellSize has been computed before computing borders.
	void ComputeConfigBorderSizes(const bool bGeneratingLinks, FRecastBuildConfig& InOutConfig)
	{
		int BorderForLinksVx = 0;
		if (bGeneratingLinks)
		{
			BorderForLinksVx = (int)FMath::CeilToInt((rcReal)InOutConfig.LinkSpillDistance / InOutConfig.cs) + InOutConfig.walkableRadius;

			// Warn if the border is large enough to cause a significant allocation overhead.
			// At this point we do not clamp -- a large JumpLength is a valid (if expensive) configuration.
			// CreateHeightField will abort the tile if size would overflow int32.
			if (BorderForLinksVx > InOutConfig.tileSize)
			{
				UE_LOGF(LogNavigationDataBuild, Warning,
					"ComputeConfigBorderSizes: BorderForLinks (%d vx) exceeds tileSize (%d vx). "
					"This will increase memory usage during nav build.",
					BorderForLinksVx, InOutConfig.tileSize);
			}
		}

		// +1 for voxelization rounding, +1 for ledge neighbor access, +1 for occasional errors
		const int BorderForAgentVx = InOutConfig.walkableRadius + 3;

		// Borders must be at least the size of BorderForAgentVx.
		InOutConfig.borderSize.low = UE::NavMesh::Private::bUseAsymetricBorderSizes ? BorderForAgentVx : FMath::Max(BorderForAgentVx, BorderForLinksVx);
		InOutConfig.borderSize.high = FMath::Max(BorderForAgentVx, BorderForLinksVx);
	}
	
#if RECAST_INTERNAL_DEBUG_DATA
	void DrawHeightfield(const EHeightFieldRenderMode RenderMode, duDebugDraw* dd, const rcHeightfield& hf)
	{
		if (RenderMode == EHeightFieldRenderMode::Solid)
		{
			duDebugDrawHeightfieldSolid(dd, hf);
		}
		else
		{
			duDebugDrawHeightfieldWalkable(dd, hf);
		}
	}
#endif //RECAST_INTERNAL_DEBUG_DATA
};

bool FRecastTileGenerator::GenerateCompressedLayers(FNavMeshBuildContext& BuildContext, const dtLinkBuilderData& InLinkBuilderData)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastBuildCompressedLayers);

	FTileRasterizationContext RasterContext;
	CompressedLayers.Reset();

	if (!CreateHeightField(BuildContext))
	{
		return false;
	}
		
	ComputeRasterizationMasks(BuildContext, RasterContext);

	RasterizeTriangles(BuildContext, RasterContext);
	if (!SolidHF || SolidHF->pools == 0)
	{
		BuildContext.log(RC_LOG_WARNING, "GenerateCompressedLayers: empty tile - aborting");
		return true;
	}

#if RECAST_INTERNAL_DEBUG_DATA
	if (IsTileDebugActive())
	{
		if (TileDebugSettings.bHeightfieldFromRasterization)
		{
			UE::NavMesh::Private::DrawHeightfield(TileDebugSettings.HeightFieldRenderMode, &BuildContext.InternalDebugData, *SolidHF);
		}
		if (TileDebugSettings.bHeightfieldBounds)
		{
			duDebugDrawHeightfieldBounds(&BuildContext.InternalDebugData, *SolidHF);
		}
	}
#endif

	// Reject voxels outside generation boundaries
	if (TileConfig.bPerformVoxelFiltering && !bFullyEncapsulatedByInclusionBounds)
	{
		ApplyVoxelFilter(SolidHF, TileConfig.walkableRadius);
	}

#if RECAST_INTERNAL_DEBUG_DATA
	if (IsTileDebugActive() && TileDebugSettings.bHeightfieldPostInclusionBoundsFiltering)
	{
		UE::NavMesh::Private::DrawHeightfield(TileDebugSettings.HeightFieldRenderMode, &BuildContext.InternalDebugData, *SolidHF);
	}
#endif

	GenerateRecastFilter(BuildContext);

#if RECAST_INTERNAL_DEBUG_DATA
	if (IsTileDebugActive() && TileDebugSettings.bHeightfieldPostHeightFiltering)
	{
		UE::NavMesh::Private::DrawHeightfield(TileDebugSettings.HeightFieldRenderMode, &BuildContext.InternalDebugData, *SolidHF);
	}
#endif

	if (!BuildCompactHeightField(BuildContext))
	{
		return false;
	}

#if RECAST_INTERNAL_DEBUG_DATA
	if (IsTileDebugActive() && TileDebugSettings.bCompactHeightfield)
	{
		duDebugDrawCompactHeightfieldSolid(&BuildContext.InternalDebugData, *CompactHF);
	}
#endif

	if (!RecastErodeWalkable(BuildContext))
	{
		return false;
	}

#if RECAST_INTERNAL_DEBUG_DATA
	if (IsTileDebugActive() && TileDebugSettings.bCompactHeightfieldEroded)
	{
		duDebugDrawCompactHeightfieldSolid(&BuildContext.InternalDebugData, *CompactHF);
	}
#endif

	if (!RecastBuildLayers(BuildContext, RasterContext))
	{
		return false;
	}

#if RECAST_INTERNAL_DEBUG_DATA
	if (IsTileDebugActive())
	{
		if (TileDebugSettings.bHeightFieldLayers)
		{
			duDebugDrawHeightfieldLayers(&BuildContext.InternalDebugData, *RasterContext.LayerSet);	
		}
		
		if (TileDebugSettings.bCompactHeightfieldRegions)
		{
			duDebugDrawCompactHeightfieldRegions(&BuildContext.InternalDebugData, *CompactHF);	
		}

		if (TileDebugSettings.bCompactHeightfieldDistances)
		{
			duDebugDrawCompactHeightfieldDistance(&BuildContext.InternalDebugData, *CompactHF);	
		}
	}
#endif
	
	return RecastBuildTileCache(BuildContext, RasterContext);
}

struct FTileGenerationContext
{
	FTileGenerationContext(dtTileCacheAlloc* MyAllocator) :	Allocator(MyAllocator)	{}

	~FTileGenerationContext()
	{
		ResetIntermediateData();
	}

	void ResetIntermediateData()
	{
		if (Allocator)
		{
			dtFreeTileCacheLayer(Allocator, Layer);
			Layer = nullptr;
			dtFreeTileCacheDistanceField(Allocator, DistanceField);
			DistanceField = nullptr;
			dtFreeTileCacheContourSet(Allocator, ContourSet);
			ContourSet = nullptr;
#if WITH_NAVMESH_CLUSTER_LINKS
			dtFreeTileCacheClusterSet(Allocator, ClusterSet);
			ClusterSet = nullptr;
#endif // WITH_NAVMESH_CLUSTER_LINKS
			dtFreeTileCachePolyMesh(Allocator, PolyMesh);
			PolyMesh = nullptr;
			dtFreeTileCachePolyMeshDetail(Allocator, DetailMesh);
			DetailMesh = nullptr;
			// don't clear NavigationData here!
		}
	}

	struct dtTileCacheAlloc* Allocator = nullptr;
	struct dtTileCacheLayer* Layer = nullptr;
	struct dtTileCacheDistanceField* DistanceField = nullptr;
	struct dtTileCacheContourSet* ContourSet = nullptr;
#if WITH_NAVMESH_CLUSTER_LINKS
	struct dtTileCacheClusterSet* ClusterSet = nullptr;
#endif //WITH_NAVMESH_CLUSTER_LINKS
	struct dtTileCachePolyMesh* PolyMesh = nullptr;
	struct dtTileCachePolyMeshDetail* DetailMesh = nullptr;
	TArray<FNavMeshTileData> NavigationData;
};

static void AddGeneratedLinks(
	TInlineGeneratedNavLinkArray& OutGeneratedLinks,
	const dtLinkBuilderConfig& linkBuilderConfig,
	const dtNavLinkBuilder& linkBuilder,
	const dtReal* TileBoundsMin,
	const dtReal* TileBoundsMax)
{
	// Helper to check if a point is inside the tile's XZ bounds
	auto IsInsideTileXZ = [&TileBoundsMin, &TileBoundsMax](const dtReal* pos) -> bool
	{
		return pos[0] >= TileBoundsMin[0] && pos[0] <= TileBoundsMax[0]
			&& pos[2] >= TileBoundsMin[2] && pos[2] <= TileBoundsMax[2];
	};

	auto AddLinkSharedLambda = [&OutGeneratedLinks](const dtNavLinkBuilderJumpConfig& config) -> FGeneratedNavigationLink&
	{
		FGeneratedNavigationLink& NewLink = OutGeneratedLinks.Emplace_GetRef();
		NewLink.bIsGenerated = true;
		NewLink.NavLinkId = FNavLinkId(config.linkUserId);
		return NewLink;
	};

	// Used when we're provided with identical up and down areas. In this situation, a single bidirectional link is sufficient for our needs
	auto AddBidirectionalLinkLambda = [&IsInsideTileXZ , &AddLinkSharedLambda](const dtNavLinkBuilderJumpConfig& config, const dtReal* posA, const dtReal* posB)
	{
		const FVector UnrealPosA = Recast2UnrealPoint(posA);
		const FVector UnrealPosB = Recast2UnrealPoint(posB);

		// Check if the endpoints are in the current tile and, if posA isn't inside this one, flip the direction.
		// Always keep the direction from owner position to far position.

		const bool bAInside = IsInsideTileXZ(posA);
		const bool bBInside = IsInsideTileXZ(posB);
		const bool bSwapForOwnership = !bAInside && bBInside;

		const FVector OwnerPos = bSwapForOwnership ? UnrealPosB : UnrealPosA;
		const FVector FarPos = bSwapForOwnership ? UnrealPosA : UnrealPosB;

		FGeneratedNavigationLink& NewLink = AddLinkSharedLambda(config);
		NewLink.generatedLinkArea = config.downDirArea;
		NewLink.generatedLinkPolyFlag = config.downDirPolyFlag;
		NewLink.Left = OwnerPos;
		NewLink.Right = FarPos;
	};

	// Used when we're provided with different up and down areas. In this situation, we need to create two different one-way link representations in order to meet our needs
	auto AddDirectionalLinksLambda = [&IsInsideTileXZ, &AddLinkSharedLambda](const dtNavLinkBuilderJumpConfig& config, const dtReal* posA, const dtReal* posB)
	{
		const FVector UnrealPosA = Recast2UnrealPoint(posA);
		const FVector UnrealPosB = Recast2UnrealPoint(posB);

		const bool bAInside = IsInsideTileXZ(posA);
		const bool bBInside = IsInsideTileXZ(posB);

		// If posA is not inside the tile but posB is, swap directions for ownership.
		const bool bSwap = !bAInside && bBInside;
		
		const bool bABDown = UnrealPosA.Z > UnrealPosB.Z;

		// If directions are swapped but posA is above posB, reverse the directionality of the links. This is to preserve the downDirArea going down, and the upDirArea going up.
		const bool bDownIsRightToLeft = (bSwap == bABDown);

		const FVector OwnerPos = bSwap ? UnrealPosB : UnrealPosA;
		const FVector FarPos = bSwap ? UnrealPosA : UnrealPosB;

		// Both directions of the links share the same Left (owner) and Right (far) positions.
		// They are distinguished by Direction: the "down" link uses LeftToRight, while the "up"
		// link uses RightToLeft, which sets DT_OFFMESH_CON_REVERSED to swap traversal semantics.
		// If ownership was swapped (bSwap), these directions are inverted.

		if (config.downDirArea != RECAST_NULL_AREA)
		{
			FGeneratedNavigationLink& DownLink = AddLinkSharedLambda(config);
			DownLink.generatedLinkArea = config.downDirArea;
			DownLink.generatedLinkPolyFlag = config.downDirPolyFlag;
			DownLink.Left = OwnerPos;
			DownLink.Right = FarPos;
			DownLink.Direction = bDownIsRightToLeft
				? ENavLinkDirection::RightToLeft
				: ENavLinkDirection::LeftToRight;
		}

		if (config.upDirArea != RECAST_NULL_AREA)
		{
			FGeneratedNavigationLink& UpLink = AddLinkSharedLambda(config);
			UpLink.generatedLinkArea = config.upDirArea;
			UpLink.generatedLinkPolyFlag = config.upDirPolyFlag;
			UpLink.Left = OwnerPos;
			UpLink.Right = FarPos;
			UpLink.Direction = bDownIsRightToLeft
				? ENavLinkDirection::LeftToRight
				: ENavLinkDirection::RightToLeft;
		}
	};

	// Make FGeneratedNavigationLinks
	for (const dtNavLinkBuilder::JumpLink& link : linkBuilder.m_links)
	{
		if (link.flags == dtNavLinkBuilder::FILTERED)
			continue;

		const dtNavLinkBuilderJumpConfig& jumpConfig = linkBuilderConfig.jumpConfigs[link.configIndex];
		if (!ensure(jumpConfig.enabled))
			continue;

		const bool bBidirectionalJumpDownLinks = jumpConfig.downDirArea == jumpConfig.upDirArea;
		TFunctionRef<void(const dtNavLinkBuilderJumpConfig&, const dtReal*, const dtReal*)> AddLinkLambda =
			bBidirectionalJumpDownLinks ?
				TFunctionRef<void(const dtNavLinkBuilderJumpConfig&, const dtReal*, const dtReal*)>(AddBidirectionalLinkLambda) :
				TFunctionRef<void(const dtNavLinkBuilderJumpConfig&, const dtReal*, const dtReal*)>(AddDirectionalLinksLambda);
		
		if (jumpConfig.linkBuilderFlags & DT_NAVLINK_CREATE_CENTER_POINT_LINK)
		{
			// Make a link using the center of the range.
			dtReal midA[3];
			dtVlerp(midA, &link.spine0[0], &link.spine1[0], 0.5);
			dtReal midB[3];
			dtVlerp(midB, &link.spine0[(link.nspine-1)*3], &link.spine1[(link.nspine-1)*3], 0.5);
			// Since trajectory validation starts at agentClimb height to ignore small bumps, remove the offset for the actual link height.
			midA[1] -= linkBuilderConfig.agentClimb;
			midB[1] -= linkBuilderConfig.agentClimb;
			AddLinkLambda(jumpConfig, midA, midB);
		}

		if (jumpConfig.linkBuilderFlags & DT_NAVLINK_CREATE_EXTREMITY_LINKS)
		{
			dtReal posA[3];
			dtReal posB[3];
			dtVcopy(posA, &link.spine0[0]);
			dtVcopy(posB, &link.spine0[(link.nspine-1)*3]);
			posA[1] -= linkBuilderConfig.agentClimb;
			posB[1] -= linkBuilderConfig.agentClimb;
			AddLinkLambda(jumpConfig, posA, posB);

			dtVcopy(posA, &link.spine1[0]);
			dtVcopy(posB, &link.spine1[(link.nspine-1)*3]);
			posA[1] -= linkBuilderConfig.agentClimb;
			posB[1] -= linkBuilderConfig.agentClimb;
			AddLinkLambda(jumpConfig, posA, posB);
		}
	}
}

dtStatus FRecastTileGenerator::BuildTileCacheLinks(FNavMeshBuildContext& BuildContext, dtTileCacheAlloc* alloc, const dtTileCacheLayer& layer,
                                                   const dtTileCacheContourSet& lcset, const float DefaultSnapHeight, TInlineGeneratedNavLinkArray& OutGeneratedLinks) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRecastTileGenerator::BuildTileCacheLinks);
	BuildContext.log(RC_LOG_PROGRESS, "Building Links:");
	const double StartTime = FPlatformTime::Seconds();
	
	duDebugDraw* dd = nullptr;
	int32 DebugEdge = -1;
	int32 DebugConfig = -1;

#if RECAST_INTERNAL_DEBUG_DATA
	if (IsTileDebugActive() && (TileDebugSettings.LinkGenerationDebugFlags != 0))
	{
		DebugEdge = TileDebugSettings.LinkGenerationSelectedEdge;
		DebugConfig = TileDebugSettings.LinkGenerationSelectedConfig;
		dd = &BuildContext.InternalDebugData;
	}
	const uint16 DebugFlags = TileDebugSettings.LinkGenerationDebugFlags;
#endif // RECAST_INTERNAL_DEBUG_DATA

	dtAssert(alloc);

	auto LogOnExit = [&]()
	{
		const double LinkBuildTime = FPlatformTime::Seconds()-StartTime;
#if RECAST_INTERNAL_DEBUG_DATA
		BuildContext.InternalDebugData.BuildLinkTime += LinkBuildTime;
#endif
		BuildContext.log(RC_LOG_PROGRESS, "   BuildTileCacheLinks time: %0.3fms.", LinkBuildTime*1000 );
	};

	if (!SolidHF || !CompactHF)
	{
		LogOnExit();
		return DT_FAILURE;
	}

	const dtReal* orig = layer.header->bmin;

	dtLinkBuilderConfig linkBuilderConfig;

	linkBuilderConfig.jumpConfigs = TileConfig.JumpConfigs;
	for (dtNavLinkBuilderJumpConfig& jumpConfig : linkBuilderConfig.jumpConfigs)
	{
		if (jumpConfig.enabled)
		{
			jumpConfig.init();
		}
	}
	
	linkBuilderConfig.agentRadius = TileConfig.walkableRadius * TileConfig.cs;
	linkBuilderConfig.agentHeight = TileConfig.walkableHeight * TileConfig.ch;
	linkBuilderConfig.agentClimb = TileConfig.walkableClimb * TileConfig.ch;
	linkBuilderConfig.cellSize = TileConfig.cs;
	linkBuilderConfig.cellHeight = TileConfig.ch;

	dtNavLinkBuilder linkBuilder;
	{
		SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastBuildLinks_FindEdges);

		if (!linkBuilder.findEdges(BuildContext, TileConfig, linkBuilderConfig, lcset, orig, SolidHF, CompactHF))
		{
			LogOnExit();
			return DT_FAILURE;
		}

		BuildContext.log(RC_LOG_PROGRESS, "   Found %i edges.", linkBuilder.getEdgeCount());
	}

	if (linkBuilder.getEdgeCount() == 0)
	{
		LogOnExit();
		return DT_SUCCESS;	
	}

	const bool bAllEdges = DebugEdge == -1;
	const bool bAllConfigs = DebugConfig == -1;
	
	if (bAllEdges && bAllConfigs)
	{
		for (const dtNavLinkBuilderJumpConfig& jumpConfig : linkBuilderConfig.jumpConfigs)
		{
			if (jumpConfig.enabled)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RecastBuildLinks_JumpConfig);
				linkBuilder.buildForAllEdges(BuildContext, linkBuilderConfig, jumpConfig);
			}
		}

#if RECAST_INTERNAL_DEBUG_DATA
		duDebugDrawNavLinkBuilder(dd, linkBuilder, DebugFlags, nullptr /*edgeSampler*/);
	}
	else
	{
		for (const dtNavLinkBuilderJumpConfig& jumpConfig : linkBuilderConfig.jumpConfigs)
		{
			if (jumpConfig.enabled && (bAllConfigs || jumpConfig.ConfigIndex == DebugConfig))
			{
				if (bAllEdges)
				{
					linkBuilder.buildForAllEdges(BuildContext, linkBuilderConfig, jumpConfig);
					duDebugDrawNavLinkBuilder(dd, linkBuilder, DebugFlags, nullptr /*edgeSampler*/);
				}
				else
				{
					dtNavLinkBuilder::EdgeSampler sampler;
					linkBuilder.debugBuildEdge(linkBuilderConfig, DebugEdge, sampler, jumpConfig);
					duDebugDrawNavLinkBuilder(dd, linkBuilder, DebugFlags, &sampler);
				}
			}
		}
#endif // RECAST_INTERNAL_DEBUG_DATA
	}

	const FBox RcTileBB = Unreal2RecastBox(TileBB);
	const FVector RcTileMin(RcTileBB.Min);
	const FVector RcTileMax(RcTileBB.Max);

	AddGeneratedLinks(OutGeneratedLinks, linkBuilderConfig, linkBuilder, &RcTileMin.X, &RcTileMax.X);

	LogOnExit();
	return DT_SUCCESS;
}

bool FRecastTileGenerator::GenerateNavigationDataLayer(FNavMeshBuildContext& BuildContext, FTileCacheCompressor& TileCompressor,
	FTileCacheAllocator& GenNavAllocator, FTileGenerationContext& GenerationContext, const dtLinkBuilderData& InLinkBuilderData, int32 LayerIdx)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_GenerateNavigationDataLayer)
		
	dtStatus status = DT_SUCCESS;

	FNavMeshTileData& CompressedData = CompressedLayers[LayerIdx];
	GenerationContext.ResetIntermediateData();

	// Decompress tile layer data. 
	status = dtDecompressTileCacheLayer(&GenNavAllocator, &TileCompressor, (const unsigned char*)CompressedData.GetData(), CompressedData.DataSize, &GenerationContext.Layer);
	if (dtStatusFailed(status))
	{
		BuildContext.log(RC_LOG_ERROR, "GenerateNavigationDataLayer: failed to decompress layer.");
		return false;
	}

	// Rasterize obstacles.
	MarkDynamicAreas(BuildContext, *GenerationContext.Layer);

	{
		SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastBuildRegions)

		// Build regions
		if (TileConfig.TileCachePartitionType == RC_REGION_MONOTONE)
		{
			status = dtBuildTileCacheRegionsMonotone(&GenNavAllocator, TileConfig.minRegionArea, TileConfig.mergeRegionArea, *GenerationContext.Layer);
		}
		else if (TileConfig.TileCachePartitionType == RC_REGION_WATERSHED)
		{
			GenerationContext.DistanceField = dtAllocTileCacheDistanceField(&GenNavAllocator);
			if (GenerationContext.DistanceField == nullptr)
			{
				BuildContext.log(RC_LOG_ERROR, "GenerateNavigationDataLayer: Out of memory 'DistanceField'.");
				return false;
			}

			status = dtBuildTileCacheDistanceField(&GenNavAllocator, *GenerationContext.Layer, *GenerationContext.DistanceField);
			if (dtStatusFailed(status))
			{
				BuildContext.log(RC_LOG_ERROR, "GenerateNavigationDataLayer: Failed to build distance field.");
				return false;
			}

			status = dtBuildTileCacheRegions(&GenNavAllocator, TileConfig.minRegionArea, TileConfig.mergeRegionArea, *GenerationContext.Layer, *GenerationContext.DistanceField);
		}
		else
		{
#if RECAST_INTERNAL_DEBUG_DATA
			if (IsTileDebugActive() && TileDebugSettings.bTileCacheLayerAreas)
			{
				duDebugDrawTileCacheLayerAreas(&BuildContext.InternalDebugData, *GenerationContext.Layer, TileConfig.cs, TileConfig.ch);
			}
#endif
			
			status = dtBuildTileCacheRegionsChunky(&GenNavAllocator, TileConfig.minRegionArea, TileConfig.mergeRegionArea, *GenerationContext.Layer, TileConfig.TileCacheChunkSize);
		}

		if (dtStatusFailed(status))
		{
			BuildContext.log(RC_LOG_ERROR, "GenerateNavigationDataLayer: Failed to build regions.");
			return false;
		}

		// skip empty layer
		if (GenerationContext.Layer->regCount <= 0)
		{
			return true;
		}
	}

#if RECAST_INTERNAL_DEBUG_DATA
	if (IsTileDebugActive() && TileDebugSettings.bTileCacheLayerRegions)
	{
		duDebugDrawTileCacheLayerRegions(&BuildContext.InternalDebugData, *GenerationContext.Layer, TileConfig.cs, TileConfig.ch);
	}
#endif

	{
		SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastBuildContours);
		// Build contour set
		GenerationContext.ContourSet = dtAllocTileCacheContourSet(&GenNavAllocator);
		if (GenerationContext.ContourSet == nullptr)
		{
			BuildContext.log(RC_LOG_ERROR, "GenerateNavigationDataLayer: Out of memory 'ContourSet'.");
			return false;
		}

		bool bSkipContourSimplification = false;
#if RECAST_INTERNAL_DEBUG_DATA
		bSkipContourSimplification = IsTileDebugActive() && TileDebugSettings.bSkipContourSimplification;
#endif
		
#if WITH_NAVMESH_CLUSTER_LINKS
		GenerationContext.ClusterSet = dtAllocTileCacheClusterSet(&GenNavAllocator);
		if (GenerationContext.ClusterSet == nullptr)
		{
			BuildContext.log(RC_LOG_ERROR, "GenerateNavigationDataLayer: Out of memory 'ClusterSet'.");
			return false;
		}

		status = dtBuildTileCacheContours(&GenNavAllocator, *GenerationContext.Layer,
			TileConfig.walkableClimb, TileConfig.maxSimplificationError, TileConfig.simplificationElevationRatio,
			TileConfig.cs, TileConfig.ch,*GenerationContext.ContourSet, *GenerationContext.ClusterSet, bSkipContourSimplification);
#else
		status = dtBuildTileCacheContours(&GenNavAllocator, *GenerationContext.Layer,
			TileConfig.walkableClimb, TileConfig.maxSimplificationError, TileConfig.simplificationElevationRatio,
			TileConfig.cs, TileConfig.ch, *GenerationContext.ContourSet, bSkipContourSimplification);
#endif //WITH_NAVMESH_CLUSTER_LINKS
		
		if (dtStatusFailed(status))
		{
			BuildContext.log(RC_LOG_ERROR, "GenerateNavigationDataLayer: Failed to generate contour set (0x%08X).", status);
			return false;
		}

		// skip empty layer, sometimes there are regions assigned but all flagged as empty (id=0)
		if (GenerationContext.ContourSet->nconts <= 0)
		{
			return true;
		}
	}

#if RECAST_INTERNAL_DEBUG_DATA
	if (IsTileDebugActive() && TileDebugSettings.bTileCacheContours)
	{
		duDebugDrawTileCacheContours(&BuildContext.InternalDebugData, *GenerationContext.ContourSet, LayerIdx, GenerationContext.Layer->header->bmin, TileConfig.cs, TileConfig.ch);
	}
#endif


	{
		SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastBuildPolyMesh);
		// Build poly mesh
		GenerationContext.PolyMesh = dtAllocTileCachePolyMesh(&GenNavAllocator);
		if (GenerationContext.PolyMesh == nullptr)
		{
			BuildContext.log(RC_LOG_ERROR, "GenerateNavigationData: Out of memory 'PolyMesh'.");
			return false;
		}

		status = dtBuildTileCachePolyMesh(&GenNavAllocator, &BuildContext, *GenerationContext.ContourSet, *GenerationContext.PolyMesh, TileConfig.walkableClimb);
		if (dtStatusFailed(status))
		{
			BuildContext.log(RC_LOG_ERROR, "GenerateNavigationData: Failed to generate poly mesh.");
			return false;
		}

#if WITH_NAVMESH_CLUSTER_LINKS
		status = dtBuildTileCacheClusters(&GenNavAllocator, *GenerationContext.ClusterSet, *GenerationContext.PolyMesh);
		if (dtStatusFailed(status))
		{
			BuildContext.log(RC_LOG_ERROR, "GenerateNavigationData: Failed to update cluster set.");
			return false;
		}
#endif // WITH_NAVMESH_CLUSTER_LINKS
	}

#if RECAST_INTERNAL_DEBUG_DATA
	if (IsTileDebugActive() && TileDebugSettings.bTileCachePolyMesh)
	{
		duDebugDrawTileCachePolyMesh(&BuildContext.InternalDebugData, *GenerationContext.PolyMesh, GenerationContext.Layer->header->bmin, TileConfig.cs, TileConfig.ch);
	}
#endif

	// Build detail mesh
	if (TileConfig.bGenerateDetailedMesh)
	{
		SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastBuildPolyDetail);

		// Build detail mesh.
		GenerationContext.DetailMesh = dtAllocTileCachePolyMeshDetail(&GenNavAllocator);
		if (GenerationContext.DetailMesh == nullptr)
		{
			BuildContext.log(RC_LOG_ERROR, "GenerateNavigationData: Out of memory 'DetailMesh'.");
			return false;
		}

		// Fills GenerationContext.PolyMesh with connectivity information.
		status = dtBuildTileCachePolyMeshDetail(&GenNavAllocator, TileConfig.cs, TileConfig.ch, TileConfig.detailSampleDist, TileConfig.detailSampleMaxError,
			*GenerationContext.Layer, *GenerationContext.PolyMesh, *GenerationContext.DetailMesh);
		if (dtStatusFailed(status))
		{
			BuildContext.log(RC_LOG_ERROR, "GenerateNavigationData: Failed to generate poly detail mesh.");
			return false;
		}

#if RECAST_INTERNAL_DEBUG_DATA
		if (IsTileDebugActive() && TileDebugSettings.bTileCacheDetailMesh)
		{
			duDebugDrawTileCacheDetailMesh(&BuildContext.InternalDebugData, *GenerationContext.DetailMesh);
		}
#endif
	}

	const float DefaultSnapHeight = static_cast<float>(TileConfig.walkableClimb) * static_cast<float>(TileConfig.ch);

	// Build Links
	TInlineGeneratedNavLinkArray GeneratedLinks;
	if (InLinkBuilderData.generatingLinks)
	{
		SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastBuildLinks);

		if (SolidHF && CompactHF)
		{
			status = BuildTileCacheLinks(BuildContext, &GenNavAllocator, *GenerationContext.Layer, *GenerationContext.ContourSet, DefaultSnapHeight, GeneratedLinks);

			if (dtStatusFailed(status))
			{
				BuildContext.log(RC_LOG_ERROR, "GenerateNavigationDataLayer: Failed to generate links (0x%08X).", status);
				return false;
			}
		}
	}

	unsigned char* NavData = nullptr;
	int32 NavDataSize = 0;

	if (TileConfig.maxVertsPerPoly <= DT_VERTS_PER_POLYGON &&
		GenerationContext.PolyMesh->npolys > 0 && GenerationContext.PolyMesh->nverts > 0)
	{
		ensureMsgf(GenerationContext.PolyMesh->npolys <= TileConfig.MaxPolysPerTile, TEXT("Polys per Tile limit exceeded!"));
		if (GenerationContext.PolyMesh->nverts >= 0xffff)
		{
			// The vertex indices are ushorts, and cannot point to more than 0xffff vertices.
			BuildContext.log(RC_LOG_ERROR, "Too many vertices per tile %d (max: %d).", GenerationContext.PolyMesh->nverts, 0xffff);
			return false;
		}

		// if we didn't fail already then it's high time we created data for off-mesh links
		FOffMeshData OffMeshData;
		if (!OffmeshLinks.IsEmpty() || !GeneratedLinks.IsEmpty())
		{
			SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastGatherOffMeshData);

			OffMeshData.Reserve(OffmeshLinks.Num() + GeneratedLinks.Num());
			OffMeshData.AreaClassToIdMap = &AdditionalCachedData.AreaClassToIdMap;
			OffMeshData.FlagsPerArea = AdditionalCachedData.FlagsPerOffMeshLinkArea;
			const FSimpleLinkNavModifier* LinkModifier = OffmeshLinks.GetData();

			for (int32 LinkModifierIndex = 0; LinkModifierIndex < OffmeshLinks.Num(); ++LinkModifierIndex, ++LinkModifier)
			{
				OffMeshData.AddLinks(LinkModifier->Links, LinkModifier->LocalToWorld, TileConfig.AgentIndex, DefaultSnapHeight);
#if WITH_NAVMESH_SEGMENT_LINKS
				OffMeshData.AddSegmentLinks(LinkModifier->SegmentLinks, LinkModifier->LocalToWorld, TileConfig.AgentIndex, DefaultSnapHeight);
#endif // WITH_NAVMESH_SEGMENT_LINKS
			}

			OffMeshData.AddLinks(GeneratedLinks, FTransform::Identity, TileConfig.AgentIndex, DefaultSnapHeight);
		}

		// fill flags, or else detour won't be able to find polygons
		// Update poly flags from areas.
		for (int32 i = 0; i < GenerationContext.PolyMesh->npolys; i++)
		{
			GenerationContext.PolyMesh->flags[i] = AdditionalCachedData.FlagsPerArea[GenerationContext.PolyMesh->areas[i]];
		}

		dtNavMeshCreateParams Params;
		memset(&Params, 0, sizeof(Params));
		Params.verts = GenerationContext.PolyMesh->verts;
		Params.vertCount = GenerationContext.PolyMesh->nverts;
		Params.polys = GenerationContext.PolyMesh->polys;
		Params.polyAreas = GenerationContext.PolyMesh->areas;
		Params.polyFlags = GenerationContext.PolyMesh->flags;
		Params.polyCount = GenerationContext.PolyMesh->npolys;
		Params.nvp = GenerationContext.PolyMesh->nvp;
		if (TileConfig.bGenerateDetailedMesh)
		{
			Params.detailMeshes = GenerationContext.DetailMesh->meshes;
			Params.detailVerts = GenerationContext.DetailMesh->verts;
			Params.detailVertsCount = GenerationContext.DetailMesh->nverts;
			Params.detailTris = GenerationContext.DetailMesh->tris;
			Params.detailTriCount = GenerationContext.DetailMesh->ntris;
		}
		Params.offMeshCons = OffMeshData.LinkParams.GetData();
		Params.offMeshConCount = OffMeshData.LinkParams.Num();
		Params.walkableHeight = TileConfig.AgentHeight;
		Params.walkableRadius = TileConfig.AgentRadius;
		Params.walkableClimb = TileConfig.AgentMaxClimb;
		Params.tileX = TileX;
		Params.tileY = TileY;
		Params.tileLayer = LayerIdx;
		rcVcopy(Params.bmin, GenerationContext.Layer->header->bmin);
		rcVcopy(Params.bmax, GenerationContext.Layer->header->bmax);
		Params.cs = TileConfig.cs;
		Params.ch = TileConfig.ch;
		Params.tileResolutionLevel = (unsigned char)TileConfig.TileResolution;

		// Skip BVtree for tiles having few polygons since linear scan is faster for low poly counts. 
		// This saves sizeof(dtBVNode) * polyCount * 2 bytes per tile.
		constexpr int32 PolyCountThreshold = 16;
		const bool bShouldBuildBvTree = TileConfig.bGenerateBVTree && (Params.polyCount > PolyCountThreshold);
		Params.buildBvTree = bShouldBuildBvTree;

		Params.bMinimizeLinkPoolSize = TileConfig.bMinimizeLinkPoolSize;

#if WITH_NAVMESH_CLUSTER_LINKS
		Params.clusterCount = IntCastChecked<unsigned short>(GenerationContext.ClusterSet->nclusters);
		Params.polyClusters = GenerationContext.ClusterSet->polyMap;
#endif // WITH_NAVMESH_CLUSTER_LINKS

		{
			SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastCreateNavMeshData);

			if (!dtCreateNavMeshData(&Params, &NavData, &NavDataSize))
			{
				BuildContext.log(RC_LOG_ERROR, "Could not build Detour navmesh.");
				return false;
			}
		}
	}

	GenerationContext.NavigationData.Add(FNavMeshTileData(NavData, NavDataSize, LayerIdx, CompressedData.LayerBBox));

	const float ModkB = 1.0f / 1024.0f;
	BuildContext.log(RC_LOG_PROGRESS, ">> Layer[%d] = Verts(%d) Polys(%d) Memory(%.2fkB) Cache(%.2fkB)",
		LayerIdx, GenerationContext.PolyMesh->nverts, GenerationContext.PolyMesh->npolys,
		static_cast<float>(GenerationContext.NavigationData.Last().DataSize) * ModkB,
		static_cast<float>(CompressedLayers[LayerIdx].DataSize) * ModkB);

	return true;
}

ETimeSliceWorkResult FRecastTileGenerator::GenerateNavigationDataTimeSliced(FNavMeshBuildContext& BuildContext, const dtLinkBuilderData& InLinkBuilderData)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastBuildNavigation);

	check(TimeSliceManager);
	ETimeSliceWorkResult WorkResult = ETimeSliceWorkResult::Succeeded;

	switch (GenerateNavDataTimeSlicedState)
	{
	case EGenerateNavDataTimeSlicedState::Invalid:
	{
		ensureMsgf(false, TEXT("Invalid EGenerateNavDataTimeSlicedState, has this function been called when its already finished processing?"));
		return ETimeSliceWorkResult::Failed;
	}

	case EGenerateNavDataTimeSlicedState::Init:
	{
		GenNavDataTimeSlicedAllocator = MakeUnique<FTileCacheAllocator>();
		GenNavDataTimeSlicedGenerationContext = MakeUnique<FTileGenerationContext>(GenNavDataTimeSlicedAllocator.Get());
		GenNavDataTimeSlicedGenerationContext->NavigationData.Reserve(CompressedLayers.Num());
		GenerateNavDataTimeSlicedState = EGenerateNavDataTimeSlicedState::GenerateLayers;
	}//fall through to next state
	case EGenerateNavDataTimeSlicedState::GenerateLayers:
	{
		for (; GenNavDataLayerTimeSlicedIdx < CompressedLayers.Num() && GenNavDataLayerTimeSlicedIdx < DirtyLayers.Num(); GenNavDataLayerTimeSlicedIdx++)
		{
			if (DirtyLayers[GenNavDataLayerTimeSlicedIdx] == false || !CompressedLayers[GenNavDataLayerTimeSlicedIdx].IsValid())
			{
				// skip layers not marked for rebuild
				continue;
			}

			if (TimeSliceManager->GetTimeSlicer().IsTimeSliceFinishedCached())
			{
				WorkResult = ETimeSliceWorkResult::CallAgainNextTimeSlice;
				break;
			}

			FTileCacheCompressor TileCompressor;
			const bool bGenDataLayerSuccess = GenerateNavigationDataLayer(BuildContext, TileCompressor, *GenNavDataTimeSlicedAllocator,
				*GenNavDataTimeSlicedGenerationContext, InLinkBuilderData, GenNavDataLayerTimeSlicedIdx);

			MARK_TIMESLICE_SECTION_DEBUG(TimeSliceManager->GetTimeSlicer(), GenerateLayers);

			//carry on iterating but don't do any more work if the time slice is finished (as we may not need to in which case we can avoid calling this function again)
			TimeSliceManager->GetTimeSlicer().TestTimeSliceFinished();

			if (!bGenDataLayerSuccess)
			{
				WorkResult = ETimeSliceWorkResult::Failed;
				break;
			}
		}

		if (WorkResult != ETimeSliceWorkResult::CallAgainNextTimeSlice)
		{
			GenNavDataLayerTimeSlicedIdx = 0;
			GenerateNavDataTimeSlicedState = EGenerateNavDataTimeSlicedState::Invalid;

			if (WorkResult == ETimeSliceWorkResult::Succeeded)
			{
				NavigationData = MoveTemp(GenNavDataTimeSlicedGenerationContext->NavigationData);
			}
			GenNavDataTimeSlicedGenerationContext->ResetIntermediateData();
		}
	}
	break;

	default:
	{
		ensureMsgf(false, TEXT("unhandled EGenerateNavDataTimeSlicedState"));
		return ETimeSliceWorkResult::Failed;
	}
	}

	return WorkResult;
}

bool FRecastTileGenerator::GenerateNavigationData(FNavMeshBuildContext& BuildContext, const dtLinkBuilderData& InLinkBuilderData)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastBuildNavigation);

	FTileCacheAllocator GenNavAllocator;
	FTileGenerationContext GenerationContext(&GenNavAllocator);
	GenerationContext.NavigationData.Reserve(CompressedLayers.Num());
	FTileCacheCompressor TileCompressor;
	bool bGenDataLayerSuccess = true;

	for (int32 LayerIdx = 0; LayerIdx < CompressedLayers.Num() && LayerIdx < DirtyLayers.Num(); LayerIdx++)
	{
		if (DirtyLayers[LayerIdx] == false || !CompressedLayers[LayerIdx].IsValid())
		{
			// skip layers not marked for rebuild
			continue;
		}

		bGenDataLayerSuccess = GenerateNavigationDataLayer(BuildContext, TileCompressor, GenNavAllocator, GenerationContext, InLinkBuilderData, LayerIdx);
		if (!bGenDataLayerSuccess)
		{
			break;
		}
	}

	if (bGenDataLayerSuccess)
	{
		NavigationData = MoveTemp(GenerationContext.NavigationData);
	}
	
	{
		QUICK_SCOPE_CYCLE_COUNTER(FRecastTileGenerator_GenerateNavigationData_Free);
		GenerationContext.ResetIntermediateData();
	}

	return bGenDataLayerSuccess;
}

void FRecastTileGenerator::ComputeRasterizationMasks(FNavMeshBuildContext& BuildContext, FTileRasterizationContext& RasterContext)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastComputeRasterizationMasks);

	UE_LOGF(LogNavigationDataBuild, VeryVerbose, "      %ls", ANSI_TO_TCHAR(__FUNCTION__));

	for (const FRecastAreaNavModifierElement& ModifierElement : Modifiers)
	{
		if (ModifierElement.bMaskFillCollisionUnderneathForNavmesh)
		{
			if (SolidHF == nullptr)
			{
				return;
			}

			const int32 Mask = ~RC_PROJECT_TO_BOTTOM;
			for (const FAreaNavModifier& ModifierArea : ModifierElement.Areas)
			{
				for (const FTransform& LocalToWorld : ModifierElement.PerInstanceTransform)
				{
					MarkRasterizationMask(&BuildContext, SolidHF, ModifierArea, LocalToWorld, Mask, RasterContext.RasterizationMasks);
				}

				if (ModifierElement.PerInstanceTransform.Num() == 0)
				{
					MarkRasterizationMask(&BuildContext, SolidHF, ModifierArea, FTransform::Identity, Mask, RasterContext.RasterizationMasks);
				}
			}
		}
	}
}

void FRecastTileGenerator::MarkDynamicAreas(FNavMeshBuildContext& BuildContext, dtTileCacheLayer& Layer)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastMarkAreas);

	if (Modifiers.Num())
	{
		if (AdditionalCachedData.bUseSortFunction && Modifiers.Num() > 1)
		{
			FGCScopeGuard GCScopeGuard;
			if (const ARecastNavMesh* ActorOwner = AdditionalCachedData.ActorOwner.Get())
			{
				ActorOwner->SortAreasForGenerator(Modifiers);
			}
		}

		// 1: if navmesh is using low areas, apply only low area replacements
		if (TileConfig.bMarkLowHeightAreas)
		{
			const int32 LowAreaId = RECAST_LOW_AREA;
			for (int32 ModIdx = 0; ModIdx < Modifiers.Num(); ModIdx++)
			{
				FRecastAreaNavModifierElement& Element = Modifiers[ModIdx];
				for (int32 AreaIdx = Element.Areas.Num() - 1; AreaIdx >= 0; AreaIdx--)
				{
					const FAreaNavModifier& AreaMod = Element.Areas[AreaIdx];
					if (AreaMod.GetApplyMode() == ENavigationAreaMode::ApplyInLowPass ||
						AreaMod.GetApplyMode() == ENavigationAreaMode::ReplaceInLowPass)
					{
						const int32* AreaIDPtr = AdditionalCachedData.AreaClassToIdMap.Find(AreaMod.GetAreaClass());
						// replace area will be fixed as LowAreaId during this pass, regardless settings in area modifier
						const int32* ReplaceAreaIDPtr = (AreaMod.GetApplyMode() == ENavigationAreaMode::ReplaceInLowPass) ? &LowAreaId : nullptr;

						if (AreaIDPtr != nullptr)
						{
							for (const FTransform& LocalToWorld : Element.PerInstanceTransform)
							{
								MarkDynamicArea(BuildContext, AreaMod, LocalToWorld, Layer, *AreaIDPtr, ReplaceAreaIDPtr);
							}

							if (Element.PerInstanceTransform.Num() == 0)
							{
								MarkDynamicArea(BuildContext, AreaMod, FTransform::Identity, Layer, *AreaIDPtr, ReplaceAreaIDPtr);
							}
						}
					}
				}
			}

			// 2. remove all low area marking
			dtReplaceArea(Layer, RECAST_NULL_AREA, RECAST_LOW_AREA);
		}

		// 3. apply remaining modifiers
		for (const FRecastAreaNavModifierElement& Element : Modifiers)
		{
			for (const FAreaNavModifier& Area : Element.Areas)
			{
				if (Area.GetApplyMode() == ENavigationAreaMode::ApplyInLowPass || Area.GetApplyMode() == ENavigationAreaMode::ReplaceInLowPass)
				{
					continue;
				}

				const int32* AreaIDPtr = AdditionalCachedData.AreaClassToIdMap.Find(Area.GetAreaClass());
				const int32* ReplaceIDPtr = (Area.GetApplyMode() == ENavigationAreaMode::Replace) && Area.GetAreaClassToReplace() ?
					AdditionalCachedData.AreaClassToIdMap.Find(Area.GetAreaClassToReplace()) : nullptr;
				
				if (AreaIDPtr)
				{
					for (const FTransform& LocalToWorld : Element.PerInstanceTransform)
					{
						MarkDynamicArea(BuildContext, Area, LocalToWorld, Layer, *AreaIDPtr, ReplaceIDPtr);
					}

					if (Element.PerInstanceTransform.Num() == 0)
					{
						MarkDynamicArea(BuildContext, Area, FTransform::Identity, Layer, *AreaIDPtr, ReplaceIDPtr);
					}
				}
			}
		}
	}
	else
	{
		if (TileConfig.bMarkLowHeightAreas)
		{
			dtReplaceArea(Layer, RECAST_NULL_AREA, RECAST_LOW_AREA);
		}
	}
}

void FRecastTileGenerator::MarkDynamicArea(FNavMeshBuildContext& BuildContext, const FAreaNavModifier& Modifier, const FTransform& LocalToWorld, dtTileCacheLayer& Layer)
{
	const int32* AreaIDPtr = AdditionalCachedData.AreaClassToIdMap.Find(Modifier.GetAreaClass());
	const int32* ReplaceIDPtr = Modifier.GetAreaClassToReplace() ? AdditionalCachedData.AreaClassToIdMap.Find(Modifier.GetAreaClassToReplace()) : nullptr;
	if (AreaIDPtr)
	{
		MarkDynamicArea(BuildContext, Modifier, LocalToWorld, Layer, *AreaIDPtr, ReplaceIDPtr);
	}
}

void MarkBoxMask(const FVector::FReal* pos, const FVector::FReal* extent, const int mask, rcHeightfield& hf, int* rasterizationMasks)
{
	FVector::FReal* orig = hf.bmin;
	FVector::FReal bmin[3], bmax[3];
	rcVsub(bmin, pos, extent);
	rcVadd(bmax, pos, extent);

	const int w = hf.width;
	const int h = hf.height;
	const FVector::FReal ics = 1.0f/hf.cs;

	int minx = (int)FMath::Floor((bmin[0]-orig[0])*ics);
	int minz = (int)FMath::Floor((bmin[2]-orig[2])*ics);
	int maxx = (int)FMath::Floor((bmax[0]-orig[0])*ics);
	int maxz = (int)FMath::Floor((bmax[2]-orig[2])*ics);

	if (maxx < 0) return;
	if (minx >= w) return;
	if (maxz < 0) return;
	if (minz >= h) return;

	if (minx < 0) minx = 0;
	if (maxx >= w) maxx = w-1;
	if (minz < 0) minz = 0;
	if (maxz >= h) maxz = h-1;

	for (int z = minz; z <= maxz; ++z)
	{
		for (int x = minx; x <= maxx; ++x)
		{
			rasterizationMasks[x+z*w] &= mask;
		}
	}
}

int PointInPoly(const FVector::FReal* verts, int nv, const FVector::FReal* p)
{
	int i, j, c = 0;
	for (i = 0, j = nv-1; i < nv; j = i++)
	{
		const FVector::FReal* vi = &verts[i*3];
		const FVector::FReal* vj = &verts[j*3];
		if (((vi[2] > p[2]) != (vj[2] > p[2])) &&
			(p[0] < (vj[0]-vi[0]) * (p[2]-vi[2]) / (vj[2]-vi[2]) + vi[0]))
			c = !c;
	}
	return c;
}

// Similar to rcMarkConvexPolyArea
void MarkConvexMask(const int mask, const FVector::FReal* verts, const int nv, rcHeightfield& hf, int* rasterizationMasks)
{
	FVector::FReal* orig = hf.bmin;
	FVector::FReal bmin[3], bmax[3];
	rcVcopy(bmin, verts);
	rcVcopy(bmax, verts);
	for (int i = 1; i < nv; ++i)
	{
		rcVmin(bmin, &verts[i*3]);
		rcVmax(bmax, &verts[i*3]);
	}

	const int w = hf.width;
	const int h = hf.height;
	const FVector::FReal ics = 1.0f/hf.cs;

	int minx = (int)((bmin[0]-orig[0])*ics);
	int minz = (int)((bmin[2]-orig[2])*ics);
	int maxx = (int)((bmax[0]-orig[0])*ics);
	int maxz = (int)((bmax[2]-orig[2])*ics);

	if (maxx < 0) return;
	if (minx >= w) return;
	if (maxz < 0) return;
	if (minz >= h) return;

	if (minx < 0) minx = 0;
	if (maxx >= w) maxx = w-1;
	if (minz < 0) minz = 0;
	if (maxz >= h) maxz = h-1;

	for (int z = minz; z <= maxz; ++z)
	{
		for (int x = minx; x <= maxx; ++x)
		{
			FVector::FReal p[3];
			p[0] = orig[0] + ((FVector::FReal)x+0.5f)*hf.cs;
			p[1] = 0.0f;
			p[2] = orig[2] + ((FVector::FReal)z+0.5f)*hf.cs;
			if (PointInPoly(verts, nv, p))
			{
				rasterizationMasks[x+z*w] &= mask;
			}
		}
	}
}

void FRecastTileGenerator::MarkRasterizationMask(rcContext* /*BuildContext*/, rcHeightfield* InSolidHF,
	const FAreaNavModifier& Modifier, const FTransform& LocalToWorld, const int32 Mask, TInlineMaskArray& OutMaskArray)
{
	FBox ModifierBounds = Modifier.GetBounds().TransformBy(LocalToWorld);
	if (!ModifierBounds.Intersect(TileBB))
	{
		return;
	}

	// Init on first use
	if (OutMaskArray.Num() == 0)
	{
		InitRasterizationMaskArray(InSolidHF, OutMaskArray);
	}

	switch (Modifier.GetShapeType())
	{
	case ENavigationShapeType::Box:
	{
		FBoxNavAreaData BoxData;
		Modifier.GetBox(BoxData);
		FBox WorldBox = FBox::BuildAABB(BoxData.Origin, BoxData.Extent).TransformBy(LocalToWorld);
		FBox RecastBox = Unreal2RecastBox(WorldBox);
		FVector RecastPos;
		FVector RecastExtent;
		RecastBox.GetCenterAndExtents(RecastPos, RecastExtent);
		check(OutMaskArray.Num() == InSolidHF->width*InSolidHF->height);
		MarkBoxMask(&(RecastPos.X), &(RecastExtent.X), Mask, *InSolidHF, OutMaskArray.GetData());
	}
	break;

	case ENavigationShapeType::Convex:
	{
		FConvexNavAreaData ConvexData;
		Modifier.GetConvex(ConvexData);
		TArray<FVector> ConvexVerts;
		const FVector::FReal Expand = 0.f;

		const TArray<FVector> Points = UE::LWC::ConvertArrayType<FVector>(ConvexData.Points);
		GrowConvexHull(Expand, Points, ConvexVerts);

		if (ConvexVerts.Num())
		{
			TArray<FVector::FReal> ConvexCoords;
			ConvexCoords.AddZeroed(ConvexVerts.Num() * 3);
			FVector::FReal* ItCoord = ConvexCoords.GetData();
			for (int32 i = 0; i < ConvexVerts.Num(); i++)
			{
				const FVector RecastV = Unreal2RecastPoint(ConvexVerts[i]);
				*ItCoord = RecastV.X; ItCoord++;
				*ItCoord = RecastV.Y; ItCoord++;
				*ItCoord = RecastV.Z; ItCoord++;
			}

			MarkConvexMask(Mask, ConvexCoords.GetData(), ConvexVerts.Num(), *InSolidHF, OutMaskArray.GetData());
		}
	}
	break;

	default: 
	break;
	}
}

void FRecastTileGenerator::MarkDynamicArea(FNavMeshBuildContext& BuildContext, const FAreaNavModifier& Modifier, const FTransform& LocalToWorld, dtTileCacheLayer& Layer, const int32 AreaID, const int32* ReplaceIDPtr)
{
	const float ExpandBy = TileConfig.AgentRadius;

	// If requested, expand by 1 cell height
	const bool bExpandTop = TileConfig.bUseExtraTopCellWhenMarkingAreas || Modifier.ShouldExpandTopByCellHeight();
	const FVector::FReal OffsetZMax = (bExpandTop ? TileConfig.ch : 0.f);
	const FVector::FReal OffsetZMin = TileConfig.ch + (Modifier.ShouldIncludeAgentHeight() ? TileConfig.AgentHeight : 0.0f);

	// Check whether modifier affects this layer
	const FBox LayerUnrealBounds = Recast2UnrealBox(Layer.header->bmin, Layer.header->bmax);
	FBox ModifierBounds = Modifier.GetBounds().TransformBy(LocalToWorld);
	ModifierBounds.Min -= FVector(ExpandBy, ExpandBy, OffsetZMin);
	ModifierBounds.Max += FVector(ExpandBy, ExpandBy, OffsetZMax);

	if (!LayerUnrealBounds.Intersect(ModifierBounds))
	{
		return;
	}

#if RECAST_INTERNAL_DEBUG_DATA
	const bool bAddShapesToDebugDraw = IsTileDebugActive() && TileDebugSettings.bDynamicAreas;
	const ARecastNavMesh* RecastParentNavmesh = nullptr;
	if (bAddShapesToDebugDraw)
	{
		TSharedPtr<FNavDataGenerator, ESPMode::ThreadSafe> ParentGenerator = ParentGeneratorWeakPtr.Pin();
		if (ParentGenerator.IsValid())
		{
			TSharedPtr<FRecastNavMeshGenerator> RecastParentGenerator = StaticCastSharedPtr<FRecastNavMeshGenerator>(ParentGenerator);
			if (const FRecastNavMeshGenerator* RecastNavMeshGeneratorPtr = RecastParentGenerator.Get())
			{
				RecastParentNavmesh = RecastNavMeshGeneratorPtr->GetOwner();
			}
		}
	}
#endif // RECAST_INTERNAL_DEBUG_DATA

	const FVector::FReal* LayerRecastOrig = Layer.header->bmin;
	switch (Modifier.GetShapeType())
	{
	case ENavigationShapeType::Cylinder:
		{
			FCylinderNavAreaData CylinderData;
			Modifier.GetCylinder(CylinderData);

			// Only scaling and translation
			FVector Scale3D = LocalToWorld.GetScale3D().GetAbs();

			CylinderData.Height = UE_REAL_TO_FLOAT(CylinderData.Height * Scale3D.Z);
			CylinderData.Radius = UE_REAL_TO_FLOAT(CylinderData.Radius * FMath::Max(Scale3D.X, Scale3D.Y));
			CylinderData.Origin = LocalToWorld.TransformPosition(CylinderData.Origin);
			
			const float OffsetZMid = UE_REAL_TO_FLOAT((OffsetZMax - OffsetZMin) * 0.5f);
			CylinderData.Origin.Z += OffsetZMid;
			CylinderData.Height += FMath::Abs(OffsetZMid) * 2.f;
			CylinderData.Radius += ExpandBy;
			
			const FVector RecastPos = Unreal2RecastPoint(CylinderData.Origin);

			if (ReplaceIDPtr)
			{
				dtReplaceCylinderArea(Layer, LayerRecastOrig, TileConfig.cs, TileConfig.ch,
					&(RecastPos.X), CylinderData.Radius, CylinderData.Height, IntCastChecked<unsigned char>(AreaID), IntCastChecked<unsigned char>(*ReplaceIDPtr));
			}
			else
			{
				dtMarkCylinderArea(Layer, LayerRecastOrig, TileConfig.cs, TileConfig.ch,
					&(RecastPos.X), CylinderData.Radius, CylinderData.Height, IntCastChecked<unsigned char>(AreaID));
			}

#if RECAST_INTERNAL_DEBUG_DATA
			if (bAddShapesToDebugDraw)
			{
				const FColor AreaColor = RecastParentNavmesh ? RecastParentNavmesh->GetAreaIDColor(IntCastChecked<unsigned char>(AreaID)) : FColor::White;
				duDrawCylinderArea(&BuildContext.InternalDebugData, &(RecastPos.X), CylinderData.Radius, CylinderData.Height, AreaColor.ToPackedRGBA());
			}
#endif // RECAST_INTERNAL_DEBUG_DATA
		}
		break;

	case ENavigationShapeType::Box:
		{
			FBoxNavAreaData BoxData;
			Modifier.GetBox(BoxData);

			FBox WorldBox = FBox::BuildAABB(BoxData.Origin, BoxData.Extent).TransformBy(LocalToWorld);
			WorldBox = WorldBox.ExpandBy(FVector(ExpandBy, ExpandBy, 0));
			WorldBox.Min.Z -= OffsetZMin;
			WorldBox.Max.Z += OffsetZMax;

			const FBox RecastBox = Unreal2RecastBox(WorldBox);
			FVector RecastPos, RecastExtent;
			RecastBox.GetCenterAndExtents(RecastPos, RecastExtent);
				
			if (ReplaceIDPtr)
			{
				dtReplaceBoxArea(Layer, LayerRecastOrig, TileConfig.cs, TileConfig.ch,
					&(RecastPos.X), &(RecastExtent.X), IntCastChecked<unsigned char>(AreaID), IntCastChecked<unsigned char>(*ReplaceIDPtr));
			}
			else
			{
				dtMarkBoxArea(Layer, LayerRecastOrig, TileConfig.cs, TileConfig.ch,
					&(RecastPos.X), &(RecastExtent.X), IntCastChecked<unsigned char>(AreaID));
			}

#if RECAST_INTERNAL_DEBUG_DATA
			if (bAddShapesToDebugDraw)
			{
				const FColor AreaColor = RecastParentNavmesh ? RecastParentNavmesh->GetAreaIDColor(IntCastChecked<unsigned char>(AreaID)) : FColor::White;
				duDrawBoxArea(&BuildContext.InternalDebugData, &(RecastPos.X), &(RecastExtent.X), AreaColor.ToPackedRGBA());
			}
#endif // RECAST_INTERNAL_DEBUG_DATA
		}
		break;

	case ENavigationShapeType::Convex:
	case ENavigationShapeType::InstancedConvex:
		{
			FConvexNavAreaData ConvexData;
			if (Modifier.GetShapeType() == ENavigationShapeType::InstancedConvex)
			{
				Modifier.GetPerInstanceConvex(LocalToWorld, ConvexData);
			} 
			else
			{
				Modifier.GetConvex(ConvexData);
			}

			TArray<FVector> ConvexVerts;

			const TArray<FVector> Points = UE::LWC::ConvertArrayType<FVector>(ConvexData.Points);
			GrowConvexHull(ExpandBy, Points, ConvexVerts);
			ConvexData.MinZ -= OffsetZMin;
			ConvexData.MaxZ += OffsetZMax;

			if (ConvexVerts.Num())
			{
				TArray<FVector::FReal> ConvexCoords;
				ConvexCoords.AddZeroed(ConvexVerts.Num() * 3);
						
				FVector::FReal* ItCoord = ConvexCoords.GetData();
				for (int32 i = 0; i < ConvexVerts.Num(); i++)
				{
					const FVector RecastV = Unreal2RecastPoint(ConvexVerts[i]);
					*ItCoord = RecastV.X; ItCoord++;
					*ItCoord = RecastV.Y; ItCoord++;
					*ItCoord = RecastV.Z; ItCoord++;
				}

				if (ReplaceIDPtr)
				{
					dtReplaceConvexArea(Layer, LayerRecastOrig, TileConfig.cs, TileConfig.ch,
						ConvexCoords.GetData(), ConvexVerts.Num(), ConvexData.MinZ, ConvexData.MaxZ, IntCastChecked<unsigned char>(AreaID), IntCastChecked<unsigned char>(*ReplaceIDPtr));
				}
				else
				{
					dtMarkConvexArea(Layer, LayerRecastOrig, TileConfig.cs, TileConfig.ch,
						ConvexCoords.GetData(), ConvexVerts.Num(), ConvexData.MinZ, ConvexData.MaxZ, IntCastChecked<unsigned char>(AreaID));
				}

#if RECAST_INTERNAL_DEBUG_DATA
				if (bAddShapesToDebugDraw)
				{
					const FColor AreaColor = RecastParentNavmesh ? RecastParentNavmesh->GetAreaIDColor(IntCastChecked<unsigned char>(AreaID)) : FColor::White;
					duDrawConvexArea(&BuildContext.InternalDebugData,ConvexCoords.GetData(), ConvexVerts.Num(), ConvexData.MinZ, ConvexData.MaxZ, AreaColor.ToPackedRGBA());
				}
#endif // RECAST_INTERNAL_DEBUG_DATA
			}
		}
		break;

	default: break;
	}
}

uint32 FRecastTileGenerator::GetUsedMemCount() const
{
	SIZE_T TotalMemory = 0;
	TotalMemory += InclusionBounds.GetAllocatedSize();
	TotalMemory += Modifiers.GetAllocatedSize();
	TotalMemory += OffmeshLinks.GetAllocatedSize();
	TotalMemory += RawGeometry.GetAllocatedSize();
	
	for (const FRecastRawGeometryElement& Element : RawGeometry)
	{
		TotalMemory += Element.GeomCoords.GetAllocatedSize();
		TotalMemory += Element.GeomIndices.GetAllocatedSize();
		TotalMemory += Element.PerInstanceTransform.GetAllocatedSize();
	}

	for (const FRecastAreaNavModifierElement& Element : Modifiers)
	{
		TotalMemory += Element.Areas.GetAllocatedSize();
		TotalMemory += Element.PerInstanceTransform.GetAllocatedSize();
	}

	const FSimpleLinkNavModifier* SimpleLink = OffmeshLinks.GetData();
	for (int32 Index = 0; Index < OffmeshLinks.Num(); ++Index, ++SimpleLink)
	{
		TotalMemory += SimpleLink->Links.GetAllocatedSize();
	}

	TotalMemory += CompressedLayers.GetAllocatedSize();
	for (int32 i = 0; i < CompressedLayers.Num(); i++)
	{
		TotalMemory += CompressedLayers[i].DataSize;
	}

	TotalMemory += NavigationData.GetAllocatedSize();
	for (int32 i = 0; i < NavigationData.Num(); i++)
	{
		TotalMemory += NavigationData[i].DataSize;
	}

	return IntCastChecked<uint32>(TotalMemory);
}

void FRecastTileGenerator::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (const TSharedRef<FNavigationRelevantData>& RelevantData : NavigationRelevantData)
	{
		// Local variable since 'AddReferencedObject' requires a non-const& parameter
		TWeakObjectPtr<const UObject> WeakObject = RelevantData->SourceElement->GetWeakUObject();
		if (WeakObject.IsValid())
		{
			Collector.AddReferencedObject(WeakObject);
		}
	}
}

FString FRecastTileGenerator::GetReferencerName() const
{
	return TEXT("FRecastTileGenerator");
}

namespace UE::NavMesh::Private
{
void CheckTileIndicesInValidRange(const TNavStatArray<FBox>& NavigableAreas, const ARecastNavMesh& NavMesh)
{
	auto CheckTileHelper = [&NavMesh](const FVector& Pos) {
		// If there are no NavigableAreas then there won't be any indices for them so default true.
		bool bIndiciesFitInInt32 = false;

		ensure(NavMesh.CheckTileIndicesInValidRange(Pos, bIndiciesFitInInt32));

		UE_CLOGF(!bIndiciesFitInInt32, LogNavigation, Error, "Magnitude of Recast tile indicies are too large to fit in an int32 for NavigableAreas extent %ls for %ls",*Pos.ToString(), *GetFullNameSafe(&NavMesh));
	};

	for (const FBox& AreaBounds : NavigableAreas)
	{
		CheckTileHelper(AreaBounds.Min);
		CheckTileHelper(AreaBounds.Max);
	}
}

int32 CalculateMaxTilesCount(const TNavStatArray<FBox>& NavigableAreas, FVector::FReal TileSizeInWorldUnits, FVector::FReal AvgLayersPerGridCell, const uint32 NavMeshVersion)
{
	int64 GridCellsCount = 0;
	for (int32 Index = 0; Index < NavigableAreas.Num(); ++Index)
	{
		const FBox& AreaBounds = NavigableAreas[Index];
		if (NavMeshVersion >= NAVMESHVER_MAXTILES_COUNT_SKIP_INCLUSION)
		{
			bool bIsInsideAnotherArea = false;
			for (int32 OtherIndex = 0; OtherIndex < NavigableAreas.Num(); ++OtherIndex)
			{
				if (Index == OtherIndex)
				{
					continue;
				}

				const FBox& OtherBox = NavigableAreas[OtherIndex];
				if (OtherBox.IsInsideXY(AreaBounds))
				{
					// The current Area Bounds is fully contained in another Area, so we don't need to count it.
					// This doesn't take into account partial overlaps
					bIsInsideAnotherArea = true;
					break;
				}
			}

			if (bIsInsideAnotherArea)
			{
				continue;
			}
		}

		UE_LOGF(LogNavigation, Verbose, "%s AreaBounds: %ls", __FUNCTION__, *AreaBounds.ToCompactString());

		// TODO: need more precise calculation, currently we don't take into account that volumes can be overlapped
		const FBox RCBox = Unreal2RecastBox(AreaBounds);

		if (NavMeshVersion >= NAVMESHVER_MAXTILES_COUNT_CHANGE)
		{
			// Keep this as an integer division to avoid imprecision between platforms and targets (since MaxTilesCount is compared with stored data).
			const int64 TileSizeUU = (int64)TileSizeInWorldUnits;
			const int64 XSize = (FMath::CeilToInt(RCBox.GetSize().X) / TileSizeUU) + 1;
			const int64 YSize = (FMath::CeilToInt(RCBox.GetSize().Z) / TileSizeUU) + 1;
			GridCellsCount += (XSize*YSize);
		}
		else
		{
			// Support old navmesh versions
			int64 XSize = FMath::CeilToInt(RCBox.GetSize().X/TileSizeInWorldUnits) + 1;
			int64 YSize = FMath::CeilToInt(RCBox.GetSize().Z/TileSizeInWorldUnits) + 1;
			GridCellsCount += (XSize*YSize);
		}
	}

	const int64 MaxTilesCount = FMath::CeilToInt(static_cast<FVector::FReal>(GridCellsCount) * AvgLayersPerGridCell);

	// Output warning with details when overflowing
	UE_SUPPRESS(LogNavigationDataBuild, Warning,
	{
		if (!IntFitsIn<int32>(MaxTilesCount))
		{
			TStringBuilder<2048> DebugInformation;
			
			DebugInformation.Appendf(TEXT("%hs Computed maximum tile count is over limit. "
				"Check inputs for a abnormally high values.\n"
				"  MaxTilesCount: %llu, GridCellsCount: %llu, AvgLayersPerGridCell: %f, TileSizeInWorldUnits: %f \n"),
				__FUNCTION__, MaxTilesCount, GridCellsCount, AvgLayersPerGridCell, TileSizeInWorldUnits);

			DebugInformation.Appendf(TEXT("  Navigable areas:\n"));
			for (const FBox& AreaBounds : NavigableAreas)
			{
				DebugInformation.Appendf(TEXT("    AreaBounds: %s"), *AreaBounds.ToString());
			}
			
			UE_LOGF(LogNavigationDataBuild, Warning, "%ls", DebugInformation.ToString());
		}
	});
	
	return IntCastChecked<int32>(FMath::Clamp(MaxTilesCount, 0, TNumericLimits<int32>::Max()));
}
} // UE::NavMesh::Private

// Whether geometry rebuild is disabled for this navmesh.
// True for Static and DynamicModifiersOnly navmesh in game worlds (neither supports rebuild from geometry),
// unless the navmesh requires an initial rebuild to recover missing nav data.
static bool IsGeometryRebuildDisabled(ARecastNavMesh* InNavMesh)
{
	return InNavMesh->GetWorld()->IsGameWorld()
		&& InNavMesh->GetRuntimeGenerationMode() != ERuntimeGenerationType::Dynamic
		&& !InNavMesh->RequiresInitialRebuild();
}

//----------------------------------------------------------------------//
// FRecastNavMeshGenerator
//----------------------------------------------------------------------//

FRecastNavMeshGenerator::FRecastNavMeshGenerator(ARecastNavMesh& InDestNavMesh)
	: NumActiveTiles(0)
	, MaxTileGeneratorTasks(1)
	, DestNavMesh(&InDestNavMesh)
	, bInitialized(false)
	, bRestrictBuildingToActiveTiles(false)
	, Version(0)
{
	INC_DWORD_STAT_BY(STAT_NavigationMemory, sizeof(*this));
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS	// Needed for ActiveTiles
FRecastNavMeshGenerator::~FRecastNavMeshGenerator()
{
	UE_CLOGF(RunningDirtyTiles.Num() > 0, LogNavigation, Log, "Discarding %d build tasks", RunningDirtyTiles.Num());
	CancelBuild();
	DEC_DWORD_STAT_BY( STAT_NavigationMemory, sizeof(*this) );
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FRecastNavMeshGenerator::SetupTileConfig(const ENavigationDataResolution TileResolution, FRecastBuildConfig& OutConfig) const
{
	check(GetOwner());
	ensure(GetConfig().cs == GetOwner()->GetCellSize(ENavigationDataResolution::Default));
	const float CellSize = GetOwner()->GetCellSize(TileResolution);
	const float CellHeight = GetOwner()->GetCellHeight(TileResolution);
	const float AgentMaxStepHeight = GetOwner()->GetAgentMaxStepHeight(TileResolution);
	
	// Update all settings that depends directly or indirectly of the CellSize
	OutConfig.TileResolution = TileResolution;
	OutConfig.cs = CellSize;
	OutConfig.walkableRadius = FMath::CeilToInt(DestNavMesh->AgentRadius / CellSize);
	OutConfig.maxStepFromWalkableSlope = OutConfig.cs * FMath::Tan(FMath::DegreesToRadians(OutConfig.walkableSlopeAngle));

	UE::NavMesh::Private::ComputeConfigBorderSizes(IsGeneratingLinks(), OutConfig);

	OutConfig.maxEdgeLen = (int32)(1200.0f / CellSize);

	OutConfig.minRegionArea = (int32)rcSqr(DestNavMesh->MinRegionArea / CellSize);
	OutConfig.mergeRegionArea = (int32)rcSqr(DestNavMesh->MergeRegionSize / CellSize);

	OutConfig.tileSize = FMath::Max(FMath::TruncToInt(DestNavMesh->TileSizeUU / CellSize), 1);
	UE_CLOGF(OutConfig.tileSize == 1, LogNavigation, Error, "RecastNavMesh TileSize of 1 is highly discouraged. This occurence indicates an issue with RecastNavMesh\'s generation properties (specifically TileSizeUU: %f, CellSize: %f). Please ensure their correctness."
		, DestNavMesh->TileSizeUU, CellSize);

	OutConfig.regionChunkSize = FMath::Max(1, OutConfig.tileSize / FMath::Max(1, DestNavMesh->LayerChunkSplits));
	OutConfig.TileCacheChunkSize = FMath::Max(1, OutConfig.tileSize / FMath::Max(1, DestNavMesh->RegionChunkSplits));

	// Update all settings that depends directly or indirectly of the CellHeight
	OutConfig.ch = CellHeight;
	OutConfig.walkableHeight = DestNavMesh->bMarkLowHeightAreas ? 1 : FMath::CeilToInt(DestNavMesh->AgentHeight / CellHeight);
	OutConfig.walkableClimb = FMath::CeilToInt(AgentMaxStepHeight / CellHeight);

	// Update all settings that depends directly or indirectly of AgentMaxStepHeight
	OutConfig.AgentMaxClimb = AgentMaxStepHeight;

	OutConfig.bIsTileSetupConfigCompleted = true;
}

void FRecastNavMeshGenerator::ConfigureBuildProperties(FRecastBuildConfig& OutConfig)
{
	// @TODO those variables should be tweakable per navmesh actor
	const float CellSize = DestNavMesh->GetCellSize(ENavigationDataResolution::Default);
	ensure(CellSize != 0.f);
	const float CellHeight = DestNavMesh->GetCellHeight(ENavigationDataResolution::Default);
	const float AgentHeight = DestNavMesh->AgentHeight;
	const float AgentMaxSlope = DestNavMesh->AgentMaxSlope;
	const float AgentMaxClimb = DestNavMesh->GetAgentMaxStepHeight(ENavigationDataResolution::Default);
	const float AgentRadius = DestNavMesh->AgentRadius;

	OutConfig.Reset();

	OutConfig.cs = CellSize;
	OutConfig.ch = CellHeight;
	OutConfig.walkableSlopeAngle = AgentMaxSlope;
	OutConfig.walkableHeight = FMath::CeilToInt(AgentHeight / CellHeight);
	OutConfig.walkableClimb = FMath::CeilToInt(AgentMaxClimb / CellHeight);
	OutConfig.walkableRadius = FMath::CeilToInt(AgentRadius / CellSize);
	OutConfig.maxStepFromWalkableSlope = OutConfig.cs * FMath::Tan(FMath::DegreesToRadians(OutConfig.walkableSlopeAngle));
	
	// For each navmesh resolutions, validate that AgentMaxStepHeight is high enough for the AgentMaxSlope angle
	for (int32 Index = 0; Index < (uint8)ENavigationDataResolution::MAX; Index++)
	{
		const ENavigationDataResolution Resolution = (ENavigationDataResolution)Index;
		
		const float MaxStepHeight = DestNavMesh->GetAgentMaxStepHeight(Resolution);
		const float TempCellHeight = DestNavMesh->GetCellHeight(Resolution);
		const int WalkableClimbVx = FMath::CeilToInt(MaxStepHeight / TempCellHeight);

		// Compute the required climb to prevent direct neighbor filtering in rcFilterLedgeSpansImp (minh < -walkableClimb).
		// See comment: "The current span is close to a ledge if the drop to any neighbour span is less than the walkableClimb."
		const float RequiredClimb = DestNavMesh->GetCellSize(Resolution) * FMath::Tan(FMath::DegreesToRadians(AgentMaxSlope));
		const int RequiredClimbVx = FMath::CeilToInt(RequiredClimb / TempCellHeight);
		
		if (WalkableClimbVx < RequiredClimbVx)
		{
			// This is a log since we need to let the user decide which one of the parameters needs to be changed (if any).
			UE_LOGF(LogNavigationDataBuild, Log, "%ls: AgentMaxStepHeight (%f) for resolution %ls is not high enough in steep slopes (AgentMaxSlope is %f). "
				"Use AgentMaxStepHeight bigger than %f or a smaller AgentMaxSlope to avoid undesirable navmesh holes in steep slopes. "
				"This can also be avoided by using smaller CellSize and CellHeight.",
				*GetNameSafe(DestNavMesh), MaxStepHeight,
				*UEnum::GetDisplayValueAsText(Resolution).ToString(), AgentMaxSlope, static_cast<float>(RequiredClimbVx-1)*TempCellHeight);
		}
	}

	if (IsGeneratingLinks())
	{
		// NavLink builder configuration
		check(OutConfig.JumpConfigs.IsEmpty());
		for (uint8 Index = 0; Index < DestNavMesh->NavLinkJumpConfigs.Num() && Index < TNumericLimits<uint8>::Max(); Index++)
		{
			const FNavLinkGenerationJumpConfig& JumpConfig = DestNavMesh->NavLinkJumpConfigs[Index];

			dtNavLinkBuilderJumpConfig& DetourJumpConfig = OutConfig.JumpConfigs.AddDefaulted_GetRef();
			JumpConfig.CopyToDetourConfig(DetourJumpConfig);
			DetourJumpConfig.ConfigIndex = Index;

			const float JumpSpillDistance = JumpConfig.bEnabled ? JumpConfig.JumpLength - JumpConfig.JumpDistanceFromEdge : 0.f;
			OutConfig.LinkSpillDistance = FMath::Max(JumpSpillDistance, OutConfig.LinkSpillDistance);
		}
	}
	
	// store original sizes
	OutConfig.AgentHeight = AgentHeight;
	OutConfig.AgentMaxClimb = AgentMaxClimb;
	OutConfig.AgentRadius = AgentRadius;

	OutConfig.AverageLayersPerTile = FMath::Max(1.f, DestNavMesh->AverageLayersPerTile);

	UE::NavMesh::Private::ComputeConfigBorderSizes(IsGeneratingLinks(), OutConfig);

	OutConfig.maxEdgeLen = (int32)(1200.0f / CellSize);

	// hardcoded, but can be overridden by RecastNavMesh params later
	OutConfig.minRegionArea = (int32)rcSqr(0);
	OutConfig.mergeRegionArea = (int32)rcSqr(20.f);

	OutConfig.maxVertsPerPoly = (int32)MAX_VERTS_PER_POLY;
	OutConfig.detailSampleDist = 600.0f;
	OutConfig.detailSampleMaxError = 1.0f;

	OutConfig.minRegionArea = (int32)rcSqr(DestNavMesh->MinRegionArea / CellSize);
	OutConfig.mergeRegionArea = (int32)rcSqr(DestNavMesh->MergeRegionSize / CellSize);
	OutConfig.maxSimplificationError = DestNavMesh->MaxSimplificationError;
	OutConfig.simplificationElevationRatio = DestNavMesh->SimplificationElevationRatio;
	OutConfig.bPerformVoxelFiltering = DestNavMesh->bPerformVoxelFiltering;
	OutConfig.bMarkLowHeightAreas = DestNavMesh->bMarkLowHeightAreas;
	OutConfig.bUseExtraTopCellWhenMarkingAreas = DestNavMesh->bUseExtraTopCellWhenMarkingAreas;
	OutConfig.bFilterLowSpanSequences = DestNavMesh->bFilterLowSpanSequences;
	OutConfig.bFilterLowSpanFromTileCache = DestNavMesh->bFilterLowSpanFromTileCache;
	OutConfig.bMinimizeLinkPoolSize = DestNavMesh->bMinimizeLinkPoolSize;
	if (DestNavMesh->bMarkLowHeightAreas)
	{
		OutConfig.walkableHeight = 1;
	}

	OutConfig.bGenerateLinks = IsGeneratingLinks();

	const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	OutConfig.AgentIndex = NavSys ? NavSys->GetSupportedAgentIndex(DestNavMesh) : 0;

	OutConfig.tileSize = FMath::Max(FMath::TruncToInt(DestNavMesh->TileSizeUU / CellSize), 1);
	UE_CLOGF(OutConfig.tileSize == 1, LogNavigation, Error, "RecastNavMesh TileSize of 1 is highly discouraged. This occurence indicates an issue with RecastNavMesh\'s generation properties (specifically TileSizeUU: %f, CellSize: %f). Please ensure their correctness."
		, DestNavMesh->TileSizeUU, CellSize);

	OutConfig.regionChunkSize = FMath::Max(1, OutConfig.tileSize / FMath::Max(1, DestNavMesh->LayerChunkSplits));
	OutConfig.TileCacheChunkSize = FMath::Max(1, OutConfig.tileSize / FMath::Max(1, DestNavMesh->RegionChunkSplits));
	OutConfig.LedgeSlopeFilterMode = DestNavMesh->LedgeSlopeFilterMode;
	OutConfig.regionPartitioning = DestNavMesh->LayerPartitioning;
	OutConfig.TileCachePartitionType = DestNavMesh->RegionPartitioning;
}

void FRecastNavMeshGenerator::Init()
{
	check(DestNavMesh);

	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());

	if (NavSys)
	{
		SyncTimeSlicedData.TimeSliceManager = &(NavSys->GetMutableNavRegenTimeSliceManager());
	}
	else
	{
		//no time slice manager no time sliced regen
		SyncTimeSlicedData.bTimeSliceRegenActive = false;
	}

	ConfigureBuildProperties(Config);

	if (UE::NavMesh::Private::bUseTightBoundExpansion)
	{
		// Recast axis are inverted so growth direction are inverted here (using the positive side border for the low box growth and vice versa).
		const rcReal HorizontalGrowhtLow = Config.borderSize.high * Config.cs;
		BBoxGrowthLow = FVector(HorizontalGrowhtLow, HorizontalGrowhtLow, Config.cs);

		const rcReal HorizontalGrowthHigh = Config.borderSize.low * Config.cs;
		BBoxGrowthHigh = FVector(HorizontalGrowthHigh, HorizontalGrowthHigh, Config.cs);

    PRAGMA_DISABLE_DEPRECATION_WARNINGS
	    BBoxGrowth = FVector(static_cast<rcReal>(FMath::Max(Config.borderSize.low, Config.borderSize.high)) * Config.cs);
    PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		// Not using bUseTightBoundExpansion is deprecated, setting values mimicking previous behavior.

    PRAGMA_DISABLE_DEPRECATION_WARNINGS
	    BBoxGrowth = FVector(2.0 * static_cast<rcReal>(FMath::Max(Config.borderSize.low, Config.borderSize.high)) * Config.cs);
    PRAGMA_ENABLE_DEPRECATION_WARNINGS

		BBoxGrowthLow = FVector(2.0 * static_cast<rcReal>(Config.borderSize.high) * Config.cs);
		BBoxGrowthHigh = FVector(2.0 * static_cast<rcReal>(Config.borderSize.low) * Config.cs);
	}
	RcNavMeshOrigin = Unreal2RecastPoint(DestNavMesh->NavMeshOriginOffset);
	
	AdditionalCachedData = FRecastNavMeshCachedData::Construct(DestNavMesh);

	// Must be called after AdditionalCachedData is set.
	ResolveGeneratedLinkAreas(Config);

	if (Config.MaxPolysPerTile <= 0 && DestNavMesh->HasValidNavmesh())
	{
		const dtNavMeshParams* SavedNavParams = DestNavMesh->GetRecastNavMeshImpl()->DetourNavMesh->getParams();
		if (SavedNavParams)
		{
			Config.MaxPolysPerTile = SavedNavParams->maxPolys;
		}
	}

	ensure(DestNavMesh->GetRecastMesh() == nullptr || DestNavMesh->GetRecastMesh()->getBVQuantFactor((unsigned char)ENavigationDataResolution::Default) != 0);

	UpdateNavigationBounds();

	/** setup maximum number of active tile generator*/
	const int32 NumberOfWorkerThreads = FTaskGraphInterface::Get().GetNumWorkerThreads();
	MaxTileGeneratorTasks = FMath::Min(FMath::Max(NumberOfWorkerThreads * 2, 1), GetOwner() ? GetOwner()->GetMaxSimultaneousTileGenerationJobsCount() : INT_MAX);
	UE_LOGF(LogNavigation, Log, "Using max of %d workers to build navigation.", MaxTileGeneratorTasks);
	NumActiveTiles = 0;

	// prepare voxel cache if needed
	if (ARecastNavMesh::IsVoxelCacheEnabled())
	{
		VoxelCacheContext.Create(Config.tileSize + Config.borderSize.low + Config.borderSize.high, Config.cs, Config.ch);
	}

	bInitialized = true;


	int32 MaxTiles = 0;
	int32 MaxPolysPerTile = 0;

	// recreate navmesh if no data was loaded, or when loaded data doesn't match current grid layout
	bool bRecreateNavmesh = true;
	if (DestNavMesh->HasValidNavmesh())
	{
		const bool bGeometryRebuildDisabled = IsGeometryRebuildDisabled(DestNavMesh);
		const dtNavMeshParams* SavedNavParams = DestNavMesh->GetRecastNavMeshImpl()->DetourNavMesh->getParams();
		if (SavedNavParams)
		{
			if (bGeometryRebuildDisabled)
			{
				bRecreateNavmesh = false;
				MaxTiles = SavedNavParams->maxTiles;
				MaxPolysPerTile = SavedNavParams->maxPolys;
			}
			else
			{
				const FVector::FReal TileDim = Config.GetTileSizeUU();
				if (SavedNavParams->tileHeight == TileDim && SavedNavParams->tileWidth == TileDim)
				{
					const FVector Orig = Recast2UnrealPoint(SavedNavParams->orig);
					const FVector OrigError(FMath::Fmod(Orig.X, TileDim), FMath::Fmod(Orig.Y, TileDim), FMath::Fmod(Orig.Z, TileDim));
					if (OrigError.IsNearlyZero())
					{
						bRecreateNavmesh = false;
					}
					else
					{
						UE_LOGF(LogNavigation, Warning, "Recreating dtNavMesh instance %ls due to saved navmesh origin (%ls, usually the RecastNavMesh location) not being aligned with tile size (%d uu) "
							, *GetNameSafe(DestNavMesh), *Orig.ToString(), int(TileDim));
					}
				}

				// if new navmesh needs more tiles, force recreating
				if (!bRecreateNavmesh)
				{
					CalcNavMeshProperties(MaxTiles, MaxPolysPerTile);

					// Only check for a tile count mismatch if there is somewhere to build navigation.
					// With no bounds (and bWholeWorldNavigable false), MaxTiles == 0 is expected and correct.
					if (NavSys && NavSys->IsThereAnywhereToBuildNavigation() &&
						FMath::CeilToInt(FMath::Log2(static_cast<float>(MaxTiles))) != FMath::CeilToInt(FMath::Log2(static_cast<float>(SavedNavParams->maxTiles))))
					{
						bRecreateNavmesh = true;
						UE_LOGF(LogNavigation, Warning, "Recreating dtNavMesh instance (%ls) due mismatch in number of bytes required to store serialized maxTiles "
							"(serialized: %d, %d bits) vs calculated required (%d, %d bits)"
							, *DestNavMesh->GetName()
							, SavedNavParams->maxTiles, FMath::CeilToInt(FMath::Log2(static_cast<float>(SavedNavParams->maxTiles)))
							, MaxTiles, FMath::CeilToInt(FMath::Log2(static_cast<float>(MaxTiles))));
					}

					UE::NavMesh::Private::CheckTileIndicesInValidRange(InclusionBounds, *DestNavMesh);
				}
			}
		};
	}

	// Ensure MaxPolysPerTile is always populated before use.
	// CalcNavMeshProperties may not have been called above if bRecreateNavmesh was set
	// by a path other than the tile count mismatch check (e.g. origin misalignment).
	if (MaxPolysPerTile == 0)
	{
		CalcNavMeshProperties(MaxTiles, MaxPolysPerTile);
	}

	if (bRecreateNavmesh && NavSys && NavSys->IsThereAnywhereToBuildNavigation())
	{
		// recreate navmesh from scratch if no data was loaded
		ConstructTiledNavMesh();

		// mark all the areas we need to update, which is the whole (known) navigable space if not restricted to active tiles
		if (NavSys && NavSys->IsActiveTilesGenerationEnabled() == false)
		{
			MarkNavBoundsDirty();
		}
	}
	else
	{
		// otherwise just update generator params
		Config.MaxPolysPerTile = MaxPolysPerTile;
		NumActiveTiles = GetTilesCountHelper(DestNavMesh->GetRecastNavMeshImpl()->DetourNavMesh);
	}
}

void FRecastNavMeshGenerator::UpdateNavigationBounds()
{
	UE_LOGF(LogNavigation, Log, "%s", __FUNCTION__);
	
	const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	if (NavSys)
	{		
		if (NavSys->ShouldGenerateNavigationEverywhere() == false)
		{
			FBox BoundsSum(ForceInit);
			if (DestNavMesh)
			{
				TArray<FBox> SupportedBounds;
				NavSys->GetNavigationBoundsForNavData(*DestNavMesh, SupportedBounds);
				InclusionBounds.Reset(SupportedBounds.Num());

				for (const FBox& Box : SupportedBounds)
				{
					InclusionBounds.Add(Box);
					BoundsSum += Box;
				}
			}
			TotalNavBounds = BoundsSum;
			UE_LOGF(LogNavigation, Log, "  %s TotalNavBounds: %ls.", __FUNCTION__, *TotalNavBounds.ToCompactString());
		}
		else
		{
			InclusionBounds.Reset(1);
			TotalNavBounds = NavSys->GetWorldBounds();
			UE_LOGF(LogNavigation, Log, "  %s bWholeWorldNavigable==true, TotalNavBounds: %ls.", __FUNCTION__, *TotalNavBounds.ToCompactString());
			if (TotalNavBounds.IsValid)
			{
				InclusionBounds.Add(TotalNavBounds);
			}
		}
	}
	else
	{
		TotalNavBounds = FBox(ForceInit);
	}
}

bool FRecastNavMeshGenerator::ConstructTiledNavMesh() 
{
	bool bSuccess = false;

	// There is should not be any active build tasks
	CancelBuild();

	// create new Detour navmesh instance
	dtNavMesh* DetourMesh = dtAllocNavMesh();	
	if (DetourMesh)
	{
		ensureMsgf(DestNavMesh, TEXT("ConstructTiledNavMesh doesn't have a valid DestNavMesh"));

		++Version;
		
		dtNavMeshParams TiledMeshParameters;
		FMemory::Memzero(TiledMeshParameters);	

		FVector NMOrigin = RcNavMeshOrigin;
		rcVcopy(TiledMeshParameters.orig, &NMOrigin.X);

		TiledMeshParameters.tileWidth = Config.GetTileSizeUU();
		TiledMeshParameters.tileHeight = Config.GetTileSizeUU();

		CalcNavMeshProperties(TiledMeshParameters.maxTiles, TiledMeshParameters.maxPolys);
		Config.MaxPolysPerTile = TiledMeshParameters.maxPolys;

		TiledMeshParameters.walkableClimb = Config.AgentMaxClimb;
		TiledMeshParameters.walkableHeight = Config.AgentHeight;
		TiledMeshParameters.walkableRadius = Config.AgentRadius;
		TiledMeshParameters.resolutionParams[(uint8)ENavigationDataResolution::Low].bvQuantFactor = 1.f / DestNavMesh->GetCellSize(ENavigationDataResolution::Low);
		TiledMeshParameters.resolutionParams[(uint8)ENavigationDataResolution::Default].bvQuantFactor = 1.f / DestNavMesh->GetCellSize(ENavigationDataResolution::Default);
		TiledMeshParameters.resolutionParams[(uint8)ENavigationDataResolution::High].bvQuantFactor = 1.f / DestNavMesh->GetCellSize(ENavigationDataResolution::High);

		if (TiledMeshParameters.maxTiles == 0)
		{
			UE_LOGF(LogNavigation, Warning, "ConstructTiledNavMesh: Failed to create navmesh of size 0.");
			bSuccess = false;
		}
		else
		{
			const dtStatus status = DetourMesh->init(&TiledMeshParameters);

			if (dtStatusFailed(status))
			{
				UE_LOGF(LogNavigation, Warning, "ConstructTiledNavMesh: Could not init navmesh.");
				bSuccess = false;
			}
			else
			{
				bSuccess = true;

				if (!ensureMsgf(DestNavMesh->GetRecastNavMeshImpl(), TEXT("ConstructTiledNavMesh doesn't have a valid DestNavMesh->GetRecastNavMeshImpl")))
				{
					bSuccess = false;
				}

				if (bSuccess)
				{
					NumActiveTiles = GetTilesCountHelper(DetourMesh);
					DestNavMesh->GetRecastNavMeshImpl()->SetRecastMesh(DetourMesh);

					UE::NavMesh::Private::CheckTileIndicesInValidRange(InclusionBounds, *DestNavMesh);
				}
			}
		}

		if (bSuccess == false)
		{
			dtFreeNavMesh(DetourMesh);
		}
	}
	else
	{
		UE_LOGF(LogNavigation, Warning, "ConstructTiledNavMesh: Could not allocate navmesh." );
		bSuccess = false;
	}
	
	return bSuccess;
}

void FRecastNavMeshGenerator::CalcPolyRefBits(ARecastNavMesh* NavMeshOwner, int32& MaxTileBits, int32& MaxPolyBits)
{
	static const int32 TotalBits = (sizeof(dtPolyRef) * 8);
#if USE_64BIT_ADDRESS
	MaxTileBits = NavMeshOwner ? static_cast<int32>(FMath::CeilToFloat(FMath::Log2(static_cast<float>(NavMeshOwner->GetTileNumberHardLimit())))) : 20;
	MaxPolyBits = FMath::Min<int32>(32, (TotalBits - DT_MIN_SALT_BITS) - MaxTileBits);
#else
	MaxTileBits = 14;
	MaxPolyBits = (TotalBits - DT_MIN_SALT_BITS) - MaxTileBits;
#endif//USE_64BIT_ADDRESS
}

bool FRecastNavMeshGenerator::IsGeneratingLinks() const
{
	return DestNavMesh && DestNavMesh->IsGeneratingLinks(); 
}

void FRecastNavMeshGenerator::ResolveGeneratedLinkAreas(FRecastBuildConfig& OutConfig)
{
	if (IsGeneratingLinks())
	{
		auto GetAreaIDAndPolyFlags = [this](const TSubclassOf<UNavAreaBase>& AreaClass, unsigned char& OutAreaID, ARecastNavMesh::FNavPolyFlags& OutPolyFlags)
		{
			if (AreaClass)
			{
				const int32* AreaIDPtr = AdditionalCachedData.AreaClassToIdMap.Find(AreaClass);
				if (AreaIDPtr != nullptr)
				{
					OutAreaID = IntCastChecked<unsigned char>(*AreaIDPtr);

					// Using FlagsPerOffMeshLinkArea rather than FlagsPerArea because this function, ResolveGeneratedLinkAreas, 
					// is concerned only with navlinks (a.k.a. off-mesh links).
					OutPolyFlags = AdditionalCachedData.FlagsPerOffMeshLinkArea[*AreaIDPtr];
				}
				else
				{
					UE_LOGF(LogNavigation, Warning, "FRecastTileGenerator: Trying to use undefined area class while resolving generated links areas. (%ls)", *GetNameSafe(AreaClass));
				}
			}
		};

		if (ensureMsgf(DestNavMesh->NavLinkJumpConfigs.Num() == OutConfig.JumpConfigs.Num(), TEXT("Link configuration sizes are expected to match.")))
		{
			for (int32 Index = 0; Index < DestNavMesh->NavLinkJumpConfigs.Num(); Index++)
			{
				const FNavLinkGenerationJumpConfig& JumpConfig = DestNavMesh->NavLinkJumpConfigs[Index];
				dtNavLinkBuilderJumpConfig& DetourJumpConfig = OutConfig.JumpConfigs[Index];

				GetAreaIDAndPolyFlags(JumpConfig.DownDirectionAreaClass, DetourJumpConfig.downDirArea, DetourJumpConfig.downDirPolyFlag);
				GetAreaIDAndPolyFlags(JumpConfig.UpDirectionAreaClass, DetourJumpConfig.upDirArea, DetourJumpConfig.upDirPolyFlag);

				if (DetourJumpConfig.downDirArea == RECAST_NULL_AREA && DetourJumpConfig.upDirArea == RECAST_NULL_AREA)
				{
					UE_LOGF(LogNavigation, Warning, "FRecastTileGenerator: no area class has been defined for the jump down links. Disabling them");
					DetourJumpConfig.enabled = false;
				}
			}
		}
	}
}

void FRecastNavMeshGenerator::CalcNavMeshProperties(int32& MaxTiles, int32& MaxPolys)
{
	int32 MaxTileBits = -1;
	int32 MaxPolyBits = -1;

	// limit max amount of tiles
	CalcPolyRefBits(DestNavMesh, MaxTileBits, MaxPolyBits);
	
	const int32 MaxTilesFromMask = (1 << MaxTileBits);
	int32 MaxRequestedTiles = 0;
	if (DestNavMesh->IsResizable())
	{
		MaxRequestedTiles = UE::NavMesh::Private::CalculateMaxTilesCount(
			InclusionBounds, Config.GetTileSizeUU(), Config.AverageLayersPerTile, DestNavMesh->NavMeshVersion);
	}
	else
	{
		MaxRequestedTiles = DestNavMesh->TilePoolSize;
	}

	if (MaxRequestedTiles < 0 || MaxRequestedTiles > MaxTilesFromMask)
	{
		UE_LOGF(LogNavigation, Error, "Navmesh bounds are too large! Limiting requested tiles count (%d) to: (%d) for %ls. To resolve this, try using bigger tiles or increasing the TileNumberHardLimit in the NavMesh properties.",
			MaxRequestedTiles, MaxTilesFromMask, *GetFullNameSafe(DestNavMesh));
		MaxRequestedTiles = MaxTilesFromMask;
	}

	// Max tiles and max polys affect how the tile IDs are calculated.
	// There are (sizeof(dtPolyRef)*8 - DT_MIN_SALT_BITS) bits available for 
	// identifying a tile and a polygon.
#if USE_64BIT_ADDRESS
	MaxPolys = (MaxPolyBits >= 32) ? INT_MAX : (1 << MaxPolyBits);
#else
	MaxPolys = 1 << ((sizeof(dtPolyRef) * 8 - DT_MIN_SALT_BITS) - MaxTileBits);
#endif // USE_64BIT_ADDRESS
	MaxTiles = MaxRequestedTiles;
}

bool FRecastNavMeshGenerator::RebuildAll()
{
	DestNavMesh->UpdateNavVersion();
	
	const UNavigationSystemV1* NavSysForRebuild = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	// Only recreate and rebuild the navmesh if there is somewhere to build navigation.
	// This accounts for both registered bounds volumes and the bWholeWorldNavigable flag.
	// If there is nowhere to build, leave the existing mesh intact to avoid a broken null-mesh state.
	if (NavSysForRebuild && NavSysForRebuild->IsThereAnywhereToBuildNavigation())
	{
		// Recreate recast navmesh
		DestNavMesh->GetRecastNavMeshImpl()->ReleaseDetourNavMesh();
		DestNavMesh->bHasNoTileData = false;

		RcNavMeshOrigin = Unreal2RecastPoint(DestNavMesh->NavMeshOriginOffset);

		ConstructTiledNavMesh();
		MarkNavBoundsDirty();
		RebuildAllStartTime = FPlatformTime::Seconds();
	}
	else
	{
		// Nowhere to build — remove all existing tiles so the mesh is empty but structurally valid.
		// This prevents stale tile data from persisting after all NavMeshBoundsVolumes are removed.
		TArray<FNavTileRef> UpdatedTiles;
		RemoveAllTiles(UpdatedTiles);
		if (UpdatedTiles.Num() > 0)
		{
			DestNavMesh->OnNavMeshTilesUpdated(UpdatedTiles);
		}
		DestNavMesh->RequestDrawingUpdate();
	}

	return true;
}

void FRecastNavMeshGenerator::EnsureBuildCompletion()
{
	const bool bHadTasks = GetNumRemaningBuildTasks() > 0;
	
	const bool bDoAsyncDataGathering = (GatherGeometryOnGameThread() == false);
	do 
	{
#if !RECAST_ASYNC_REBUILDING
		// When time slicing is enabled, we need to setup the time slice before processing tiles.
		// Use a large duration since EnsureBuildCompletion wants to complete all work immediately.
		if (SyncTimeSlicedData.TimeSliceManager && SyncTimeSlicedData.bTimeSliceRegenActive)
		{
			SyncTimeSlicedData.TimeSliceManager->GetTimeSlicer().SetupTimeSlice(TNumericLimits<double>::Max());
		}
#endif // !RECAST_ASYNC_REBUILDING

		const int32 NumTasksToProcess = (bDoAsyncDataGathering ? 1 : MaxTileGeneratorTasks) - RunningDirtyTiles.Num();
		ProcessTileTasksAndGetUpdatedTiles(NumTasksToProcess);
		
		// Block until tasks are finished
		for (TRunningTileElement<FRecastTileGeneratorWrapper>& Element : RunningDirtyTiles)
		{
			Element.AsyncTask->EnsureCompletion();
		}
	}
	while (GetNumRemaningBuildTasks() > 0);

	// Update navmesh drawing only if we had something to build
	if (bHadTasks)
	{
		DestNavMesh->RequestDrawingUpdate();
	}
}

void FRecastNavMeshGenerator::CancelBuild()
{
	DiscardCurrentBuildingTasks();

#if	WITH_EDITOR	
	RecentlyBuiltTiles.Empty();
#endif//WITH_EDITOR
}

void FRecastNavMeshGenerator::TickAsyncBuild(float DeltaSeconds)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_TickAsyncBuild);

	bool bRequestDrawingUpdate = false;

#if	WITH_EDITOR
	// Remove expired tiles
	{
		const double Timestamp = FPlatformTime::Seconds();
		const int32 NumPreRemove = RecentlyBuiltTiles.Num();
		
		RecentlyBuiltTiles.RemoveAllSwap([&](const FTileTimestamp& Tile) { return (Timestamp - Tile.Timestamp) > UE::NavMesh::Private::RecentlyBuildTileDisplayTime; });

		const int32 NumPostRemove = RecentlyBuiltTiles.Num();
		bRequestDrawingUpdate = (NumPreRemove != NumPostRemove);
	}
#endif//WITH_EDITOR

	const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	if (!ensureMsgf(NavSys != nullptr, TEXT("FRecastNavMeshGenerator can't find valid navigation system: Owner=[%s] World=[%s]"), *GetFullNameSafe(GetOwner()), *GetFullNameSafe(GetWorld())))
	{
		return;
	}

	// Submit async tile build tasks in case we have dirty tiles and have room for them
	const int32 NumRunningTasks = NavSys->GetNumRunningBuildTasks();
	// this is a temp solution to enforce only one worker thread if GatherGeometryOnGameThread == false
	// due to missing safety features
	const bool bDoAsyncDataGathering = GatherGeometryOnGameThread() == false;

	const int32 NumTasksToSubmit = (bDoAsyncDataGathering ? 1 : MaxTileGeneratorTasks) - NumRunningTasks;
	TArray<FNavTileRef> UpdatedTileRefs = ProcessTileTasksAndGetUpdatedTiles(NumTasksToSubmit);
			
	if (UpdatedTileRefs.Num() > 0)
	{
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_OnNavMeshTilesUpdated);

			// Invalidate active paths that go through regenerated tiles
			DestNavMesh->OnNavMeshTilesUpdated(UpdatedTileRefs);
		}

		bRequestDrawingUpdate = true;

#if	WITH_EDITOR
		// Store completed tiles with timestamps to have ability to distinguish during debug draw
		const double Timestamp = FPlatformTime::Seconds();
		RecentlyBuiltTiles.Reserve(RecentlyBuiltTiles.Num() + UpdatedTileRefs.Num());
		for (const FNavTileRef TileRef : UpdatedTileRefs)
		{
			UE_SUPPRESS(LogNavigation, VeryVerbose,
			{
				if (DestNavMesh->GetRecastNavMeshImpl())
				{
					if (const dtNavMesh* DetourMesh = DestNavMesh->GetRecastNavMeshImpl()->GetRecastMesh())
					{
						const uint32 TileIndex = DetourMesh->decodePolyIdTile((dtTileRef)TileRef);
						const uint32 Salt = DetourMesh->decodePolyIdSalt((dtTileRef)TileRef);
						UE_LOGF(LogNavigation, VeryVerbose, "%ls Adding to RecentlyBuiltTiles TileId: %d Salt: %d TileRef: 0x%llx", ANSI_TO_TCHAR(__FUNCTION__), TileIndex, Salt, (dtTileRef)TileRef);
					}
				}
			});
			
			FTileTimestamp TileTimestamp;
			TileTimestamp.NavTileRef = TileRef;
			TileTimestamp.Timestamp = Timestamp;
			RecentlyBuiltTiles.Add(TileTimestamp);
		}
#endif//WITH_EDITOR
	}

	if (bRequestDrawingUpdate)
	{
		DestNavMesh->RequestDrawingUpdate();
	}
}

void FRecastNavMeshGenerator::OnNavigationBoundsChanged()
{
	check(DestNavMesh);

	UpdateNavigationBounds();
	
	dtNavMesh* DetourMesh = DestNavMesh->GetRecastNavMeshImpl() ? DestNavMesh->GetRecastNavMeshImpl()->GetRecastMesh() : nullptr;
	if (!IsGeometryRebuildDisabled(DestNavMesh) && DetourMesh)
	{
		const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
		if (NavSys && !NavSys->IsThereAnywhereToBuildNavigation())
		{
			// Nowhere to build navigation — clear all existing tiles so the mesh is empty but structurally valid.
			// This accounts for both the case where all bounds volumes are removed and bWholeWorldNavigable is false.
			UE_LOGF(LogNavigation, Log, "%ls> No navigation bounds, clearing navmesh tiles.", *DestNavMesh->GetName());
			TArray<FNavTileRef> UpdatedTiles;
			RemoveAllTiles(UpdatedTiles);
			if (UpdatedTiles.Num() > 0)
			{
				DestNavMesh->OnNavMeshTilesUpdated(UpdatedTiles);
			}
		}
		else if (DestNavMesh->IsResizable())
		{
			if (NavSys && !NavSys->IsNavigationBuildingLocked())
			{
				// Check whether Navmesh size needs to be changed
				const int32 MaxRequestedTiles = UE::NavMesh::Private::CalculateMaxTilesCount(
					InclusionBounds, Config.GetTileSizeUU(), Config.AverageLayersPerTile, DestNavMesh->NavMeshVersion);

				if (DetourMesh->getMaxTiles() != MaxRequestedTiles)
				{
					UE_LOGF(LogNavigation, Log, "%ls> Navigation bounds changed, rebuilding navmesh", *DestNavMesh->GetName());

					// Destroy and recreate the navmesh with the new tile count.
					DestNavMesh->GetRecastNavMeshImpl()->SetRecastMesh(nullptr);

					TArray<FNavigationDirtyArea> AsDirtyAreas;
					AsDirtyAreas.Reserve(InclusionBounds.Num());
					for (const FBox& BBox : InclusionBounds)
					{
						AsDirtyAreas.Add(FNavigationDirtyArea(BBox, ENavigationDirtyFlag::NavigationBounds));
					}

					RebuildDirtyAreas(AsDirtyAreas);
				}

				UE::NavMesh::Private::CheckTileIndicesInValidRange(InclusionBounds, *DestNavMesh);
			}
		}
	}
}

void FRecastNavMeshGenerator::RebuildDirtyAreas(const TArray<FNavigationDirtyArea>& InDirtyAreas)
{
	dtNavMesh* DetourMesh = DestNavMesh->GetRecastNavMeshImpl()->GetRecastMesh();
	if (DetourMesh == nullptr)
	{
		ConstructTiledNavMesh();
	}
	
	MarkDirtyTiles(InDirtyAreas);
}

void FRecastNavMeshGenerator::OnAreaAdded(const UClass* AreaClass, int32 AreaID)
{
	AdditionalCachedData.OnAreaAdded(AreaClass, AreaID);
}

void FRecastNavMeshGenerator::OnAreaRemoved(const UClass* AreaClass)
{
	AdditionalCachedData.OnAreaRemoved(AreaClass);
}

int32 FRecastNavMeshGenerator::FindInclusionBoundEncapsulatingBox(const FBox& Box) const
{
	for (int32 Index = 0; Index < InclusionBounds.Num(); ++Index)
	{
		if (DoesBoxContainBox(InclusionBounds[Index], Box))
		{
			return Index;
		}
	}

	return INDEX_NONE;
}

void FRecastNavMeshGenerator::RestrictBuildingToActiveTiles(bool InRestrictBuildingToActiveTiles) 
{ 
	if (bRestrictBuildingToActiveTiles != InRestrictBuildingToActiveTiles)
	{
		bRestrictBuildingToActiveTiles = InRestrictBuildingToActiveTiles;
		if (InRestrictBuildingToActiveTiles)
		{
			// gather non-empty tiles and add them to ActiveTileSet

			const dtNavMesh* DetourMesh = DestNavMesh->GetRecastNavMeshImpl()->GetRecastMesh();

			if (DetourMesh != nullptr && DetourMesh->isEmpty() == false)
			{
				ActiveTileSet.Reset();
				int32 TileCount = DetourMesh->getMaxTiles();
				for (int32 TileIndex = 0; TileIndex < TileCount; ++TileIndex)
				{
					const dtMeshTile* Tile = DetourMesh->getTile(TileIndex);
					if (Tile != nullptr && Tile->header != nullptr && Tile->header->polyCount > 0)
					{
						ActiveTileSet.FindOrAdd(FIntPoint(Tile->header->x, Tile->header->y));
					}
				}
			}
		}
	}
}

bool FRecastNavMeshGenerator::IsInActiveSet(const FIntPoint& Tile) const
{
	return bRestrictBuildingToActiveTiles == false || ActiveTileSet.Contains(Tile);
}

void FRecastNavMeshGenerator::ResetTimeSlicedTileGeneratorSync()
{
	SyncTimeSlicedData.TileGeneratorSync.Reset();

	//reset variables used for timeslicing TileGenratorSync
	SyncTimeSlicedData.ProcessTileTasksSyncState = EProcessTileTasksSyncTimeSlicedState::Init;
	SyncTimeSlicedData.UpdatedTilesCache.Reset();
	SyncTimeSlicedData.OldLayerTileIdMapCached.Reset();
	SyncTimeSlicedData.ResultTileRefsCached.Reset();
	SyncTimeSlicedData.AddGeneratedTilesState = EAddGeneratedTilesTimeSlicedState::Init;
	SyncTimeSlicedData.AddGenTilesLayerIndex = 0;
}

//@TODO Investigate removing from RunningDirtyTiles here too (or at least not using the results in any way)
void FRecastNavMeshGenerator::RemoveTiles(const TArray<FIntPoint>& Tiles)
{
	dtNavMesh* DetourMesh = DestNavMesh->GetRecastNavMeshImpl()->GetRecastMesh();
	const bool bIsDetourMeshValid = DetourMesh && !DetourMesh->isEmpty(); 
	
	for (const FIntPoint& TileXY : Tiles)
	{
		if (bIsDetourMeshValid)
		{
			RemoveTileLayers(DetourMesh, TileXY.X, TileXY.Y);
		}

		if (PendingDirtyTiles.Num() > 0)
		{
			FPendingTileElement DirtyTile;
			DirtyTile.Coord = TileXY;
			PendingDirtyTiles.RemoveSwap(DirtyTile);
		}

		if (SyncTimeSlicedData.TileGeneratorSync.IsValid())
		{
			if (SyncTimeSlicedData.TileGeneratorSync->GetTileX() == TileXY.X && SyncTimeSlicedData.TileGeneratorSync->GetTileY() == TileXY.Y)
			{
				ResetTimeSlicedTileGeneratorSync();
			}
		}
	}
}

void FRecastNavMeshGenerator::ReAddTiles(const TArray<FNavMeshDirtyTileElement>& Tiles)
{
	static const FVector Expansion(1, 1, BIG_NUMBER);
	// a little trick here - adding a dirty area so that navmesh building figures it out on its own
	dtNavMesh* DetourMesh = DestNavMesh->GetRecastNavMeshImpl()->GetRecastMesh();
	const dtNavMeshParams* SavedNavParams = DestNavMesh->GetRecastNavMeshImpl()->DetourNavMesh->getParams();

	TSet<FPendingTileElement> DirtyTiles;

	const double CurrentTimeSeconds = FPlatformTime::Seconds();
	const FVector::FReal NearDistanceSquared = FMath::Square(DestNavMesh->InvokerTilePriorityBumpDistanceThresholdInTileUnits*GetConfig().GetTileSizeUU());
	const uint8 PriorityIncrease = DestNavMesh->InvokerTilePriorityBumpIncrease;

	// @note we act on assumption all items in Tiles are unique
	for (const FNavMeshDirtyTileElement& Tile : Tiles)
	{
		FPendingTileElement Element;
		Element.Coord = Tile.Coordinates;
#if !UE_BUILD_SHIPPING		
		Element.DebugInvokerDistanceSquared = Tile.InvokerDistanceSquared;
		Element.DebugInvokerPriority = Tile.InvokerPriority;
#endif // !UE_BUILD_SHIPPING			

		// Bump sorting priority for tiles near invokers
		if (Tile.InvokerDistanceSquared < NearDistanceSquared)
		{
			Element.SortingPriority = (ENavigationInvokerPriority)FMath::Min((int)Tile.InvokerPriority+PriorityIncrease, (int)ENavigationInvokerPriority::MAX-1);	
		}
		else
		{
			Element.SortingPriority = Tile.InvokerPriority;
		}

		Element.bRebuildGeometry = true;
		Element.CreationTime = CurrentTimeSeconds;
		DirtyTiles.Add(Element);
	}

	const int32 NumTilesMarked = DirtyTiles.Num();

	// Merge all pending tiles into one container
	for (const FPendingTileElement& Element : PendingDirtyTiles)
	{
		FPendingTileElement* ExistingElement = DirtyTiles.Find(Element);
		if (ExistingElement)
		{
#if !UE_BUILD_SHIPPING			
			ExistingElement->DebugInvokerDistanceSquared = FMath::Min(ExistingElement->DebugInvokerDistanceSquared, Element.DebugInvokerDistanceSquared);
			ExistingElement->DebugInvokerPriority = FMath::Max(ExistingElement->DebugInvokerPriority, Element.DebugInvokerPriority);
#endif // !UE_BUILD_SHIPPING			
			ExistingElement->SortingPriority = FMath::Max(ExistingElement->SortingPriority, Element.SortingPriority);
			ExistingElement->bRebuildGeometry |= Element.bRebuildGeometry;
			ExistingElement->CreationTime = FMath::Min(Element.CreationTime, ExistingElement->CreationTime);
			// Append area bounds to existing list 
			if (ExistingElement->bRebuildGeometry == false)
			{
				ExistingElement->DirtyAreas.Append(Element.DirtyAreas);
			}
			else
			{
				ExistingElement->DirtyAreas.Empty();
			}
		}
		else
		{
			DirtyTiles.Add(Element);
		}
	}

	// Dump results into array
	PendingDirtyTiles.Empty(DirtyTiles.Num());
	for (const FPendingTileElement& Element : DirtyTiles)
	{
		PendingDirtyTiles.Add(Element);
	}

	// Sort tiles by proximity to players 
	if (NumTilesMarked > 0)
	{
		SortPendingBuildTiles();
	}
}

namespace RecastTileVersionHelper
{
	inline uint32 GetUpdatedTileId(dtPolyRef& TileRef, dtNavMesh* DetourMesh)
	{
		uint32 DecodedTileId = 0, DecodedPolyId = 0, DecodedSaltId = 0;
		DetourMesh->decodePolyId(TileRef, DecodedSaltId, DecodedTileId, DecodedPolyId);

		DecodedSaltId = (DecodedSaltId + 1) & ((1 << DetourMesh->getSaltBits()) - 1);
		if (DecodedSaltId == 0)
		{
			DecodedSaltId++;
		}

		TileRef = DetourMesh->encodePolyId(DecodedSaltId, DecodedTileId, DecodedPolyId);
		return DecodedTileId;
	}
}

TArray<FNavTileRef> FRecastNavMeshGenerator::RemoveTileLayersAndGetUpdatedTiles(const int32 TileX, const int32 TileY, TMap<int32, dtPolyRef>* OldLayerTileIdMap)
{
	dtNavMesh* DetourMesh = DestNavMesh->GetRecastNavMeshImpl()->GetRecastMesh();
	TArray<FNavTileRef> UpdatedIndices;
	
	if (DetourMesh != nullptr && DetourMesh->isEmpty() == false)
	{
		const int32 NumLayers = DetourMesh->getTileCountAt(TileX, TileY);

		if (NumLayers > 0)
		{
			TArray<dtMeshTile*> Tiles;
			Tiles.AddZeroed(NumLayers);
			DetourMesh->getTilesAt(TileX, TileY, (const dtMeshTile**)Tiles.GetData(), NumLayers);

			for (int32 i = 0; i < NumLayers; i++)
			{
				const int32 LayerIndex = Tiles[i]->header->layer;
				dtPolyRef TileRef = DetourMesh->getTileRef(Tiles[i]);

				NumActiveTiles--;
				DestNavMesh->LogRecastTile(ANSI_TO_TCHAR(__FUNCTION__), FName(), FName("removing"), *DetourMesh, TileX, TileY, LayerIndex, TileRef);

				DetourMesh->removeTile(TileRef, nullptr, nullptr);

				UpdatedIndices.AddUnique(FNavTileRef(TileRef));

				RecastTileVersionHelper::GetUpdatedTileId(TileRef, DetourMesh);	// Updates TileRef

				if (OldLayerTileIdMap)
				{
					OldLayerTileIdMap->Add(LayerIndex, TileRef);
				}
			}
		}

		// Remove compressed tile cache layers
		DestNavMesh->RemoveTileCacheLayers(TileX, TileY);

#if RECAST_INTERNAL_DEBUG_DATA
		DestNavMesh->RemoveTileDebugData(TileX, TileY);
#endif
	}

	return UpdatedIndices;
}

void FRecastNavMeshGenerator::RemoveTileLayers(dtNavMesh* DetourMesh, const int32 TileX, const int32 TileY)
{
	check(DetourMesh && !DetourMesh->isEmpty())
	
	const int32 NumLayers = DetourMesh->getTileCountAt(TileX, TileY);

	if (NumLayers > 0)
	{
		TArray<dtMeshTile*, TInlineAllocator<16>> Tiles;
		Tiles.AddZeroed(NumLayers);
		DetourMesh->getTilesAt(TileX, TileY, (const dtMeshTile**)Tiles.GetData(), NumLayers);

		for (int32 i = 0; i < NumLayers; i++)
		{
			const dtPolyRef TileRef = DetourMesh->getTileRef(Tiles[i]);
			NumActiveTiles--;

			UE_SUPPRESS(LogNavigation, VeryVerbose,
			{
				const int32 LayerIndex = Tiles[i]->header->layer;
				DestNavMesh->LogRecastTile(ANSI_TO_TCHAR(__FUNCTION__), FName(), FName("removing"), *DetourMesh, TileX, TileY, LayerIndex, TileRef);
			});

			DetourMesh->removeTile(TileRef, nullptr, nullptr);
		}
	}

	// Remove compressed tile cache layers
	DestNavMesh->RemoveTileCacheLayers(TileX, TileY);

#if RECAST_INTERNAL_DEBUG_DATA
	DestNavMesh->RemoveTileDebugData(TileX, TileY);
#endif
}

FRecastNavMeshGenerator::FSyncTimeSlicedData::FSyncTimeSlicedData()
	: CurrentTileRegenDuration(0.)
#if TIME_SLICE_NAV_REGEN
	, bTimeSliceRegenActive(true)
	, bNextTimeSliceRegenActive(true)
#else
	, bTimeSliceRegenActive(false)
	, bNextTimeSliceRegenActive(false)
#endif
	, ProcessTileTasksSyncState(EProcessTileTasksSyncTimeSlicedState::Init)
	, AddGeneratedTilesState(EAddGeneratedTilesTimeSlicedState::Init)
	, AddGenTilesLayerIndex(0)
	, TimeSliceManager(nullptr)
{
}

void FRecastNavMeshGenerator::AddGeneratedTileLayer(int32 LayerIndex, FRecastTileGenerator& TileGenerator, const TMap<int32, dtPolyRef>& OldLayerTileIdMap, TArray<FNavTileRef>& OutResultTileRefs)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastAddGeneratedTileLayer);

	struct FLayerIndexFinder
	{
		int32 LayerIndex;
		explicit FLayerIndexFinder(const int32 InLayerIndex) : LayerIndex(InLayerIndex) {}
		bool operator()(const FNavMeshTileData& LayerData) const
		{
			return LayerData.LayerIndex == LayerIndex;
		}
	};

	const int32 TileX = TileGenerator.GetTileX();
	const int32 TileY = TileGenerator.GetTileY();
	dtNavMesh* DetourMesh = DestNavMesh->GetRecastNavMeshImpl()->GetRecastMesh();
	TArray<FNavMeshTileData>& TileLayers = TileGenerator.GetNavigationData();
	dtTileRef OldTileRef = DetourMesh->getTileRefAt(TileX, TileY, LayerIndex);
	const int32 LayerDataIndex = TileLayers.IndexOfByPredicate(FLayerIndexFinder(LayerIndex));

	if (LayerDataIndex != INDEX_NONE)
	{
		FNavMeshTileData& LayerData = TileLayers[LayerDataIndex];
		if (OldTileRef)
		{
			NumActiveTiles--;
			DestNavMesh->LogRecastTile(ANSI_TO_TCHAR(__FUNCTION__), FName(), FName("removing"), *DetourMesh, TileX, TileY, LayerIndex, OldTileRef);

			DetourMesh->removeTile(OldTileRef, nullptr, nullptr);

			OutResultTileRefs.AddUnique(FNavTileRef(OldTileRef));
			
			RecastTileVersionHelper::GetUpdatedTileId(OldTileRef, DetourMesh);	// Updates OldTileRef
		}
		else
		{
			OldTileRef = OldLayerTileIdMap.FindRef(LayerIndex);
		}

		if (LayerData.IsValid())
		{
			bool bRejectNavmesh = false;
			dtTileRef ResultTileRef = 0;

			dtStatus status = 0;

			{
				// let navmesh know it's tile generator who owns the data
				status = DetourMesh->addTile(LayerData.GetData(), LayerData.DataSize, DT_TILE_FREE_DATA, OldTileRef, &ResultTileRef);

				// if tile index was already taken by other layer try adding it on first free entry (salt was already updated by whatever took that spot)
				if (dtStatusFailed(status) && dtStatusDetail(status, DT_OUT_OF_MEMORY) && OldTileRef)
				{
					OldTileRef = 0;
					status = DetourMesh->addTile(LayerData.GetData(), LayerData.DataSize, DT_TILE_FREE_DATA, OldTileRef, &ResultTileRef);
				}
			}

			if (dtStatusFailed(status))
			{
				if (dtStatusDetail(status, DT_OUT_OF_MEMORY))
				{
					UE_LOGF(LogNavigation, Warning, 
						"%ls> Failed to add tile (%d,%d:%d), %d tile limit reached! (from %s). Increase AverageLayersPerTile or, if using FixedTilePoolSize, increase the TilePoolSize or use bigger tiles.",
						*DestNavMesh->GetName(), TileX, TileY, LayerIndex, DetourMesh->getMaxTiles(), __FUNCTION__);
				}
			}
			else
			{
				OutResultTileRefs.AddUnique(FNavTileRef(ResultTileRef));
				NumActiveTiles++;

				DestNavMesh->LogRecastTile(ANSI_TO_TCHAR(__FUNCTION__), FName(), FName("added generated"), *DetourMesh, TileX, TileY, LayerIndex, ResultTileRef);

				{
					// NavMesh took the ownership of generated data, so we don't need to deallocate it
					uint8* ReleasedData = LayerData.Release();
				}
			}
		}
	}
	else
	{
		// remove the layer since it ended up empty
		DetourMesh->removeTile(OldTileRef, nullptr, nullptr);

		OutResultTileRefs.AddUnique(FNavTileRef(OldTileRef));
	}
}

bool FRecastNavMeshGenerator::IsAllowedToAddTileLayers(const FIntPoint Tile) const
{
	check(DestNavMesh);
	return !DestNavMesh->IsWorldPartitionedDynamicNavmesh() || IsInActiveSet(Tile) || (GetWorld() && !GetWorld()->IsGameWorld());
}

#if !UE_BUILD_SHIPPING
void FRecastNavMeshGenerator::LogDirtyAreas(const UObject& OwnerNav,
	const TMap<FPendingTileElement, TArray<FNavigationDirtyAreaPerTileDebugInformation>>& DirtyAreasDebuggingInformation) const
{
	// Helper struct used to collate the raw information provided to the method, needed for the log results
	struct FNavigationDirtyAreaDebugInformation
	{
		FNavigationDirtyArea DirtyArea;
		int32 NewlyAddedDirtyTiles = 0;
		int32 TotalDirtyTiles = 0;
	};
	
	// Array used to describe per dirty area the amount of dirtied tiles
	TArray<FNavigationDirtyAreaDebugInformation> DirtyAreaToDirtyTilesCount;
	for (const TPair<FPendingTileElement, TArray<FNavigationDirtyAreaPerTileDebugInformation>>& DirtyAreasDebuggingInformationPair : DirtyAreasDebuggingInformation)
	{
		for (const FNavigationDirtyAreaPerTileDebugInformation& DirtyAreaPerTileDebugInformation : DirtyAreasDebuggingInformationPair.Value)
		{
			FNavigationDirtyAreaDebugInformation* DirtyResultsTuple = DirtyAreaToDirtyTilesCount.FindByPredicate([&DirtyAreaPerTileDebugInformation](const FNavigationDirtyAreaDebugInformation& OtherDirtyResultsTuple)
			{
				return OtherDirtyResultsTuple.DirtyArea == DirtyAreaPerTileDebugInformation.DirtyArea;
			});

			if (!DirtyResultsTuple)
			{
				DirtyResultsTuple = &DirtyAreaToDirtyTilesCount.Add_GetRef({DirtyAreaPerTileDebugInformation.DirtyArea, 0, 0});
			}

			if (!DirtyAreaPerTileDebugInformation.bTileWasAlreadyAdded)
			{
				DirtyResultsTuple->NewlyAddedDirtyTiles++;
			}
			DirtyResultsTuple->TotalDirtyTiles++;
		}
	}

	for (const auto& [DirtyArea, NewlyAddedDirtyTiles, TotalDirtyTiles] : DirtyAreaToDirtyTilesCount)
	{
		const FVector2D BoundsSize(DirtyArea.Bounds.GetSize());
		
		UE_LOGF(LogNavigationDirtyArea, VeryVerbose,
			"(navmesh: %-30ls) Dirty area trying to dirty %2d tiles (out of which %2d are newly added/not pending) | Source = %ls | Bounds size = %ls)",
			*GetNameSafe(GetOwner()), TotalDirtyTiles, NewlyAddedDirtyTiles, *DirtyArea.GetSourceDescription(), *BoundsSize.ToString());

		UE_VLOG_BOX(&OwnerNav, LogNavigationDirtyArea, VeryVerbose, DirtyArea.Bounds, FColor::Purple,
			TEXT("Tiles %d (new: %d), Source: %s"), TotalDirtyTiles, NewlyAddedDirtyTiles, *DirtyArea.GetSourceDescription());
	}
}
#endif

ETimeSliceWorkResult FRecastNavMeshGenerator::AddGeneratedTilesTimeSliced(FRecastTileGenerator& TileGenerator, TArray<FNavTileRef>& OutResultTileRefs)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastAddGeneratedTiles);

	check(SyncTimeSlicedData.TimeSliceManager);

	const int32 TileX = TileGenerator.GetTileX();
	const int32 TileY = TileGenerator.GetTileY();
	dtNavMesh* DetourMesh = DestNavMesh->GetRecastNavMeshImpl()->GetRecastMesh();
	TArray<FNavMeshTileData>& TileLayers = TileGenerator.GetNavigationData();
	ETimeSliceWorkResult WorkResult = ETimeSliceWorkResult::Succeeded;
	bool bIteratedThroughDirtyLayers = true;

	switch (SyncTimeSlicedData.AddGeneratedTilesState)
	{
	case EAddGeneratedTilesTimeSlicedState::Init:
	{
		SyncTimeSlicedData.ResultTileRefsCached.Reset();
		SyncTimeSlicedData.ResultTileRefsCached.Reserve(TileLayers.Num());
		SyncTimeSlicedData.OldLayerTileIdMapCached.Reset();
		SyncTimeSlicedData.OldLayerTileIdMapCached.Reserve(TileLayers.Num());
		SyncTimeSlicedData.AddGenTilesLayerIndex = TileGenerator.GetDirtyLayersMask().Find(true);
		if (TileGenerator.IsFullyRegenerated())
		{
			// remove all layers
			SyncTimeSlicedData.ResultTileRefsCached = RemoveTileLayersAndGetUpdatedTiles(TileX, TileY, &SyncTimeSlicedData.OldLayerTileIdMapCached);
		}

		SyncTimeSlicedData.AddGeneratedTilesState = EAddGeneratedTilesTimeSlicedState::AddTiles;
	}//fall through to next state
	case EAddGeneratedTilesTimeSlicedState::AddTiles:
	{
		if (DetourMesh != nullptr
			&& IsAllowedToAddTileLayers(FIntPoint(TileX, TileY))
			&& SyncTimeSlicedData.AddGenTilesLayerIndex != INDEX_NONE)
		{
			for (; SyncTimeSlicedData.AddGenTilesLayerIndex < TileGenerator.GetDirtyLayersMask().Num(); ++SyncTimeSlicedData.AddGenTilesLayerIndex)
			{
				if (TileGenerator.IsLayerChanged(SyncTimeSlicedData.AddGenTilesLayerIndex))
				{
					if (SyncTimeSlicedData.TimeSliceManager->GetTimeSlicer().IsTimeSliceFinishedCached())
					{
						WorkResult = ETimeSliceWorkResult::CallAgainNextTimeSlice;
						break;
					}

					AddGeneratedTileLayer(SyncTimeSlicedData.AddGenTilesLayerIndex, TileGenerator, SyncTimeSlicedData.OldLayerTileIdMapCached, SyncTimeSlicedData.ResultTileRefsCached);

					MARK_TIMESLICE_SECTION_DEBUG(SyncTimeSlicedData.TimeSliceManager->GetTimeSlicer(), AddTiles);

					SyncTimeSlicedData.TimeSliceManager->GetTimeSlicer().TestTimeSliceFinished();
				}
			}
		}
		else
		{
			WorkResult = ETimeSliceWorkResult::Failed;
			bIteratedThroughDirtyLayers = false;
		}
	}
	break;

	default:
	{
		ensureMsgf(false, TEXT("unhandled EAddGeneratedTilesTimeSlicedState"));
		WorkResult = ETimeSliceWorkResult::Failed;
	}
	}

	if (SyncTimeSlicedData.AddGenTilesLayerIndex == TileGenerator.GetDirtyLayersMask().Num() || !bIteratedThroughDirtyLayers)
	{
		SyncTimeSlicedData.AddGenTilesLayerIndex = 0;
		SyncTimeSlicedData.AddGeneratedTilesState = EAddGeneratedTilesTimeSlicedState::Init;

		OutResultTileRefs = MoveTemp(SyncTimeSlicedData.ResultTileRefsCached);
	}

	return WorkResult;
}

TArray<FNavTileRef> FRecastNavMeshGenerator::AddGeneratedTilesAndGetUpdatedTiles(FRecastTileGenerator& TileGenerator)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RecastAddGeneratedTiles);

	TMap<int32, dtPolyRef> OldLayerTileIdMap;
	TArray<FNavTileRef> ResultTileRefs;
	const int32 TileX = TileGenerator.GetTileX();
	const int32 TileY = TileGenerator.GetTileY();

	if (TileGenerator.IsFullyRegenerated())
	{
		// remove all layers
		ResultTileRefs = RemoveTileLayersAndGetUpdatedTiles(TileX, TileY, &OldLayerTileIdMap);
	}

	dtNavMesh* DetourMesh = DestNavMesh->GetRecastNavMeshImpl()->GetRecastMesh();
	const int32 FirstDirtyTileIndex = TileGenerator.GetDirtyLayersMask().Find(true);

	if (DetourMesh != nullptr
		&& IsAllowedToAddTileLayers(FIntPoint(TileX, TileY))
		&& FirstDirtyTileIndex != INDEX_NONE)
	{
		TArray<FNavMeshTileData> TileLayers = TileGenerator.GetNavigationData();
		ResultTileRefs.Reserve(TileLayers.Num());

		for (int32 LayerIndex = FirstDirtyTileIndex; LayerIndex < TileGenerator.GetDirtyLayersMask().Num(); ++LayerIndex)
		{
			if (TileGenerator.IsLayerChanged(LayerIndex))
			{
				AddGeneratedTileLayer(LayerIndex, TileGenerator, OldLayerTileIdMap, ResultTileRefs);
			}
		}
	}

	return ResultTileRefs;
}

void FRecastNavMeshGenerator::DiscardCurrentBuildingTasks()
{
	PendingDirtyTiles.Empty();
	
	for (TRunningTileElement<FRecastTileGeneratorWrapper>& Element : RunningDirtyTiles)
	{
		if (Element.AsyncTask)
		{
			Element.AsyncTask->EnsureCompletion();
			delete Element.AsyncTask;
			Element.AsyncTask = nullptr;
		}
	}

	ResetTimeSlicedTileGeneratorSync();

	RunningDirtyTiles.Empty();
}

bool FRecastNavMeshGenerator::HasDirtyTiles() const
{
	return (PendingDirtyTiles.Num() > 0 
		|| RunningDirtyTiles.Num() > 0
		|| SyncTimeSlicedData.TileGeneratorSync.IsValid()
		);
}

FBox FRecastNavMeshGenerator::GrowBoundingBox(const FBox& BBox, bool bIncludeAgentHeight) const
{
	const FVector BBoxGrowOffsetMin = FVector(0, 0, bIncludeAgentHeight ? Config.AgentHeight : 0.0f);

	return FBox(BBox.Min - BBoxGrowthLow - BBoxGrowOffsetMin, BBox.Max + BBoxGrowthHigh);
}

FBox FRecastNavMeshGenerator::GrowDirtyBounds(const FBox& BBox, bool bIncludeAgentHeight) const
{
	const FVector BBoxGrowOffsetMin = FVector(0, 0, bIncludeAgentHeight ? Config.AgentHeight : 0.0f);

	return FBox(BBox.Min - BBoxGrowthHigh - BBoxGrowOffsetMin, BBox.Max + BBoxGrowthLow);
}

bool FRecastNavMeshGenerator::ShouldGenerateGeometryForOctreeElement(const FNavigationOctreeElement& Element, const FNavDataConfig& NavDataConfig) const
{
	return Element.ShouldUseGeometry(NavDataConfig);
}

static bool IntersectBounds(const FBox& TestBox, const TNavStatArray<FBox>& Bounds)
{
	for (const FBox& Box : Bounds)
	{
		if (Box.Intersect(TestBox))
		{
			return true;
		}
	}

	return false;
}

namespace 
{
	FBox CalculateBoxIntersection(const FBox& BoxA, const FBox& BoxB)
	{
		// assumes boxes overlap
		ensure(BoxA.Intersect(BoxB));
		return FBox(FVector(FMath::Max(BoxA.Min.X, BoxB.Min.X)
							, FMath::Max(BoxA.Min.Y, BoxB.Min.Y)
							, FMath::Max(BoxA.Min.Z, BoxB.Min.Z))
					, FVector(FMath::Min(BoxA.Max.X, BoxB.Max.X)
							, FMath::Min(BoxA.Max.Y, BoxB.Max.Y)
							, FMath::Min(BoxA.Max.Z, BoxB.Max.Z))
					);
	}
}

bool FRecastNavMeshGenerator::HasDirtyTiles(const FBox& AreaBounds) const
{
	if (!ensureMsgf(AreaBounds.IsValid, TEXT("%hs AreaBounds is not valid"), __FUNCTION__))
	{
		return false;
	}

	if (HasDirtyTiles() == false)
	{
		return false;
	}

	bool bRetDirty = false;
	const FVector::FReal TileSizeInWorldUnits = Config.GetTileSizeUU();
	const FRcTileBox TileBox(AreaBounds, RcNavMeshOrigin, TileSizeInWorldUnits);
		
	for (int32 Index = 0; bRetDirty == false && Index < PendingDirtyTiles.Num(); ++Index)
	{
		bRetDirty = TileBox.Contains(PendingDirtyTiles[Index].Coord);
	}
	for (int32 Index = 0; bRetDirty == false && Index < RunningDirtyTiles.Num(); ++Index)
	{
		bRetDirty = TileBox.Contains(RunningDirtyTiles[Index].Coord);
	}

	return bRetDirty;
}

int32 FRecastNavMeshGenerator::GetDirtyTilesCount(const FBox& AreaBounds) const
{
	if (!ensureMsgf(AreaBounds.IsValid, TEXT("%hs AreaBounds is not valid"), __FUNCTION__))
	{
		return 0;
	}

	const FVector::FReal TileSizeInWorldUnits = Config.GetTileSizeUU();
	const FRcTileBox TileBox(AreaBounds, RcNavMeshOrigin, TileSizeInWorldUnits);

	int32 DirtyPendingCount = 0;
	for (const FPendingTileElement& PendingElement : PendingDirtyTiles)
	{
		DirtyPendingCount += TileBox.Contains(PendingElement.Coord) ? 1 : 0;
	}

	int32 RunningCount = 0;
	for (const TRunningTileElement<FRecastTileGeneratorWrapper>& RunningElement : RunningDirtyTiles)
	{
		RunningCount += TileBox.Contains(RunningElement.Coord) ? 1 : 0;
	}

	return DirtyPendingCount + RunningCount;
}

bool FRecastNavMeshGenerator::MarkNavBoundsDirty()
{
	// if rebuilding all no point in keeping "old" invalidated areas
	TArray<FNavigationDirtyArea> DirtyAreas;
	for (FBox AreaBounds : InclusionBounds)
	{
		FNavigationDirtyArea DirtyArea(AreaBounds, ENavigationDirtyFlag::All | ENavigationDirtyFlag::NavigationBounds);
		DirtyAreas.Add(DirtyArea);
	}

	if (DirtyAreas.Num())
	{
		MarkDirtyTiles(DirtyAreas);
		return true;
	}
	return false;
}

void FRecastNavMeshGenerator::MarkDirtyTiles(const TArray<FNavigationDirtyArea>& DirtyAreas)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_MarkDirtyTiles);
	
	check(bInitialized);
	const FVector::FReal TileSizeInWorldUnits = Config.GetTileSizeUU();
	check(TileSizeInWorldUnits > 0);

	const bool bGeometryRebuildDisabled = IsGeometryRebuildDisabled(DestNavMesh);

	const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	const ARecastNavMesh* const OwnerNav = GetOwner();
	const bool bUseVirtualGeometryFilteringAndDirtying = OwnerNav != nullptr && OwnerNav->bUseVirtualGeometryFilteringAndDirtying;
	// Those are set only if bUseVirtualGeometryFilteringAndDirtying is enabled since we do not use them for anything else
	const FNavigationOctree* const NavOctreeInstance = (bUseVirtualGeometryFilteringAndDirtying && NavSys) ? NavSys->GetNavOctree() : nullptr;
	const FNavDataConfig* const NavDataConfig = bUseVirtualGeometryFilteringAndDirtying ? &DestNavMesh->GetConfig() : nullptr;
	// ~

	const double CurrentTimeSeconds = FPlatformTime::Seconds();
		
	// find all tiles that need regeneration
	TSet<FPendingTileElement> DirtyTiles;
#if !UE_BUILD_SHIPPING
	// Used for debug purposes to track the number of new dirty tiles per area; Updated only if LogNavigationDirtyArea is VeryVerbose
	TMap<FPendingTileElement, TArray<FNavigationDirtyAreaPerTileDebugInformation>> DirtyAreasDebugging;
#endif

	if (!bRestrictBuildingToActiveTiles || !ActiveTileSet.IsEmpty())
	{
		for (const FNavigationDirtyArea& DirtyArea : DirtyAreas)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_DirtyArea);

			if (!ensureMsgf(DirtyArea.Bounds.IsValid && !DirtyArea.Bounds.ContainsNaN(), TEXT("%hs Attempting to use DirtyArea.Bounds which are not valid%s. Bounds: %s, Source: %s"),
					__FUNCTION__, DirtyArea.Bounds.ContainsNaN() ? TEXT(" (contains NaN)") : TEXT(""), *DirtyArea.Bounds.ToString(), *DirtyArea.GetSourceDescription()))
			{
				continue;
			}

			// Navmeshes with geometry rebuild disabled accept only area modifier updates
			if (bGeometryRebuildDisabled && (!DirtyArea.HasFlag(ENavigationDirtyFlag::DynamicModifier) || DirtyArea.HasFlag(ENavigationDirtyFlag::NavigationBounds)))
			{
				continue;
			}

			UE_VLOG_BOX(OwnerNav, LogNavigation, VeryVerbose, DirtyArea.Bounds, FColor::Blue, TEXT("DirtyArea %s"), *DirtyArea.GetSourceDescription());

			// (if bUseVirtualGeometryFilteringAndDirtying is true) Ignore dirty areas flagged by a source object that is not supposed to apply to this navmesh
			if (bUseVirtualGeometryFilteringAndDirtying && NavSys && NavOctreeInstance && NavDataConfig)
			{
				const FNavigationElement* SourceElement = DirtyArea.OptionalSourceElement.Get();
				if (SourceElement
					&& !ShouldDirtyTilesRequestedByElement(*NavSys, *NavOctreeInstance, SourceElement->GetHandle(), *NavDataConfig))
				{
					continue;
				}
			}
		
			bool bDoTileInclusionTest = false;
			FBox AdjustedAreaBounds = DirtyArea.Bounds;
		
			// if it's not expanding the navigable area
			if (DirtyArea.HasFlag(ENavigationDirtyFlag::NavigationBounds) == false)
			{
				// and is outside of current bounds
				if (GetTotalBounds().Intersect(DirtyArea.Bounds) == false)
				{
					// skip it
					continue;
				}

				const FBox CutDownArea = CalculateBoxIntersection(GetTotalBounds(), DirtyArea.Bounds);
				AdjustedAreaBounds = GrowDirtyBounds(CutDownArea, DirtyArea.HasFlag(ENavigationDirtyFlag::UseAgentHeight));

				// @TODO this and the following test share some work in common
				if (IntersectBounds(AdjustedAreaBounds, InclusionBounds) == false)
				{
					continue;
				}

				// check if any of inclusion volumes encapsulates this box
				// using CutDownArea not AdjustedAreaBounds since if the area is on the border of navigable space
				// then FindInclusionBoundEncapsulatingBox can produce false negative
				bDoTileInclusionTest = (FindInclusionBoundEncapsulatingBox(CutDownArea) == INDEX_NONE);
			}

			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_CheckTilesInBounds);

				uint32 PendingTilesMarked = 0;

				const FRcTileBox TileBox(AdjustedAreaBounds, RcNavMeshOrigin, TileSizeInWorldUnits);

				for (int32 TileY = TileBox.YMin; TileY <= TileBox.YMax; ++TileY)
				{
					for (int32 TileX = TileBox.XMin; TileX <= TileBox.XMax; ++TileX)
					{
						if (IsInActiveSet(FIntPoint(TileX, TileY)) == false)
						{
							UE_SUPPRESS(LogNavigation, VeryVerbose,
							{
								const FBox TileBounds = FRecastTileGenerator::CalculateTileBounds(TileX, TileY, RcNavMeshOrigin, TotalNavBounds, TileSizeInWorldUnits);
								UE_VLOG_BOX(OwnerNav, LogNavigation, VeryVerbose, TileBounds, FColor::Red, TEXT("Not in active set"));
							});
							continue;
						}

						if (bDoTileInclusionTest == true && DirtyArea.HasFlag(ENavigationDirtyFlag::NavigationBounds) == false)
						{
							const FBox TileBounds = FRecastTileGenerator::CalculateTileBounds(TileX, TileY, RcNavMeshOrigin, TotalNavBounds, TileSizeInWorldUnits);

							// do per tile check since we can have lots of tiles in between navigable bounds volumes
							if (IntersectBounds(TileBounds, InclusionBounds) == false)
							{
								// Skip this tile
								continue;
							}
						}
											
						FPendingTileElement Element;
						Element.Coord = FIntPoint(TileX, TileY);
						// Make sure to prevent bRebuildGeometry when geometry rebuild is disabled.
						// Navmeshes with geometry rebuild disabled accept only area modifier updates. Rebuilding geometry would bRegenerateCompressedLayers without having the geometry for them.
						Element.bRebuildGeometry = !bGeometryRebuildDisabled && (DirtyArea.HasFlag(ENavigationDirtyFlag::Geometry) || DirtyArea.HasFlag(ENavigationDirtyFlag::NavigationBounds));
						Element.CreationTime = CurrentTimeSeconds;
						if (Element.bRebuildGeometry == false)
						{
							Element.DirtyAreas.Add(AdjustedAreaBounds);
						}
						PendingTilesMarked++;
			
						FPendingTileElement* ExistingElement = DirtyTiles.Find(Element);
						if (ExistingElement)
						{
							ExistingElement->bRebuildGeometry |= Element.bRebuildGeometry;
							// Append area bounds to existing list 
							if (ExistingElement->bRebuildGeometry == false)
							{
								ExistingElement->DirtyAreas.Append(Element.DirtyAreas);
							}
							else
							{
								ExistingElement->DirtyAreas.Empty();
							}
						}
						else
						{
							DirtyTiles.Add(Element);
						}
			
#if !UE_BUILD_SHIPPING
						UE_SUPPRESS(LogNavigationDirtyArea, VeryVerbose, 
						{
							const bool bAlreadyAdded = ExistingElement != nullptr;
							DirtyAreasDebugging.FindOrAdd(Element).Add({DirtyArea, bAlreadyAdded});
						});
#endif
					}
				}

#if !UE_BUILD_SHIPPING
				// Warn if this is from a big dirty area
				UE_SUPPRESS(LogNavigationDirtyArea, Warning,
				{
					if (PendingTilesMarked > 0)
					{
						if (NavSys == nullptr)
						{
							// NavSys might not have been initialized yet if not using bUseVirtualGeometryFilteringAndDirtying
							NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
						}
			
						// If not using active tile generation, those are reported earlier in FNavigationDirtyAreasController::AddArea
						if (NavSys && NavSys->GetOperationMode() == FNavigationSystemRunMode::GameMode && NavSys->IsActiveTilesGenerationEnabled() &&
							NavSys->GetDirtyAreaWarningSizeThreshold() > 0.f && AdjustedAreaBounds.GetSize().GetMax() > NavSys->GetDirtyAreaWarningSizeThreshold())
						{
							const FVector2D AdjustedAreaBoundsSize(AdjustedAreaBounds.GetSize());

							UE_LOGF(LogNavigationDirtyArea, Warning,
								"(navmesh: %-30ls) Added an oversized dirty area | Tiles marked: %2u | Source Element = %ls | Bounds size = %ls | Threshold: %.0f",
								*GetNameSafe(GetOwner()), PendingTilesMarked, *DirtyArea.GetSourceDescription(),
								*AdjustedAreaBoundsSize.ToString(), NavSys->GetDirtyAreaWarningSizeThreshold());
						}
					}
				});
#endif // !UE_BUILD_SHIPPING

			} // QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_DirtyArea);
		}
	}
	
	int32 NumTilesMarked = DirtyTiles.Num();

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_Merging);
		
		// Merge new dirty tiles info with existing pending elements
		for (FPendingTileElement& ExistingElement : PendingDirtyTiles)
		{
			FSetElementId Id = DirtyTiles.FindId(ExistingElement);
			if (Id.IsValidId())
			{
				const FPendingTileElement& DirtyElement = DirtyTiles[Id];

				ExistingElement.bRebuildGeometry |= DirtyElement.bRebuildGeometry;
				ExistingElement.CreationTime = FMath::Min(DirtyElement.CreationTime, ExistingElement.CreationTime);
				// Append area bounds to existing list 
				if (ExistingElement.bRebuildGeometry == false)
				{
					ExistingElement.DirtyAreas.Append(DirtyElement.DirtyAreas);
				}
				else
				{
					ExistingElement.DirtyAreas.Empty();
				}
				DirtyTiles.Remove(Id);

#if !UE_BUILD_SHIPPING
				UE_SUPPRESS(LogNavigationDirtyArea, VeryVerbose, 
				{
					// Flag everything in the map to set a true boolean (since it was already inserted)
					for (FNavigationDirtyAreaPerTileDebugInformation& Pair : DirtyAreasDebugging.FindChecked(ExistingElement))
					{
						Pair.bTileWasAlreadyAdded = true;
					}
				});
#endif
			}
		}
	}

#if !UE_BUILD_SHIPPING
	UE_SUPPRESS(LogNavigationDirtyArea, VeryVerbose, 
	{
		if (OwnerNav)
		{
			LogDirtyAreas(*OwnerNav, DirtyAreasDebugging);
		}
	});
#endif

	// Append remaining new dirty tile elements
	PendingDirtyTiles.Reserve(PendingDirtyTiles.Num() + DirtyTiles.Num());
	for(const FPendingTileElement& Element : DirtyTiles)
	{
		PendingDirtyTiles.Add(Element);
	}

	// Sort tiles by proximity to players 
	if (NumTilesMarked > 0)
	{
		SortPendingBuildTiles();
	}
}

// Deprecated
bool FRecastNavMeshGenerator::ShouldDirtyTilesRequestedByObject(const UNavigationSystemV1& NavSys,
	const FNavigationOctree& NavOctreeInstance, const UObject& SourceObject, const FNavDataConfig& NavDataConfig) const
{
	return ShouldDirtyTilesRequestedByElement(NavSys, NavOctreeInstance, FNavigationElementHandle(&SourceObject), NavDataConfig); 
}

bool FRecastNavMeshGenerator::ShouldDirtyTilesRequestedByElement(
	const UNavigationSystemV1& NavSys,
	const FNavigationOctree& NavOctreeInstance,
	const FNavigationElementHandle SourceElement,
	const FNavDataConfig& NavDataConfig) const
{
	const FOctreeElementId2* const OctreeElementId = NavSys.GetNavOctreeIdForElement(SourceElement);

	return (OctreeElementId == nullptr) || ShouldGenerateGeometryForOctreeElement(NavOctreeInstance.GetElementById(*OctreeElementId), NavDataConfig);
}

void FRecastNavMeshGenerator::SortPendingBuildTiles()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_SortPendingBuildTiles);
	
	if (SortPendingTilesMethod == ENavigationSortPendingTilesMethod::SortWithSeedLocations)
	{
		UWorld* CurWorld = GetWorld();
		if (CurWorld == nullptr)
		{
			return;
		}

		TArray<FVector2D> SeedLocations;
		GetSeedLocations(*CurWorld, SeedLocations);

		if (SeedLocations.Num() == 0)
		{
			// Use navmesh origin for sorting
			SeedLocations.Add(FVector2D(TotalNavBounds.GetCenter()));
		}

		if (SeedLocations.Num() > 0)
		{
			const FVector::FReal TileSizeInWorldUnits = Config.GetTileSizeUU();
		
			// Calculate shortest distances between tiles and players
			for (FPendingTileElement& Element : PendingDirtyTiles)
			{
				const FBox TileBox = FRecastTileGenerator::CalculateTileBounds(Element.Coord.X, Element.Coord.Y, FVector::ZeroVector, TotalNavBounds, TileSizeInWorldUnits);
				FVector2D TileCenter2D = FVector2D(TileBox.GetCenter());
				for (FVector2D SeedLocation : SeedLocations)
				{
					Element.SeedDistance = FMath::Min(Element.SeedDistance, FVector2D::DistSquared(TileCenter2D, SeedLocation));
				}
			}

			// Nearest tiles should be at the end of the list
			PendingDirtyTiles.Sort();
		}
	}
	else if (SortPendingTilesMethod == ENavigationSortPendingTilesMethod::SortByPriority)
	{
		// Highest priority should be at the end of the list
		PendingDirtyTiles.Sort([](const FPendingTileElement& A, const FPendingTileElement& B){ return A.SortingPriority < B.SortingPriority; });

		UE_SUPPRESS(LogNavigation, VeryVerbose,
		{
			for (int32 Index = 0; Index < PendingDirtyTiles.Num(); Index++)
			{
				const FPendingTileElement& Element = PendingDirtyTiles[Index];
				const FVector::FReal TileSizeInWorldUnits = Config.GetTileSizeUU();
				const FBox TileBox = FRecastTileGenerator::CalculateTileBounds(Element.Coord.X, Element.Coord.Y, FVector::ZeroVector, TotalNavBounds, TileSizeInWorldUnits);
				const ARecastNavMesh* const OwnerNav = GetOwner();
				UE_VLOG_BOX(OwnerNav, LogNavigation, VeryVerbose, TileBox, FColor::Silver, TEXT("Index: %i, Priority %u"), Index, uint32{EnumToUnderlyingType(Element.SortingPriority)});
			}
		});
	}
}

void FRecastNavMeshGenerator::GetSeedLocations(UWorld& World, TArray<FVector2D>& OutSeedLocations) const
{
	// Collect players positions
	for (FConstPlayerControllerIterator PlayerIt = World.GetPlayerControllerIterator(); PlayerIt; ++PlayerIt)
	{
		APlayerController* PC = PlayerIt->Get();
		if (PC && PC->GetPawn() != NULL)
		{
			const FVector2D SeedLoc(PC->GetPawn()->GetActorLocation());
			OutSeedLocations.Add(SeedLoc);
		}
	}
}

TSharedRef<FRecastTileGenerator> FRecastNavMeshGenerator::CreateTileGenerator(const FIntPoint& Coord, const TArray<FBox>& DirtyAreas, const double PendingTileCreationTime /*=0.*/)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_CreateTileGenerator);

	return ConstructTileGeneratorImpl<FRecastTileGenerator>(Coord, DirtyAreas, PendingTileCreationTime);
}

void FRecastNavMeshGenerator::RemoveAllTiles(TArray<FNavTileRef>& UpdatedTiles)
{
	const dtNavMesh* DetourMesh = DestNavMesh->GetRecastNavMeshImpl() ? DestNavMesh->GetRecastNavMeshImpl()->GetRecastMesh() : nullptr;
	if (!DetourMesh)
	{
		return;
	}

	// Remove all tiles that have active data.
	// RemoveLayers is safe to call multiple times on the same coordinate — it checks
	// the layer count before acting, so duplicate coords from multi-layer tiles are harmless.
	for (int32 i = 0; i < DetourMesh->getMaxTiles(); ++i)
	{
		const dtMeshTile* Tile = DetourMesh->getTile(i);
		if (Tile && Tile->header && Tile->dataSize > 0)
		{
			RemoveLayers(FIntPoint(Tile->header->x, Tile->header->y), UpdatedTiles);
		}
	}
}

void FRecastNavMeshGenerator::RemoveLayers(const FIntPoint& Tile, TArray<FNavTileRef>& UpdatedTiles)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RemoveLayers);

	// If there is nothing to generate remove all tiles from navmesh at specified grid coordinates
	UpdatedTiles.Append(
		RemoveTileLayersAndGetUpdatedTiles(Tile.X, Tile.Y)
	);
	DestNavMesh->MarkEmptyTileCacheLayers(Tile.X, Tile.Y);
}

void FRecastNavMeshGenerator::StoreCompressedTileCacheLayers(const FRecastTileGenerator& TileGenerator, int32 TileX, int32 TileY)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_StoringCompressedLayers);

	// Store compressed tile cache layers so it can be reused later
	if (TileGenerator.GetCompressedLayers().Num())
	{
		DestNavMesh->AddTileCacheLayers(TileX, TileY, TileGenerator.GetCompressedLayers());
	}
	else
	{
		DestNavMesh->MarkEmptyTileCacheLayers(TileX, TileY);
	}
}

#if RECAST_INTERNAL_DEBUG_DATA
void FRecastNavMeshGenerator::StoreDebugData(const FRecastTileGenerator& TileGenerator, int32 TileX, int32 TileY)
{
	DestNavMesh->AddTileDebugData(TileX, TileY, TileGenerator.GetDebugData());
}
#endif

#if RECAST_ASYNC_REBUILDING
TArray<FNavTileRef> FRecastNavMeshGenerator::ProcessTileTasksAsyncAndGetUpdatedTiles(const int32 NumTasksToProcess)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_ProcessTileTasksAsync);
	LLM_SCOPE(ELLMTag::NavigationRecast);

	TArray<FNavTileRef> UpdatedTiles;
	const bool bGeometryRebuildDisabled = IsGeometryRebuildDisabled(DestNavMesh);

	int32 NumProcessedTasks = 0;
	// Submit pending tile elements
	for (int32 ElementIdx = PendingDirtyTiles.Num()-1; ElementIdx >= 0 && NumProcessedTasks < NumTasksToProcess; ElementIdx--)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_ProcessTileTasks_NewTasks);

		FPendingTileElement& PendingElement = PendingDirtyTiles[ElementIdx];
		TRunningTileElement<FRecastTileGeneratorWrapper> RunningElement(PendingElement.Coord);
		
		// Make sure that we are not submitting generator for grid cell that is currently being regenerated
		if (!RunningDirtyTiles.Contains(RunningElement))
		{
			// Spawn async task
			TUniquePtr<FAsyncTask<FRecastTileGeneratorWrapper>> TileTask = MakeUnique<FAsyncTask<FRecastTileGeneratorWrapper>>(CreateTileGenerator(PendingElement.Coord, PendingElement.DirtyAreas, PendingElement.CreationTime));

			// Start it in background in case it has something to build
			if (TileTask->GetTask().TileGenerator->HasDataToBuild())
			{
				RunningElement.AsyncTask = TileTask.Release();

				if (!GNavmeshSynchronousTileGeneration)
				{
					RunningElement.AsyncTask->StartBackgroundTask();
				}
				else
				{
					RunningElement.AsyncTask->StartSynchronousTask();
				}

				static int32 Count = 1;
				UE_LOGF(LogNavigationDataBuild, VeryVerbose, "   Tile generation task #%i)", Count);
				Count++;
				
				RunningDirtyTiles.Add(RunningElement);
			}
			else if (!bGeometryRebuildDisabled)
			{
				RemoveLayers(PendingElement.Coord, UpdatedTiles);
			}

			// Remove submitted element from pending list
			PendingDirtyTiles.RemoveAt(ElementIdx, EAllowShrinking::No);
			NumProcessedTasks++;
		}
	}

	// Release memory, list could be quite big after map load
	if (NumProcessedTasks > 0 && PendingDirtyTiles.Num() == 0)
	{
		PendingDirtyTiles.Empty(64);
	}
	
	// Collect completed tasks and apply generated data to navmesh
	for (int32 Idx = RunningDirtyTiles.Num() - 1; Idx >=0; --Idx)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_ProcessTileTasks_FinishedTasks);

		TRunningTileElement<FRecastTileGeneratorWrapper>& Element = RunningDirtyTiles[Idx];
		check(Element.AsyncTask);

		if (Element.AsyncTask->IsDone())
		{
			FRecastTileGenerator& TileGenerator = *(Element.AsyncTask->GetTask().TileGenerator);
			
			// Add generated tiles to navmesh
			if (!Element.bShouldDiscard)
			{
				TArray<FNavTileRef> UpdatedTileRefs = AddGeneratedTilesAndGetUpdatedTiles(TileGenerator);
				UpdatedTiles.Append(UpdatedTileRefs);
			
				StoreCompressedTileCacheLayers(TileGenerator, Element.Coord.X, Element.Coord.Y);

#if RECAST_INTERNAL_DEBUG_DATA
				StoreDebugData(TileGenerator, Element.Coord.X, Element.Coord.Y);
#endif
			}

			TileGenerator.DumpSyncData();

			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_TileGeneratorRemoval);

				// Destroy tile generator task
				delete Element.AsyncTask;
				Element.AsyncTask = nullptr;
				// Remove completed tile element from a list of running tasks
				RunningDirtyTiles.RemoveAtSwap(Idx, EAllowShrinking::No);
			}
		}
	}

	return UpdatedTiles;
}
#endif

#if !RECAST_ASYNC_REBUILDING
TSharedRef<FRecastTileGenerator> FRecastNavMeshGenerator::CreateTileGeneratorFromPendingElement(FIntPoint& OutTileLocation, const int32 ForcedPendingTileIdx)
{
	ensureMsgf(PendingDirtyTiles.Num() > 0, TEXT("Its an assumption of this function that PendingDirtyTiles.Num() > 0"));
	ensureMsgf(ForcedPendingTileIdx == INDEX_NONE || PendingDirtyTiles.IsValidIndex(ForcedPendingTileIdx), TEXT("The pending tile index provided (%d) is invalid. There are %d pending dirty tiles"), ForcedPendingTileIdx, PendingDirtyTiles.Num());

	const int32 PendingItemIdx = ForcedPendingTileIdx == INDEX_NONE ? PendingDirtyTiles.Num() - 1 : ForcedPendingTileIdx; 
	FPendingTileElement& PendingElement = PendingDirtyTiles[PendingItemIdx];

	OutTileLocation.X = PendingElement.Coord.X;
	OutTileLocation.Y = PendingElement.Coord.Y;

	TSharedRef<FRecastTileGenerator> TileGenerator = CreateTileGenerator(PendingElement.Coord, PendingElement.DirtyAreas, PendingElement.CreationTime);

	PendingDirtyTiles.RemoveAt(PendingItemIdx, EAllowShrinking::No);

	return TileGenerator;
}

TArray<FNavTileRef> FRecastNavMeshGenerator::ProcessTileTasksSyncTimeSlicedAndGetUpdatedTiles()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_ProcessTileTasksSyncTimeSliced);
	CSV_SCOPED_TIMING_STAT(NAVREGEN, ProcessTileTasksSyncTimeSliced);
	LLM_SCOPE(ELLMTag::NavigationRecast);

	check(SyncTimeSlicedData.TimeSliceManager);

	TArray<FNavTileRef> UpdatedTiles;
	double TimeStartProcessingTileThisFrame = 0.;

	auto HasWorkToDo = [this](int32& OutNextPendingDirtyTileIndex)
	{
		OutNextPendingDirtyTileIndex = INDEX_NONE;
		if (SyncTimeSlicedData.TileGeneratorSync.IsValid())
		{
			return true;
		}

		OutNextPendingDirtyTileIndex = GetNextPendingDirtyTileToBuild();
		return OutNextPendingDirtyTileIndex != INDEX_NONE;
	};

	auto EndFunction = [&, this](bool bCalcTileRegenDuration, bool bStartedTimeSlice)
	{
		// Release memory, list could be quite big after map load
		if (PendingDirtyTiles.Num() == 0)
		{
			PendingDirtyTiles.Empty(64);
		}

		//this will only be true when we haven't finished generating this tile but are ending
		//the function and need to record the TileRegenDuration so far for the tile
		//being currently processed
		if (bCalcTileRegenDuration)
		{
			SyncTimeSlicedData.CurrentTileRegenDuration += (FPlatformTime::Seconds() - TimeStartProcessingTileThisFrame);
		}

		if (bStartedTimeSlice)
		{
			SyncTimeSlicedData.TimeSliceManager->GetTimeSlicer().EndTimeSliceAndAdjustDuration();
		}

#if ALLOW_TIME_SLICE_DEBUG
		// Reset the debug function to make sure the captured variables can't be used when invalid.
		// This is just bombproofing the code as TestTimeSliceFinished() should not be called until 
		// ProcessTileTasksSyncTimeSlicedAndGetUpdatedTiles() is next called.
		SyncTimeSlicedData.TimeSliceManager->GetTimeSlicer().DebugResetLongTimeSliceFunction();
#endif // ALLOW_TIME_SLICE_DEBUG

		return UpdatedTiles;
	};

#if ALLOW_TIME_SLICE_DEBUG
	auto DebugLongTileRegenFunction = [this](FName DebugSectionName, double Duration)
	{
		const FRecastTileGenerator* const TileGenerator = SyncTimeSlicedData.TileGeneratorSync.Get();

		// This shouldn't trigger but its fairly easy during development to accidentaly call TestTimeSliceFinished() when TileGenerator == nullptr.
		if (ensure(TileGenerator))
		{
			const FVector Pos = SyncTimeSlicedData.TileGeneratorSync->GetTileBB().GetCenter();

			check(DestNavMesh);

			// I'd quite like to make this a Warning, but it would be too frequently logged as things stand.
			UE_LOGF(LogNavigation, Verbose, "Nav mesh data: %ls, tile at %d, %d, coordinate %f, %f, %f, section %ls is taking %f secs to partially regenerate!", *DestNavMesh->GetName(), TileGenerator->GetTileX(), TileGenerator->GetTileY(), Pos.X, Pos.Y, Pos.Z, *DebugSectionName.ToString(), Duration);
			UE_VLOG_BOX(DestNavMesh, LogNavigation, Verbose, TileGenerator->GetTileBB(), FColor::Red, TEXT("Nav mesh data : %s,  tile at %d, %d, section %s is taking %f secs to partially regenerate!"), *DestNavMesh->GetName(), TileGenerator->GetTileX(), TileGenerator->GetTileY(), *DebugSectionName.ToString(), Duration);
		}
	};

	SyncTimeSlicedData.TimeSliceManager->GetTimeSlicer().DebugSetLongTimeSliceData(DebugLongTileRegenFunction, DestNavMesh->TimeSliceLongDurationDebug);
#endif // ALLOW_TIME_SLICE_DEBUG

	int32 NextPendingDirtyTileIndex = INDEX_NONE;
	const bool bHadWorkToDo = HasWorkToDo(NextPendingDirtyTileIndex);

	// Only calculate the time slice and process tiles if we have work to do.
	if (bHadWorkToDo)
	{
		SyncTimeSlicedData.TimeSliceManager->GetTimeSlicer().StartTimeSlice();

		const bool bGeometryRebuildDisabled = IsGeometryRebuildDisabled(DestNavMesh);

		// Submit pending tile elements
		do
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_ProcessTileTasks_NewTasks);

			FIntPoint TileLocation;
			TimeStartProcessingTileThisFrame = FPlatformTime::Seconds();

			if (SyncTimeSlicedData.ProcessTileTasksSyncState == EProcessTileTasksSyncTimeSlicedState::Init)
			{
				//if the next time slice regen state is false, we want to go to non time sliced tile regen so break here and switch
				//next frame (as we've finished time slice processing the last tile)
				if (!SyncTimeSlicedData.bNextTimeSliceRegenActive)
				{
					return EndFunction(false /* bCalcTileRegenDuration */, bHadWorkToDo);
				}

				SyncTimeSlicedData.TileGeneratorSync = CreateTileGeneratorFromPendingElement(TileLocation, NextPendingDirtyTileIndex);

				check(SyncTimeSlicedData.TileGeneratorSync);

				SyncTimeSlicedData.CurrentTileRegenDuration = 0.;

#if !UE_BUILD_SHIPPING				
				SyncTimeSlicedData.TileRegenStartFrame = GFrameCounter;
#endif // !UE_BUILD_SHIPPING 

				if (SyncTimeSlicedData.TileGeneratorSync->HasDataToBuild())
				{
					SyncTimeSlicedData.ProcessTileTasksSyncState = EProcessTileTasksSyncTimeSlicedState::DoWork;
				}
				else
				{
					SyncTimeSlicedData.ProcessTileTasksSyncState = EProcessTileTasksSyncTimeSlicedState::Finish;

					if (!bGeometryRebuildDisabled)
					{
						RemoveLayers(TileLocation, UpdatedTiles);
					}
				}

				MARK_TIMESLICE_SECTION_DEBUG(SyncTimeSlicedData.TimeSliceManager->GetTimeSlicer(), CreateTileGenerator);

				if (SyncTimeSlicedData.TimeSliceManager->GetTimeSlicer().TestTimeSliceFinished())
				{
					return EndFunction(true /* bCalcTileRegenDuration */, bHadWorkToDo);
				}
			}
			else
			{
				check(SyncTimeSlicedData.TileGeneratorSync);

				TileLocation.X = SyncTimeSlicedData.TileGeneratorSync->GetTileX();
				TileLocation.Y = SyncTimeSlicedData.TileGeneratorSync->GetTileY();
			}

			FRecastTileGenerator& TileGeneratorRef = *SyncTimeSlicedData.TileGeneratorSync;

			switch (SyncTimeSlicedData.ProcessTileTasksSyncState)
			{
			case EProcessTileTasksSyncTimeSlicedState::Init:
			{
				//do nothing 
				ensureMsgf(false, TEXT("This State should not be used here!"));
			}
			break;

			case EProcessTileTasksSyncTimeSlicedState::DoWork:
			{
				const ETimeSliceWorkResult WorkResult = TileGeneratorRef.DoWorkTimeSliced();

				if (WorkResult != ETimeSliceWorkResult::CallAgainNextTimeSlice)
				{
					SyncTimeSlicedData.ProcessTileTasksSyncState = EProcessTileTasksSyncTimeSlicedState::AddGeneratedTiles;
				}

				if (SyncTimeSlicedData.TimeSliceManager->GetTimeSlicer().IsTimeSliceFinishedCached())
				{
					return EndFunction(true /* bCalcTileRegenDuration */, bHadWorkToDo);
				}
			}//fall through to next state
			case EProcessTileTasksSyncTimeSlicedState::AddGeneratedTiles:
			{
				const ETimeSliceWorkResult WorkResult = AddGeneratedTilesTimeSliced(TileGeneratorRef, SyncTimeSlicedData.UpdatedTilesCache);

				if (WorkResult != ETimeSliceWorkResult::CallAgainNextTimeSlice)
				{
					SyncTimeSlicedData.ProcessTileTasksSyncState = EProcessTileTasksSyncTimeSlicedState::StoreCompessedTileCacheLayers;
				}
				 
				if (SyncTimeSlicedData.TimeSliceManager->GetTimeSlicer().IsTimeSliceFinishedCached())
				{
					return EndFunction(true /* bCalcTileRegenDuration */, bHadWorkToDo);
				}
			}//fall through to next state
			case EProcessTileTasksSyncTimeSlicedState::StoreCompessedTileCacheLayers:
			{
				StoreCompressedTileCacheLayers(TileGeneratorRef, TileLocation.X, TileLocation.Y);

				//no need to check time slicing as not much work done
				SyncTimeSlicedData.ProcessTileTasksSyncState = EProcessTileTasksSyncTimeSlicedState::AppendUpdateTiles;
			}//fall through to next state
			case EProcessTileTasksSyncTimeSlicedState::AppendUpdateTiles: //this state was added purely to separate the functionality and allow the code to be more easily changed in future.
			{
				UpdatedTiles.Append(SyncTimeSlicedData.UpdatedTilesCache);
				SyncTimeSlicedData.UpdatedTilesCache.Empty();

				//no need to check time slicing as not much work done
				SyncTimeSlicedData.ProcessTileTasksSyncState = EProcessTileTasksSyncTimeSlicedState::Finish;
			}//fall through to next state
			case EProcessTileTasksSyncTimeSlicedState::Finish:
			{
				//reset state to Init for next tile to be processed
				SyncTimeSlicedData.ProcessTileTasksSyncState = EProcessTileTasksSyncTimeSlicedState::Init;

				SyncTimeSlicedData.CurrentTileRegenDuration += (FPlatformTime::Seconds() - TimeStartProcessingTileThisFrame);
#if !UE_BUILD_SHIPPING					
				const double TempRegenDuration = SyncTimeSlicedData.CurrentTileRegenDuration;
#endif					

				SyncTimeSlicedData.TimeSliceManager->PushTileRegenTime(SyncTimeSlicedData.CurrentTileRegenDuration);

				SyncTimeSlicedData.CurrentTileRegenDuration = 0.;

#if !UE_BUILD_SHIPPING					
				SyncTimeSlicedData.TileRegenEndFrame = GFrameCounter;
					
				const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
				if (NavSys)
				{
					const TSharedPtr<FRecastTileGenerator>& TileGenerator = SyncTimeSlicedData.TileGeneratorSync;
					const double WaitTime = (FPlatformTime::Seconds() - TileGenerator->TileCreationTime);
					const int32 NavDataIndex = NavSys->NavDataSet.Find(DestNavMesh);	
					SyncTimeSlicedData.TimeSliceManager->PushTileWaitTime(NavDataIndex, WaitTime);

					UE_SUPPRESS(LogNavigationHistory, Log,
					{
						FTileHistoryData HistoryData;
						HistoryData.TileX = TileGenerator->TileX;
						HistoryData.TileY = TileGenerator->TileY;
						HistoryData.TileRegenTime = (float)TempRegenDuration;
						HistoryData.TileWaitTime = (float)WaitTime;
						HistoryData.StartRegenFrame = SyncTimeSlicedData.TileRegenStartFrame;
						HistoryData.EndRegenFrame = SyncTimeSlicedData.TileRegenEndFrame;
						SyncTimeSlicedData.TimeSliceManager->PushTileHistoryData(NavDataIndex, HistoryData);
					});
				}
#endif // !UE_BUILD_SHIPPING					
				
				MARK_TIMESLICE_SECTION_DEBUG(SyncTimeSlicedData.TimeSliceManager->GetTimeSlicer(), FinishTile);

				//test time slice 
				const bool bTimeSliceFinished = SyncTimeSlicedData.TimeSliceManager->GetTimeSlicer().TestTimeSliceFinished();

				//reset TileGeneratorSync after the last call to TestTimeSliceFinished for this tile, otherwise we may end up
				//trying to access TileGeneratorSync from DebugLongTileRegenFunction()
				SyncTimeSlicedData.TileGeneratorSync.Reset();

				if (bTimeSliceFinished)
				{
					//we just calculated and set TileRegenDuration so no need to calculate it again
					return EndFunction(false /* bCalcTileRegenDuration */, bHadWorkToDo);
				}
			}
			break;
			default:
			{
				ensureMsgf(false, TEXT("unhandled EProcessTileTasksSyncTimeSlicedState"));
			}
			}
		}
		while (HasWorkToDo(NextPendingDirtyTileIndex));
	}

	// we only hit this if we have processed too many tiles in a frame and we will already
	// have calculated the tile regen duration, or if we have processed no tiles and we also
	// don't want to calculate the tile regen duration
	return EndFunction(false /* bCalcTileRegenDuration */, bHadWorkToDo);
}

int32 FRecastNavMeshGenerator::GetNextPendingDirtyTileToBuild() const
{
	return PendingDirtyTiles.IsEmpty() ? INDEX_NONE : PendingDirtyTiles.Num() - 1;
}
	
//this code path is approx 10% faster than ProcessTileTasksSyncTimeSliced, however it spikes far worse for most use cases.
TArray<FNavTileRef> FRecastNavMeshGenerator::ProcessTileTasksSyncAndGetUpdatedTiles(const int32 NumTasksToProcess)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_ProcessTileTasksSync);

	const bool bGeometryRebuildDisabled = IsGeometryRebuildDisabled(DestNavMesh);
	int32 NumProcessedTasks = 0;
	TArray<FNavTileRef> UpdatedTiles;
	FIntPoint TileLocation;

	// Submit pending tile elements
	while ((PendingDirtyTiles.Num() > 0 && NumProcessedTasks < NumTasksToProcess))
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_ProcessTileTasks_NewTasks);

		TSharedRef<FRecastTileGenerator> TileGenerator = CreateTileGeneratorFromPendingElement(TileLocation);
		
		FRecastTileGenerator& TileGeneratorRef = *TileGenerator;

		//Does this remain true whenever we stop time slicing?
		if (TileGeneratorRef.HasDataToBuild())
		{
			TileGeneratorRef.DoWork();

			UpdatedTiles = AddGeneratedTilesAndGetUpdatedTiles(TileGeneratorRef);

			StoreCompressedTileCacheLayers(TileGeneratorRef, TileLocation.X, TileLocation.Y);
		}
		else if (!bGeometryRebuildDisabled)
		{
			RemoveLayers(TileLocation, UpdatedTiles);
		}

		NumProcessedTasks++;
	}

	// Release memory, list could be quite big after map load
	if (PendingDirtyTiles.Num() == 0)
	{
		PendingDirtyTiles.Empty(64);
	}

	return UpdatedTiles;
}
#endif

TArray<FNavTileRef> FRecastNavMeshGenerator::ProcessTileTasksAndGetUpdatedTiles(const int32 NumTasksToProcess)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_ProcessTileTasks);

	const bool bHasTasksAtStart = GetNumRemaningBuildTasks() > 0;
	TArray<FNavTileRef> UpdatedTiles;

#if RECAST_ASYNC_REBUILDING
	UpdatedTiles = ProcessTileTasksAsyncAndGetUpdatedTiles(NumTasksToProcess);
#else
	if (SyncTimeSlicedData.TimeSliceManager)
	{
		//only switch bTimeSliceRegen state if we are not time slicing or if we are but aren't part way through time slicing a tile
		if (SyncTimeSlicedData.bTimeSliceRegenActive != SyncTimeSlicedData.bNextTimeSliceRegenActive)
		{
			if (!SyncTimeSlicedData.bTimeSliceRegenActive)
			{
				SyncTimeSlicedData.bTimeSliceRegenActive = SyncTimeSlicedData.bNextTimeSliceRegenActive;
			}
			else if (!SyncTimeSlicedData.TileGeneratorSync.IsValid())//test if we have finished processing a tile
			{
				SyncTimeSlicedData.bTimeSliceRegenActive = SyncTimeSlicedData.bNextTimeSliceRegenActive;
			}
		}	
	}
	else
	{
		//No time slice manager no timesliced regen
		SyncTimeSlicedData.bTimeSliceRegenActive = false;
	}

	if (SyncTimeSlicedData.bTimeSliceRegenActive)
	{
		UpdatedTiles = ProcessTileTasksSyncTimeSlicedAndGetUpdatedTiles();
	}
	else
	{
		UpdatedTiles = ProcessTileTasksSyncAndGetUpdatedTiles(NumTasksToProcess);
	}
#endif

#if WITH_NAVMESH_SEGMENT_LINKS
	const dtNavMesh* DetourMesh = DestNavMesh->GetRecastNavMeshImpl()->GetRecastMesh();
	if (DetourMesh)
	{
		TilesPendingSegmentLinkConnections.Reserve(TilesPendingSegmentLinkConnections.Num() + UpdatedTiles.Num());
		for (FNavTileRef TileRef : UpdatedTiles)
		{
			TilesPendingSegmentLinkConnections.Add(TileRef);
		}
	}
#endif

	// Notify owner in case all tasks has been completed
	const bool bHasTasksAtEnd = GetNumRemaningBuildTasks() > 0;
	if (bHasTasksAtStart && !bHasTasksAtEnd)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_OnNavMeshGenerationFinished);

		if (RebuildAllStartTime != 0)
		{
			UE_LOGF(LogNavigationDataBuild, Display, "   %s build time: %.2fs", __FUNCTION__, (FPlatformTime::Seconds() - RebuildAllStartTime));
			RebuildAllStartTime = 0;
		}

#if WITH_NAVMESH_SEGMENT_LINKS
		DestNavMesh->CreateSegmentLinkConnections(TilesPendingSegmentLinkConnections);
		TilesPendingSegmentLinkConnections.Empty();
#endif

		DestNavMesh->OnNavMeshGenerationFinished();
	}

#if !UE_BUILD_SHIPPING && OUTPUT_NAV_TILE_LAYER_COMPRESSION_DATA && FRAMEPRO_ENABLED
	//only do this if framepro is recording as its an expensive operation
	if (FFrameProProfiler::IsFrameProRecording())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_GetCompressedTileCacheSize);

		int32 TileCacheSize = DestNavMesh->GetCompressedTileCacheSize();

		FPlatformMisc::CustomNamedStat("TotalTileCacheSize", static_cast<float>(TileCacheSize), "NavMesh", "Bytes");
	}
#endif
	return UpdatedTiles;
}

#if UE_ENABLE_DEBUG_DRAWING
void FRecastNavMeshGenerator::GetDebugGeometry(const FNavigationRelevantData& EncodedData, FNavDebugMeshData& DebugMeshData)
{
	const uint8* RawMemory = EncodedData.CollisionData.GetData();
	if (RawMemory == nullptr)
	{
		return;
	}
	const FRecastGeometryCache::FHeader* HeaderInfo = reinterpret_cast<const FRecastGeometryCache::FHeader*>(RawMemory);
	if (HeaderInfo->NumVerts == 0 || HeaderInfo->NumFaces == 0)
	{
		return;
	}
	
	const int32 HeaderSize = sizeof(FRecastGeometryCache);
	const int32 IndicesCount = HeaderInfo->NumFaces * 3;
		
	DebugMeshData.Vertices.AddZeroed(HeaderInfo->NumVerts);
	FDynamicMeshVertex* DebugVert = DebugMeshData.Vertices.GetData();
	// we cannot copy verts directly since not only are the EncodedData's verts in
	// a FVector::FReal[3] format, they're also in Recast coords so we need to translate it 
	// back to Unreal coords
	const FVector::FReal* VertCoord = reinterpret_cast<const FVector::FReal*>(RawMemory + HeaderSize);
	for (int VertIndex = 0; VertIndex < HeaderInfo->NumVerts; ++VertIndex, ++DebugVert, VertCoord += 3)
	{
		new (DebugVert) FDynamicMeshVertex((FVector3f)Recast2UnrealPoint(VertCoord));
	}

	DebugMeshData.Indices.AddZeroed(IndicesCount);
	FMemory::Memcpy(DebugMeshData.Indices.GetData(), RawMemory + HeaderSize + HeaderInfo->NumVerts * 3 * sizeof(FVector::FReal), IndicesCount * sizeof(int32));
}
#endif // !UE_BUILD_SHIPPING

void FRecastGeometryExport::ExportElementGeometry(const FNavigationElement& InElement, FNavigationRelevantData& OutData)
{
	FRecastGeometryExport GeomExport(OutData);
	RecastGeometryExport::ExportObject(InElement, GeomExport);

#if !UE_BUILD_SHIPPING
	RecastGeometryExport::ValidateGeometryExport(GeomExport);
#endif
	
	RecastGeometryExport::ConvertCoordDataToRecast(GeomExport.VertexBuffer);
	RecastGeometryExport::StoreCollisionCache(GeomExport);
}

void FRecastGeometryExport::ExportVertexSoupGeometry(const TArray<FVector>& InVerts, FNavigationRelevantData& OutData)
{
	FRecastGeometryExport GeomExport(OutData);
	RecastGeometryExport::ExportVertexSoup(InVerts, GeomExport.VertexBuffer, GeomExport.IndexBuffer, GeomExport.Data->Bounds);

#if !UE_BUILD_SHIPPING
	RecastGeometryExport::ValidateGeometryExport(GeomExport);
#endif

	RecastGeometryExport::StoreCollisionCache(GeomExport);
}

void FRecastGeometryExport::ExportRigidBodyGeometry(UBodySetup& InOutBodySetup, TNavStatArray<FVector>& OutVertexBuffer, TNavStatArray<int32>& OutIndexBuffer, FBox& OutBounds, const FTransform& LocalToWorld)
{
	TNavStatArray<FVector::FReal> VertCoords;
	RecastGeometryExport::ExportRigidBodySetup(InOutBodySetup, VertCoords, OutIndexBuffer, OutBounds, LocalToWorld);

	OutVertexBuffer.Reserve(OutVertexBuffer.Num() + (VertCoords.Num() / 3));
	for (int32 i = 0; i < VertCoords.Num(); i += 3)
	{
		OutVertexBuffer.Add(FVector(VertCoords[i + 0], VertCoords[i + 1], VertCoords[i + 2]));
	}
}

void FRecastGeometryExport::ExportRigidBodyGeometry(UBodySetup& InOutBodySetup, TNavStatArray<FVector>& OutTriMeshVertexBuffer, TNavStatArray<int32>& OutTriMeshIndexBuffer, TNavStatArray<FVector>& OutConvexVertexBuffer, TNavStatArray<int32>& OutConvexIndexBuffer, TNavStatArray<int32>& OutShapeBuffer, FBox& OutBounds, const FTransform& LocalToWorld)
{
	InOutBodySetup.CreatePhysicsMeshes();

	TNavStatArray<FVector::FReal> VertCoords;
	RecastGeometryExport::ExportRigidBodyTriMesh(InOutBodySetup, VertCoords, OutTriMeshIndexBuffer, OutBounds, LocalToWorld);

	OutTriMeshVertexBuffer.Reserve(OutTriMeshVertexBuffer.Num() + (VertCoords.Num() / 3));
	for (int32 i = 0; i < VertCoords.Num(); i += 3)
	{
		OutTriMeshVertexBuffer.Add(FVector(VertCoords[i + 0], VertCoords[i + 1], VertCoords[i + 2]));
	}

	const int32 NumExistingVerts = OutConvexVertexBuffer.Num();
	VertCoords.Reset();
	RecastGeometryExport::ExportRigidBodyConvexElements(InOutBodySetup, VertCoords, OutConvexIndexBuffer, OutShapeBuffer, OutBounds, LocalToWorld);
	RecastGeometryExport::ExportRigidBodyBoxElements(InOutBodySetup.AggGeom, VertCoords, OutConvexIndexBuffer, OutShapeBuffer, OutBounds, LocalToWorld, NumExistingVerts);
	RecastGeometryExport::ExportRigidBodySphylElements(InOutBodySetup.AggGeom, VertCoords, OutConvexIndexBuffer, OutShapeBuffer, OutBounds, LocalToWorld, NumExistingVerts);
	RecastGeometryExport::ExportRigidBodySphereElements(InOutBodySetup.AggGeom, VertCoords, OutConvexIndexBuffer, OutShapeBuffer, OutBounds, LocalToWorld, NumExistingVerts);
	
	OutConvexVertexBuffer.Reserve(OutConvexVertexBuffer.Num() + (VertCoords.Num() / 3));
	for (int32 i = 0; i < VertCoords.Num(); i += 3)
	{
		OutConvexVertexBuffer.Add(FVector(VertCoords[i + 0], VertCoords[i + 1], VertCoords[i + 2]));
	}
}

void FRecastGeometryExport::ExportAggregatedGeometry(const FKAggregateGeom& AggGeom, TNavStatArray<FVector>& OutConvexVertexBuffer, TNavStatArray<int32>& OutConvexIndexBuffer, TNavStatArray<int32>& OutShapeBuffer, FBox& OutBounds, const FTransform& LocalToWorld)
{
	TNavStatArray<FVector::FReal> VertCoords;
	const int32 NumExistingVerts = OutConvexVertexBuffer.Num();

	// convex and tri mesh are NOT supported, since they require BodySetup.CreatePhysicsMeshes() call
	// only simple shapes

	RecastGeometryExport::ExportRigidBodyBoxElements(AggGeom, VertCoords, OutConvexIndexBuffer, OutShapeBuffer, OutBounds, LocalToWorld, NumExistingVerts);
	RecastGeometryExport::ExportRigidBodySphylElements(AggGeom, VertCoords, OutConvexIndexBuffer, OutShapeBuffer, OutBounds, LocalToWorld, NumExistingVerts);
	RecastGeometryExport::ExportRigidBodySphereElements(AggGeom, VertCoords, OutConvexIndexBuffer, OutShapeBuffer, OutBounds, LocalToWorld, NumExistingVerts);

	OutConvexVertexBuffer.Reserve(OutConvexVertexBuffer.Num() + (VertCoords.Num() / 3));
	for (int32 i = 0; i < VertCoords.Num(); i += 3)
	{
		OutConvexVertexBuffer.Add(FVector(VertCoords[i + 0], VertCoords[i + 1], VertCoords[i + 2]));
	}
}

bool FRecastNavMeshGenerator::IsBuildInProgressCheckDirty() const
{
	return RunningDirtyTiles.Num()
		|| PendingDirtyTiles.Num()
		|| SyncTimeSlicedData.TileGeneratorSync.IsValid();
}

#if !RECAST_ASYNC_REBUILDING
bool FRecastNavMeshGenerator::GetTimeSliceData(int32& OutNumRemainingBuildTasks, double& OutCurrentBuildTaskDuration) const
{
	//it's probably just faster to calculate these rather than branch and only calculate them if bTimeSliceRegenActive is true;
	OutNumRemainingBuildTasks = GetNumRemaningBuildTasksHelper();
	OutCurrentBuildTaskDuration = SyncTimeSlicedData.CurrentTileRegenDuration;
	return SyncTimeSlicedData.bTimeSliceRegenActive;
}
#endif

int32 FRecastNavMeshGenerator::GetNumRemaningBuildTasks() const
{
	return GetNumRemaningBuildTasksHelper();
}

int32 FRecastNavMeshGenerator::GetNumRunningBuildTasks() const
{
	return RunningDirtyTiles.Num()
		+ (SyncTimeSlicedData.TileGeneratorSync.Get() ? 1 : 0);
}

bool FRecastNavMeshGenerator::GatherGeometryOnGameThread() const 
{ 
	return DestNavMesh == nullptr || DestNavMesh->ShouldGatherDataOnGameThread() == true;
}

bool FRecastNavMeshGenerator::IsTimeSliceRegenActive() const
{
	return SyncTimeSlicedData.bTimeSliceRegenActive;
}

bool FRecastNavMeshGenerator::IsTileChanged(const FNavTileRef InTileRef) const
{
#if WITH_EDITOR	
	// Check recently built tiles
	if (InTileRef.IsValid())
	{
		FTileTimestamp TileTimestamp;
		TileTimestamp.NavTileRef = InTileRef;
		if (RecentlyBuiltTiles.Contains(TileTimestamp))
		{
			return true;
		}
	}
#endif//WITH_EDITOR

	return false;
}

uint32 FRecastNavMeshGenerator::LogMemUsed() const 
{
	UE_LOGF(LogNavigation, Display, "    FRecastNavMeshGenerator: self %llu", sizeof(FRecastNavMeshGenerator));
	
	uint32 GeneratorsMem = 0;
	for (const TRunningTileElement<FRecastTileGeneratorWrapper>& Element : RunningDirtyTiles)
	{
		GeneratorsMem += Element.AsyncTask->GetTask().TileGenerator->UsedMemoryOnStartup;
		if (SyncTimeSlicedData.TileGeneratorSync.IsValid())
		{
			GeneratorsMem += SyncTimeSlicedData.TileGeneratorSync->UsedMemoryOnStartup;
		}
	}

	UE_LOGF(LogNavigation, Display, "    FRecastNavMeshGenerator: Total Generator\'s size %u, count %d", GeneratorsMem, RunningDirtyTiles.Num());

	return GeneratorsMem + sizeof(FRecastNavMeshGenerator) + PendingDirtyTiles.GetAllocatedSize() + RunningDirtyTiles.GetAllocatedSize();
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && ENABLE_VISUAL_LOG
void FRecastNavMeshGenerator::GrabDebugSnapshot(struct FVisualLogEntry* Snapshot, const FBox& BoundingBox, const FName& CategoryName, ELogVerbosity::Type LogVerbosity) const
{
	const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	const FNavigationOctree* NavOctree = NavSys ? NavSys->GetNavOctree() : NULL;
	if (Snapshot == nullptr)
	{
		return;
	}

	if (NavOctree == NULL)
	{
		UE_LOGF(LogNavigation, Error, "Failed to vlog navigation data due to %ls being NULL", NavSys == NULL ? TEXT("NavigationSystem") : TEXT("NavOctree"));
		return;
	}

	ELogVerbosity::Type NavAreaVerbosity = FMath::Clamp(ELogVerbosity::Type(LogVerbosity + 1), ELogVerbosity::NoLogging, ELogVerbosity::VeryVerbose);

	for (int32 Index = 0; Index < NavSys->NavDataSet.Num(); ++Index)
	{
		TArray<FVector> CoordBuffer;
		TArray<int32> Indices;
		TNavStatArray<FVector> Faces;
		const ARecastNavMesh* NavData = Cast<const ARecastNavMesh>(NavSys->NavDataSet[Index]);
		if (NavData)
		{
			const bool bUseVirtualGeometryFilteringAndDirtying = NavData->bUseVirtualGeometryFilteringAndDirtying;
			
			NavOctree->FindElementsWithBoundsTest(BoundingBox,
				[this, NavData, &Indices, &CoordBuffer, Snapshot, &CategoryName, LogVerbosity, NavAreaVerbosity, bUseVirtualGeometryFilteringAndDirtying](const FNavigationOctreeElement& Element)
			{
				const bool bExportGeometry = Element.Data->HasGeometry() && (
					bUseVirtualGeometryFilteringAndDirtying ?
						ShouldGenerateGeometryForOctreeElement(Element, NavData->GetConfig()) :
						Element.ShouldUseGeometry(NavData->GetConfig())
					);

				TArray<FTransform> InstanceTransforms;
				Element.Data->NavDataPerInstanceTransformDelegate.ExecuteIfBound(Element.Bounds.GetBox(), InstanceTransforms);

				if (bExportGeometry && Element.Data->CollisionData.Num())
				{
					FRecastGeometryCache CachedGeometry(Element.Data->CollisionData.GetData());
					
					const uint32 NumIndices = CachedGeometry.Header.NumFaces * 3;
					Indices.SetNum(NumIndices, EAllowShrinking::No);
					for (uint32 IndicesIdx = 0; IndicesIdx < NumIndices; ++IndicesIdx)
					{
						Indices[IndicesIdx] = CachedGeometry.Indices[IndicesIdx];
					}

					auto AddElementFunc = [&](const FTransform& Transform)
					{
						const uint32 NumVerts = CachedGeometry.Header.NumVerts;
						CoordBuffer.Reset(NumVerts);
						for (uint32 VertIdx = 0; VertIdx < NumVerts * 3; VertIdx += 3)
						{
							CoordBuffer.Add(Transform.TransformPosition(Recast2UnrealPoint(&CachedGeometry.Verts[VertIdx])));
						}

						Snapshot->AddMesh(CoordBuffer, Indices, CategoryName, LogVerbosity, FColorList::LightGrey.WithAlpha(255));
					};

					if (InstanceTransforms.Num() == 0)
					{
						AddElementFunc(FTransform::Identity);
					}
					for (const FTransform& InstanceTransform : InstanceTransforms)
					{
						AddElementFunc(InstanceTransform);
					}
				}
				else
				{
					TArray<FVector> Verts;
					for (const FAreaNavModifier& AreaMod : Element.Data->Modifiers.GetAreas())
					{
						ENavigationShapeType::Type ShapeType = AreaMod.GetShapeType();
						if (ShapeType == ENavigationShapeType::Unknown)
						{
							continue;
						}

						const int32 AreaId = NavData->GetAreaID(AreaMod.GetAreaClass());
						const UClass* AreaClass = NavData->GetAreaClass(AreaId);
						const UNavArea* DefArea = AreaClass ? ((UClass*)AreaClass)->GetDefaultObject<UNavArea>() : nullptr;
						const FColor PolygonColor = AreaClass != FNavigationSystem::GetDefaultWalkableArea() ? (DefArea ? DefArea->DrawColor : NavData->GetConfig().Color) : FColorList::Cyan;

						if (ShapeType == ENavigationShapeType::Box)
						{
							FBoxNavAreaData Box;
							AreaMod.GetBox(Box);

							Snapshot->AddBox(FBox::BuildAABB(Box.Origin, Box.Extent), FMatrix::Identity, CategoryName, NavAreaVerbosity, PolygonColor.WithAlpha(255));
						}
						else if (ShapeType == ENavigationShapeType::Cylinder)
						{
							FCylinderNavAreaData Cylinder;
							AreaMod.GetCylinder(Cylinder);

							Snapshot->AddCylinder(Cylinder.Origin, Cylinder.Origin + FVector(0, 0, Cylinder.Height), Cylinder.Radius, CategoryName, NavAreaVerbosity, PolygonColor.WithAlpha(255));
						}
						else if (ShapeType == ENavigationShapeType::Convex || ShapeType == ENavigationShapeType::InstancedConvex)
						{
							auto AddElementFunc = [&](const FConvexNavAreaData& InConvexNavAreaData)
							{
								Verts.Reset();

								const TArray<FVector> Points = UE::LWC::ConvertArrayType<FVector>(InConvexNavAreaData.Points);
								GrowConvexHull(NavData->AgentRadius, Points, Verts);

								if (Verts.Num())
								{
									const float CellHeight = NavData->GetCellHeight(ENavigationDataResolution::Default);
									Snapshot->AddPulledConvex(
										Verts,
										InConvexNavAreaData.MinZ - CellHeight,
										InConvexNavAreaData.MaxZ + CellHeight,
										CategoryName, NavAreaVerbosity, PolygonColor.WithAlpha(255));
								}
							};

							if (ShapeType == ENavigationShapeType::Convex)
							{
								FConvexNavAreaData Convex;
								AreaMod.GetConvex(Convex);
								AddElementFunc(Convex);
							}
							else // ShapeType == ENavigationShapeType::InstancedConvex
							{
								for (const FTransform& InstanceTransform : InstanceTransforms)
								{
									FConvexNavAreaData Convex;
									AreaMod.GetPerInstanceConvex(InstanceTransform, Convex);
									AddElementFunc(Convex);
								}
							}
						}
					}
				}
			});
		}

	}
}
#endif

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && ENABLE_VISUAL_LOG
void FRecastNavMeshGenerator::ExportNavigationData(const FString& FileName) const
{
	const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	const FNavigationOctree* NavOctree = NavSys ? NavSys->GetNavOctree() : NULL;
	if (NavOctree == NULL)
	{
		UE_LOGF(LogNavigation, Error, "Failed to export navigation data due to %ls being NULL", NavSys == NULL ? TEXT("NavigationSystem") : TEXT("NavOctree"));
		return;
	}

	const double StartExportTime = FPlatformTime::Seconds();

	FString CurrentTimeStr = FDateTime::Now().ToString();
	for (int32 Index = 0; Index < NavSys->NavDataSet.Num(); ++Index)
	{
		// feed data from octtree and mark for rebuild				
		TNavStatArray<FVector::FReal> CoordBuffer;
		TNavStatArray<int32> IndexBuffer;
		const ARecastNavMesh* NavData = Cast<const ARecastNavMesh>(NavSys->NavDataSet[Index]);
		if (NavData)
		{
			const bool bUseVirtualGeometryFilteringAndDirtying = NavData->bUseVirtualGeometryFilteringAndDirtying;
			
			struct FAreaExportData
			{
				FConvexNavAreaData Convex;
				uint8 AreaId;
			};
			TArray<FAreaExportData> AreaExport;

			NavOctree->FindElementsWithBoundsTest(TotalNavBounds,
				[this, NavData, &IndexBuffer, &CoordBuffer, &AreaExport, bUseVirtualGeometryFilteringAndDirtying](const FNavigationOctreeElement& Element)
			{
				const bool bExportGeometry = Element.Data->HasGeometry() && (
					bUseVirtualGeometryFilteringAndDirtying ?
						ShouldGenerateGeometryForOctreeElement(Element, NavData->GetConfig()) :
						Element.ShouldUseGeometry(NavData->GetConfig())
				);

				TArray<FTransform> InstanceTransforms;
				Element.Data->NavDataPerInstanceTransformDelegate.ExecuteIfBound(Element.Bounds.GetBox(), InstanceTransforms);
				
				if (bExportGeometry && Element.Data->CollisionData.Num())
				{
					const int32 NumInstances = FMath::Max(InstanceTransforms.Num(), 1);
					FRecastGeometryCache CachedGeometry(Element.Data->CollisionData.GetData());
					IndexBuffer.Reserve( IndexBuffer.Num() + (CachedGeometry.Header.NumFaces * 3 ) * NumInstances );
					CoordBuffer.Reserve( CoordBuffer.Num() + (CachedGeometry.Header.NumVerts * 3 ) * NumInstances );

					if (InstanceTransforms.Num() == 0)
					{
						for (int32 i = 0; i < CachedGeometry.Header.NumFaces * 3; i++)
						{
							IndexBuffer.Add(CachedGeometry.Indices[i] + CoordBuffer.Num() / 3);
						}
						for (int32 i = 0; i < CachedGeometry.Header.NumVerts * 3; i++)
						{
							CoordBuffer.Add(CachedGeometry.Verts[i]);
						}
					}
					for (const FTransform& InstanceTransform : InstanceTransforms)
					{
						for (int32 i = 0; i < CachedGeometry.Header.NumFaces * 3; i++)
						{
							IndexBuffer.Add(CachedGeometry.Indices[i] + CoordBuffer.Num() / 3);
						}

						FMatrix LocalToRecastWorld = InstanceTransform.ToMatrixWithScale()*Unreal2RecastMatrix();

						for (int32 i = 0; i < CachedGeometry.Header.NumVerts * 3; i += 3)
						{
							// collision cache stores coordinates in recast space, convert them to unreal and transform to recast world space
							FVector WorldRecastCoord = LocalToRecastWorld.TransformPosition(Recast2UnrealPoint(&CachedGeometry.Verts[i]));

							CoordBuffer.Add(WorldRecastCoord.X);
							CoordBuffer.Add(WorldRecastCoord.Y);
							CoordBuffer.Add(WorldRecastCoord.Z);
						}
					}
				}
				else
				{
					for (const FAreaNavModifier& AreaMod : Element.Data->Modifiers.GetAreas())
					{
						ENavigationShapeType::Type ShapeType = AreaMod.GetShapeType();
						
						if (ShapeType == ENavigationShapeType::Convex || ShapeType == ENavigationShapeType::InstancedConvex)
						{
							FAreaExportData ExportInfo;
							const int32 AreaId = NavData->GetAreaID(AreaMod.GetAreaClass());
							ExportInfo.AreaId = AreaId != INDEX_NONE ? IntCastChecked<uint8>(AreaId) : INDEX_NONE;

							auto AddAreaExportDataFunc = [&](const FConvexNavAreaData& InConvexNavAreaData)
							{
								TArray<FVector> ConvexVerts;

								const TArray<FVector> Points = UE::LWC::ConvertArrayType<FVector>(ExportInfo.Convex.Points);
								GrowConvexHull(NavData->AgentRadius, Points, ConvexVerts);
								if (ConvexVerts.Num())
								{
									const float CellHeight = NavData->GetCellHeight(ENavigationDataResolution::Default);
									ExportInfo.Convex.MinZ -= CellHeight;
									ExportInfo.Convex.MaxZ += CellHeight;

									ExportInfo.Convex.Points = UE::LWC::ConvertArrayType<FVector>(ConvexVerts);

									AreaExport.Add(ExportInfo);
								}								
							};

							if (ShapeType == ENavigationShapeType::Convex)
							{
								AreaMod.GetConvex(ExportInfo.Convex);
								AddAreaExportDataFunc(ExportInfo.Convex);
							}
							else // ShapeType == ENavigationShapeType::InstancedConvex
							{
								for (const FTransform& InstanceTransform : InstanceTransforms)
								{
									AreaMod.GetPerInstanceConvex(InstanceTransform, ExportInfo.Convex);
									AddAreaExportDataFunc(ExportInfo.Convex);
								}
							}
						}
					}
				}
			});
			
			UWorld* NavigationWorld = GetWorld();
			for (int32 LevelIndex = 0; LevelIndex < NavigationWorld->GetNumLevels(); ++LevelIndex) 
			{
				const ULevel* const Level =  NavigationWorld->GetLevel(LevelIndex);
				if (Level == NULL)
				{
					continue;
				}

				const TArray<FVector>* LevelGeom = Level->GetStaticNavigableGeometry();
				if (LevelGeom != NULL && LevelGeom->Num() > 0)
				{
					TNavStatArray<FVector> Verts;
					TNavStatArray<int32> Faces;
					// For every ULevel in World take its pre-generated static geometry vertex soup
					RecastGeometryExport::TransformVertexSoupToRecast(*LevelGeom, Verts, Faces);

					IndexBuffer.Reserve( IndexBuffer.Num() + Faces.Num() );
					CoordBuffer.Reserve( CoordBuffer.Num() + Verts.Num() * 3);
					for (int32 i = 0; i < Faces.Num(); i++)
					{
						IndexBuffer.Add(Faces[i] + CoordBuffer.Num() / 3);
					}
					for (int32 i = 0; i < Verts.Num(); i++)
					{
						CoordBuffer.Add(Verts[i].X);
						CoordBuffer.Add(Verts[i].Y);
						CoordBuffer.Add(Verts[i].Z);
					}
				}
			}
			
			
			FString AreaExportStr;
			for (int32 i = 0; i < AreaExport.Num(); i++)
			{
				const FAreaExportData& ExportInfo = AreaExport[i];
				AreaExportStr += FString::Printf(TEXT("\nAE %d %d %f %f\n"),
					ExportInfo.AreaId, ExportInfo.Convex.Points.Num(), ExportInfo.Convex.MinZ, ExportInfo.Convex.MaxZ);

				for (int32 iv = 0; iv < ExportInfo.Convex.Points.Num(); iv++)
				{
					FVector Pt = Unreal2RecastPoint(ExportInfo.Convex.Points[iv]);
					AreaExportStr += FString::Printf(TEXT("Av %f %f %f\n"), Pt.X, Pt.Y, Pt.Z);
				}
			}
			
			FString AdditionalData;
			
			if (AreaExport.Num())
			{
				AdditionalData += "# Area export\n";
				AdditionalData += AreaExportStr;
				AdditionalData += "\n";
			}

			AdditionalData += "# RecastDemo specific data\n";
	#if 0
			// use this bounds to have accurate navigation data bounds
			const FVector Center = Unreal2RecastPoint(NavData->GetBounds().GetCenter());
			FVector Extent = FVector(NavData->GetBounds().GetExtent());
			Extent = FVector(Extent.X, Extent.Z, Extent.Y);
	#else
			// this bounds match navigation bounds from level
			FBox RCNavBounds = Unreal2RecastBox(TotalNavBounds);
			const FVector Center = RCNavBounds.GetCenter();
			const FVector Extent = RCNavBounds.GetExtent();
	#endif
			const FBox Box = FBox::BuildAABB(Center, Extent);
			AdditionalData += FString::Printf(
				TEXT("rd_bbox %7.7f %7.7f %7.7f %7.7f %7.7f %7.7f\n"), 
				Box.Min.X, Box.Min.Y, Box.Min.Z, 
				Box.Max.X, Box.Max.Y, Box.Max.Z
			);
			
			const FRecastNavMeshGenerator* CurrentGen = static_cast<const FRecastNavMeshGenerator*>(NavData->GetGenerator());
			check(CurrentGen);
			AdditionalData += FString::Printf(TEXT("# AgentHeight\n"));
			AdditionalData += FString::Printf(TEXT("rd_agh %5.5f\n"), CurrentGen->Config.AgentHeight);
			AdditionalData += FString::Printf(TEXT("# AgentRadius\n"));
			AdditionalData += FString::Printf(TEXT("rd_agr %5.5f\n"), CurrentGen->Config.AgentRadius);

			AdditionalData += FString::Printf(TEXT("# Cell Size\n"));
			AdditionalData += FString::Printf(TEXT("rd_cs %5.5f\n"), CurrentGen->Config.cs);
			AdditionalData += FString::Printf(TEXT("# Cell Height\n"));
			AdditionalData += FString::Printf(TEXT("rd_ch %5.5f\n"), CurrentGen->Config.ch);

			AdditionalData += FString::Printf(TEXT("# Agent max climb\n"));
			AdditionalData += FString::Printf(TEXT("rd_amc %d\n"), (int)CurrentGen->Config.AgentMaxClimb);
			AdditionalData += FString::Printf(TEXT("# Agent max slope\n"));
			AdditionalData += FString::Printf(TEXT("rd_ams %5.5f\n"), CurrentGen->Config.walkableSlopeAngle);

			AdditionalData += FString::Printf(TEXT("# Region min size\n"));
			AdditionalData += FString::Printf(TEXT("rd_rmis %d\n"), (uint32)FMath::Sqrt(static_cast<float>(CurrentGen->Config.minRegionArea)));
			AdditionalData += FString::Printf(TEXT("# Region merge size\n"));
			AdditionalData += FString::Printf(TEXT("rd_rmas %d\n"), (uint32)FMath::Sqrt(static_cast<float>(CurrentGen->Config.mergeRegionArea)));

			AdditionalData += FString::Printf(TEXT("# Max edge len\n"));
			AdditionalData += FString::Printf(TEXT("rd_mel %d\n"), CurrentGen->Config.maxEdgeLen);

			AdditionalData += FString::Printf(TEXT("# Perform Voxel Filtering\n"));
			AdditionalData += FString::Printf(TEXT("rd_pvf %d\n"), CurrentGen->Config.bPerformVoxelFiltering);
			AdditionalData += FString::Printf(TEXT("# Generate Detailed Mesh\n"));
			AdditionalData += FString::Printf(TEXT("rd_gdm %d\n"), CurrentGen->Config.bGenerateDetailedMesh);
			AdditionalData += FString::Printf(TEXT("# MaxPolysPerTile\n"));
			AdditionalData += FString::Printf(TEXT("rd_mppt %d\n"), CurrentGen->Config.MaxPolysPerTile);
			AdditionalData += FString::Printf(TEXT("# maxVertsPerPoly\n"));
			AdditionalData += FString::Printf(TEXT("rd_mvpp %d\n"), CurrentGen->Config.maxVertsPerPoly);
			AdditionalData += FString::Printf(TEXT("# Tile size\n"));
			AdditionalData += FString::Printf(TEXT("rd_ts %d\n"), CurrentGen->Config.tileSize);

			AdditionalData += FString::Printf(TEXT("\n"));
			
			const FString FilePathName = FileName + FString::Printf(TEXT("_NavDataSet%d_%s.obj"), Index, *CurrentTimeStr) ;
			ExportGeomToOBJFile(FilePathName, CoordBuffer, IndexBuffer, AdditionalData);
		}
	}
	UE_LOGF(LogNavigation, Log, "ExportNavigation time: %.3f sec .", FPlatformTime::Seconds() - StartExportTime);
}
#endif

static class FNavigationGeomExec : private FSelfRegisteringExec
{
protected:
	/** Console commands, see embeded usage statement **/
	virtual bool Exec_Dev( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar ) override
	{
		bool bExported = false;
#if ALLOW_DEBUG_FILES && ENABLE_VISUAL_LOG
		if (FParse::Command(&Cmd, TEXT("ExportNavigation")))
		{
			if (InWorld == nullptr)
			{
				UE_LOGF(LogNavigation, Error, "Failed to export navigation data due to missing UWorld");
			}
			else 
			{
				UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(InWorld);
				if (NavSys)
				{
					for (ANavigationData* NavData : NavSys->NavDataSet)
					{
						if (const FNavDataGenerator* Generator = NavData->GetGenerator())
						{
							Generator->ExportNavigationData(FString::Printf(TEXT("%s/%s"), *FPaths::ProjectSavedDir(), *NavData->GetName()));
							bExported = true;
						}
						else
						{
							UE_LOGF(LogNavigation, Error, "Failed to export navigation data %ls due to missing generator", *NavData->GetName());
						}
					}
				}
				else
				{
					UE_LOGF(LogNavigation, Error, "Failed to export navigation data due to missing navigation system");
				}
			}
		}
#endif // ALLOW_DEBUG_FILES && !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		return bExported;
	}
} NavigationGeomExec;

#if RECAST_INTERNAL_DEBUG_DATA
bool FRecastTileGenerator::IsTileDebugActive() const
{
	return (TileDebugSettings.bEnabled && TileDebugSettings.IsWithinTileCoordinates(TileX, TileY))
		|| (TileX == GNavmeshDebugTileX && TileY == GNavmeshDebugTileY);
}

bool FRecastTileGenerator::IsTileDebugAllowingGeneration() const
{
	if (TileDebugSettings.bEnabled && TileDebugSettings.bGenerateDebugTileOnly)
	{
		return TileDebugSettings.IsWithinTileCoordinates(TileX, TileY);
	}
	else if (GNavmeshGenerateDebugTileOnly)
	{
		return TileX == GNavmeshDebugTileX && TileY == GNavmeshDebugTileY;
	}
	else
	{
		return true;
	}
}
#endif //RECAST_INTERNAL_DEBUG_DATA
#endif // WITH_RECAST
