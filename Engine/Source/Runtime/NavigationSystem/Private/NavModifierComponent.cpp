// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavModifierComponent.h"
#include "GameFramework/Actor.h"
#include "Components/PrimitiveComponent.h"
#include "AI/NavigationModifier.h"
#include "NavAreas/NavArea_Null.h"
#include "PhysicsEngine/BodySetup.h"
#include "NavigationSystem.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "Engine/StaticMesh.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavModifierComponent)

namespace UE::Navigation::ModComponent::Private
{
#if WITH_EDITOR
	void OnNavAreaRegistrationChanged(UNavModifierComponent& ModifierComponent, const UWorld& World, const UClass* NavAreaClass)
	{
		if (NavAreaClass && (NavAreaClass == ModifierComponent.AreaClass || NavAreaClass == ModifierComponent.AreaClassToReplace) && &World == ModifierComponent.GetWorld())
		{
			ModifierComponent.RefreshNavigationModifiers();
		}
	}
#endif // WITH_EDITOR

	FAreaNavModifier CreateAreaModifier(const FBox& Box, const FQuat& Quat, const TSubclassOf<UNavArea>& AreaClass, const TSubclassOf<UNavArea>& AreaClassToReplace, const bool bIncludeAgentHeight)
	{
		FAreaNavModifier AreaNavModifier(Box, FTransform(Quat), AreaClass);
		AreaNavModifier.SetIncludeAgentHeight(bIncludeAgentHeight);
		if (AreaClassToReplace)
		{
			AreaNavModifier.SetAreaClassToReplace(AreaClassToReplace);
		}
		return AreaNavModifier;
	}
} // UE::Navigation::ModComponent::Private

UNavModifierComponent::UNavModifierComponent(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	AreaClass = UNavArea_Null::StaticClass();
	AreaClassToReplace = nullptr;
	FailsafeExtent = FVector(100, 100, 100);
	bIncludeAgentHeight = true;
	NavMeshResolution = ENavigationDataResolution::Invalid;
}

void UNavModifierComponent::OnRegister()
{
	Super::OnRegister();

#if WITH_EDITOR
	if (GIsEditor && !HasAnyFlags(RF_ClassDefaultObject))
	{
		OnNavAreaRegisteredDelegateHandle = UNavigationSystemBase::OnNavAreaRegisteredDelegate().AddUObject(this, &UNavModifierComponent::OnNavAreaRegistered);
		OnNavAreaUnregisteredDelegateHandle = UNavigationSystemBase::OnNavAreaUnregisteredDelegate().AddUObject(this, &UNavModifierComponent::OnNavAreaUnregistered);
	}
#endif // WITH_EDITOR 
}

void UNavModifierComponent::OnUnregister()
{
#if WITH_EDITOR
	if (GIsEditor && !HasAnyFlags(RF_ClassDefaultObject))
	{
		UNavigationSystemBase::OnNavAreaRegisteredDelegate().Remove(OnNavAreaRegisteredDelegateHandle);
		UNavigationSystemBase::OnNavAreaUnregisteredDelegate().Remove(OnNavAreaUnregisteredDelegateHandle);
	}
#endif // WITH_EDITOR 

	Super::OnUnregister();
}

#if WITH_EDITOR
// This function is only called if GIsEditor == true for non default objects components that are registered.
void UNavModifierComponent::OnNavAreaRegistered(const UWorld& World, const UClass* NavAreaClass)
{
	UE::Navigation::ModComponent::Private::OnNavAreaRegistrationChanged(*this, World, NavAreaClass);
}

// This function is only called if GIsEditor == true for non default objects components that are registered.
void UNavModifierComponent::OnNavAreaUnregistered(const UWorld& World, const UClass* NavAreaClass)
{
	UE::Navigation::ModComponent::Private::OnNavAreaRegistrationChanged(*this, World, NavAreaClass);
}
#endif // WITH_EDITOR 

void UNavModifierComponent::PopulateComponentBounds(FTransform InParentTransform, const UBodySetup& InBodySetup) const
{
	const FVector Scale3D = InParentTransform.GetScale3D();
	InParentTransform.RemoveScaling();
	
	for (int32 SphereIdx = 0; SphereIdx < InBodySetup.AggGeom.SphereElems.Num(); SphereIdx++)
	{
		const FKSphereElem& ElemInfo = InBodySetup.AggGeom.SphereElems[SphereIdx];
		FTransform ElemTM = ElemInfo.GetTransform();
		ElemTM.ScaleTranslation(Scale3D);
		ElemTM *= InParentTransform;

		const FBox SphereBounds = FBox::BuildAABB(ElemTM.GetLocation(), ElemInfo.Radius * Scale3D);
		ComponentBounds.Add(FRotatedBox(SphereBounds, ElemTM.GetRotation()));
	}

	for (int32 BoxIdx = 0; BoxIdx < InBodySetup.AggGeom.BoxElems.Num(); BoxIdx++)
	{
		const FKBoxElem& ElemInfo = InBodySetup.AggGeom.BoxElems[BoxIdx];
		FTransform ElemTM = ElemInfo.GetTransform();
		ElemTM.ScaleTranslation(Scale3D);
		ElemTM *= InParentTransform;

		const FBox BoxBounds = FBox::BuildAABB(ElemTM.GetLocation(), FVector(ElemInfo.X, ElemInfo.Y, ElemInfo.Z) * Scale3D * 0.5f);
		ComponentBounds.Add(FRotatedBox(BoxBounds, ElemTM.GetRotation()));
	}

	for (int32 SphylIdx = 0; SphylIdx < InBodySetup.AggGeom.SphylElems.Num(); SphylIdx++)
	{
		const FKSphylElem& ElemInfo = InBodySetup.AggGeom.SphylElems[SphylIdx];
		FTransform ElemTM = ElemInfo.GetTransform();
		ElemTM.ScaleTranslation(Scale3D);
		ElemTM *= InParentTransform;

		const FBox SphylBounds = FBox::BuildAABB(ElemTM.GetLocation(), FVector(ElemInfo.Radius, ElemInfo.Radius, ElemInfo.Length) * Scale3D);
		ComponentBounds.Add(FRotatedBox(SphylBounds, ElemTM.GetRotation()));
	}

	for (int32 ConvexIdx = 0; ConvexIdx < InBodySetup.AggGeom.ConvexElems.Num(); ConvexIdx++)
	{
		const FKConvexElem& ElemInfo = InBodySetup.AggGeom.ConvexElems[ConvexIdx];
		FTransform ElemTM = ElemInfo.GetTransform();

		const FBox ConvexBounds = FBox::BuildAABB(InParentTransform.TransformPosition(ElemInfo.ElemBox.GetCenter() * Scale3D), ElemInfo.ElemBox.GetExtent() * Scale3D);
		ComponentBounds.Add(FRotatedBox(ConvexBounds, ElemTM.GetRotation() * InParentTransform.GetRotation()));
	}
}

void UNavModifierComponent::CalculateBounds() const
{
	const AActor* MyOwner = GetOwner();
	if (!MyOwner)
	{
		return;
	}

	Bounds = FBox(ForceInit);
    ComponentBounds.Reset();
    for (UActorComponent* Component : MyOwner->GetComponents())
    {
    	UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(Component);
    	if (PrimComp && PrimComp->IsRegistered() && PrimComp->IsCollisionEnabled() && PrimComp->CanEverAffectNavigation())
    	{
    		UBodySetup* BodySetup = PrimComp->GetBodySetup();
    		if (BodySetup)
    		{
    			Bounds += PrimComp->Bounds.GetBox();
    				
    			const FTransform& ParentTM = PrimComp->GetComponentTransform();
    			PopulateComponentBounds(ParentTM, *BodySetup);
    		}
    		else if (const UGeometryCollectionComponent* GeometryCollection = Cast<UGeometryCollectionComponent>(PrimComp))
    		{
    			// If it's a GC, use the bodySetups from the proxyMeshes.
    			if (const TObjectPtr<const UGeometryCollection> RestCollection = GeometryCollection->RestCollection)
    			{
    				Bounds += GeometryCollection->Bounds.GetBox();

					const FGeometryCollectionProxyMeshData& ProxyMeshData = RestCollection->RootProxyData;
					for (int32 MeshIdx = 0; MeshIdx < ProxyMeshData.ProxyMeshes.Num(); ++MeshIdx)
    				{
						const TObjectPtr<UStaticMesh>& ProxyMesh = ProxyMeshData.ProxyMeshes[MeshIdx];
    					if (ProxyMesh != nullptr)
    					{
    						const UBodySetup* Body = ProxyMesh->GetBodySetup();
    						if (Body)
    						{
    							const FTransform& ParentTM = PrimComp->GetComponentTransform();
								const FTransform LocalMeshTransform(ProxyMeshData.GetMeshTransform(MeshIdx));
    							PopulateComponentBounds(LocalMeshTransform* ParentTM, *Body);
    						}
    					}
    				}
    			}
    		}	
    	}
    }

    if (ComponentBounds.Num() == 0)
    {
    	Bounds = FBox::BuildAABB(MyOwner->GetActorLocation(), FailsafeExtent);
    	ComponentBounds.Add(FRotatedBox(Bounds, MyOwner->GetActorQuat()));
    }

    for (int32 Idx = 0; Idx < ComponentBounds.Num(); Idx++)
    {
    	const FVector BoxOrigin = ComponentBounds[Idx].Box.GetCenter();
    	const FVector BoxExtent = ComponentBounds[Idx].Box.GetExtent();

    	const FVector NavModBoxOrigin = FTransform(ComponentBounds[Idx].Quat).InverseTransformPosition(BoxOrigin);
    	ComponentBounds[Idx].Box = FBox::BuildAABB(NavModBoxOrigin, BoxExtent);
    }
}

void UNavModifierComponent::CalcAndCacheBounds() const
{
	const AActor* MyOwner = GetOwner();
	if (MyOwner)
	{
		CachedTransform = MyOwner->GetActorTransform();
		
		if (TransformUpdateHandle.IsValid() == false && MyOwner->GetRootComponent())
		{
			// binding to get notifies when the root component moves. We need
			// this only when the rootcomp is nav-irrelevant (since the default 
			// mechanisms won't kick in) but we're binding without checking it since
			// this property can change without re-running CalcAndCacheBounds.
			// We're filtering for nav relevancy in OnTransformUpdated.
			TransformUpdateHandle = MyOwner->GetRootComponent()->TransformUpdated.AddUObject(const_cast<UNavModifierComponent*>(this), &UNavModifierComponent::OnTransformUpdated);
		}
	}

	CalculateBounds();

	UE_SUPPRESS(LogNavigation, VeryVerbose,
	{
		TArray<FAreaNavModifier> Areas;
		for (int32 Idx = 0; Idx < ComponentBounds.Num(); Idx++)
		{
			Areas.Add(UE::Navigation::ModComponent::Private::CreateAreaModifier(ComponentBounds[Idx].Box, ComponentBounds[Idx].Quat, AreaClass, AreaClassToReplace, bIncludeAgentHeight));
		}

		for(const FAreaNavModifier& Modifier : Areas)
		{
			UE_VLOG_BOX(this, LogNavigation, VeryVerbose, Modifier.GetBounds(), FColor::Yellow, TEXT(""));	
		}
	});
}

void UNavModifierComponent::GetNavigationData(FNavigationRelevantData& Data) const
{
	for (int32 Idx = 0; Idx < ComponentBounds.Num(); Idx++)
	{
		Data.Modifiers.Add(UE::Navigation::ModComponent::Private::CreateAreaModifier(ComponentBounds[Idx].Box, ComponentBounds[Idx].Quat, AreaClass, AreaClassToReplace, bIncludeAgentHeight));
	}

	Data.Modifiers.SetNavMeshResolution(NavMeshResolution);
}

void UNavModifierComponent::SetAreaClass(TSubclassOf<UNavArea> NewAreaClass)
{
	if (AreaClass != NewAreaClass)
	{
		AreaClass = NewAreaClass;
		RefreshNavigationModifiers();
	}
}

void UNavModifierComponent::SetAreaClassToReplace(TSubclassOf<UNavArea> NewAreaClassToReplace)
{
	if (AreaClassToReplace != NewAreaClassToReplace)
	{
		AreaClassToReplace = NewAreaClassToReplace;
		RefreshNavigationModifiers();
	}
}

void UNavModifierComponent::OnTransformUpdated(USceneComponent* RootComponent, EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	// force bounds recaching next time GetNavigationBounds gets called.
	bBoundsInitialized = false;

	// otherwise the update will be handled by the default path
	if (RootComponent && RootComponent->CanEverAffectNavigation() == false)
	{
		// since the parent is not nav-relevant we need to manually tell nav sys to update
		RefreshNavigationModifiers();
	}
}

