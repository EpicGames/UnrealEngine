// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavModifierVolume.h"
#include "NavigationSystem.h"
#include "NavigationSystemTypes.h"
#include "AI/NavigationModifier.h"
#include "NavAreas/NavArea_Null.h"
#include "NavigationOctree.h"
#include "Components/BrushComponent.h"
#include "AI/NavigationSystemHelpers.h"
#include "Engine/CollisionProfile.h"
#include "Model.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavModifierVolume)

#if WITH_EDITOR
namespace UE::Navigation::ModVolume::Private
{
	void OnNavAreaRegistrationChanged(ANavModifierVolume& ModifierVolume, const UWorld& World, const UClass* NavAreaClass)
	{
		if (NavAreaClass
			&& (NavAreaClass == ModifierVolume.GetAreaClass() || NavAreaClass == ModifierVolume.GetAreaClassToReplace())
			&& &World == ModifierVolume.GetWorld()
			&& ModifierVolume.HasActorRegisteredAllComponents()) // Update only required after initial registration was completed
		{
			FNavigationSystem::UpdateActorData(ModifierVolume);
		}
	}
} // UE::Navigation::ModVolume::Private
#endif // WITH_EDITOR

//----------------------------------------------------------------------//
// ANavModifierVolume
//----------------------------------------------------------------------//
ANavModifierVolume::ANavModifierVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, AreaClass(UNavArea_Null::StaticClass())
	, AreaClassToReplace(nullptr)
	, NavMeshResolution(ENavigationDataResolution::Invalid)
{
	if (GetBrushComponent())
	{
		GetBrushComponent()->SetGenerateOverlapEvents(false);
		GetBrushComponent()->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	}
}

void ANavModifierVolume::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	if (GIsEditor && !HasAnyFlags(RF_ClassDefaultObject) && !IsInAsyncLoadingThread())
	{
		OnNavAreaRegisteredDelegateHandle = UNavigationSystemBase::OnNavAreaRegisteredDelegate().AddUObject(this, &ANavModifierVolume::OnNavAreaRegistered);
		OnNavAreaUnregisteredDelegateHandle = UNavigationSystemBase::OnNavAreaUnregisteredDelegate().AddUObject(this, &ANavModifierVolume::OnNavAreaUnregistered);
	}
#endif // WITH_EDITOR
}

void ANavModifierVolume::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (GIsEditor && !HasAnyFlags(RF_ClassDefaultObject))
	{
		OnNavAreaRegisteredDelegateHandle = UNavigationSystemBase::OnNavAreaRegisteredDelegate().AddUObject(this, &ANavModifierVolume::OnNavAreaRegistered);
		OnNavAreaUnregisteredDelegateHandle = UNavigationSystemBase::OnNavAreaUnregisteredDelegate().AddUObject(this, &ANavModifierVolume::OnNavAreaUnregistered);
	}
#endif // WITH_EDITOR
}

void ANavModifierVolume::BeginDestroy()
{
	Super::BeginDestroy();

#if WITH_EDITOR
	if (GIsEditor && !HasAnyFlags(RF_ClassDefaultObject))
	{
		UNavigationSystemBase::OnNavAreaRegisteredDelegate().Remove(OnNavAreaRegisteredDelegateHandle);
		UNavigationSystemBase::OnNavAreaUnregisteredDelegate().Remove(OnNavAreaUnregisteredDelegateHandle);
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR

void ANavModifierVolume::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	if (RootComponent)
	{
		RootComponent->TransformUpdated.AddLambda([this](USceneComponent*, EUpdateTransformFlags, ETeleportType)
			{
				FNavigationSystem::UpdateActorData(*this);
			});
	}
}

void ANavModifierVolume::PostUnregisterAllComponents()
{
	if (RootComponent)
	{
		RootComponent->TransformUpdated.RemoveAll(this);
	}

	Super::PostUnregisterAllComponents();
}

// This function is only called if GIsEditor == true
void ANavModifierVolume::OnNavAreaRegistered(const UWorld& World, const UClass* NavAreaClass)
{
	UE::Navigation::ModVolume::Private::OnNavAreaRegistrationChanged(*this, World, NavAreaClass);
}

// This function is only called if GIsEditor == true
void ANavModifierVolume::OnNavAreaUnregistered(const UWorld& World, const UClass* NavAreaClass)
{
	UE::Navigation::ModVolume::Private::OnNavAreaRegistrationChanged(*this, World, NavAreaClass);
}
#endif // WITH_EDITOR

void ANavModifierVolume::GetNavigationData(FNavigationRelevantData& Data) const
{
	if (Brush && AreaClass)
	{
		// No need to create modifiers if the AreaClass we want to set is the default one unless we want to replace a NavArea to default.
		if (AreaClass != FNavigationSystem::GetDefaultWalkableArea() || AreaClassToReplace)
		{
			Data.Modifiers.CreateAreaModifiers(GetBrushComponent(), AreaClass, AreaClassToReplace);
		}
	}

	if (GetBrushComponent()->Brush != nullptr)
	{
		if (bMaskFillCollisionUnderneathForNavmesh)
		{
			const FBox& Box = GetBrushComponent()->Brush->Bounds.GetBox();
			FAreaNavModifier AreaMod(Box, GetBrushComponent()->GetComponentTransform(), AreaClass);
			if (AreaClassToReplace)
			{
				AreaMod.SetAreaClassToReplace(AreaClassToReplace);
			}
			Data.Modifiers.SetMaskFillCollisionUnderneathForNavmesh(true);
			Data.Modifiers.Add(AreaMod);
		}

		if (NavMeshResolution != ENavigationDataResolution::Invalid)
		{
			Data.Modifiers.SetNavMeshResolution(NavMeshResolution);
		}
	}
}

FBox ANavModifierVolume::GetNavigationBounds() const
{
	return GetComponentsBoundingBox(/*bNonColliding=*/ true);
}

void ANavModifierVolume::SetAreaClass(TSubclassOf<UNavArea> NewAreaClass)
{
	if (NewAreaClass != AreaClass)
	{
		AreaClass = NewAreaClass;

		FNavigationSystem::UpdateActorData(*this);
	}
}

void ANavModifierVolume::SetAreaClassToReplace(TSubclassOf<UNavArea> NewAreaClassToReplace)
{
	if (NewAreaClassToReplace != AreaClassToReplace)
	{
		AreaClassToReplace = NewAreaClassToReplace;

		FNavigationSystem::UpdateActorData(*this);
	}
}

void ANavModifierVolume::RebuildNavigationData()
{
	FNavigationSystem::UpdateActorData(*this);
}

#if WITH_EDITOR

void ANavModifierVolume::PostEditUndo()
{
	Super::PostEditUndo();

	if (GetBrushComponent())
	{
		GetBrushComponent()->BuildSimpleBrushCollision();
	}
	FNavigationSystem::UpdateActorData(*this);
}

void ANavModifierVolume::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName NAME_AreaClass = GET_MEMBER_NAME_CHECKED(ANavModifierVolume, AreaClass);
	static const FName NAME_AreaClassToReplace = GET_MEMBER_NAME_CHECKED(ANavModifierVolume, AreaClassToReplace);
	static const FName NAME_BrushComponent = "BrushComponent";

	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	const FName PropName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (PropName == NAME_AreaClass || PropName == NAME_AreaClassToReplace)
	{
		FNavigationSystem::UpdateActorData(*this);
	}
	else if (PropName == NAME_BrushComponent)
	{
		if (GetBrushComponent())
		{
			if (GetBrushComponent()->GetBodySetup() && NavigationHelper::IsBodyNavigationRelevant(*GetBrushComponent()->GetBodySetup()))
			{
				FNavigationSystem::UpdateActorData(*this);
			}
			else
			{
				FNavigationSystem::OnActorUnregistered(*this);
			}
		}
	}
}

#endif

