// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavRelevantComponent.h"
#include "NavigationSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavRelevantComponent)

UNavRelevantComponent::UNavRelevantComponent(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
, bAttachToOwnersRoot(true)
, bBoundsInitialized(false)
, bNavParentCacheInitialized(false)
{
	bCanEverAffectNavigation = true;
	bNavigationRelevant = true;
}

void UNavRelevantComponent::OnRegister()
{
	Super::OnRegister();

	if (bAttachToOwnersRoot)
	{
		bool bUpdateCachedParent = true;
#if WITH_EDITOR
		UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
		if (NavSys && NavSys->IsNavigationRegisterLocked())
		{
			bUpdateCachedParent = false;
		}
#endif

		AActor* OwnerActor = GetOwner();
		if (OwnerActor && bUpdateCachedParent)
		{
			// attach to root component if it's relevant for navigation
			UActorComponent* ActorComp = OwnerActor->GetRootComponent();			
			INavRelevantInterface* NavInterface = ActorComp ? Cast<INavRelevantInterface>(ActorComp) : nullptr;
			if (NavInterface && NavInterface->IsNavigationRelevant() &&
				OwnerActor->IsComponentRelevantForNavigation(ActorComp))
			{
				CachedNavParent = ActorComp;
			}

			// otherwise try actor itself under the same condition
			if (CachedNavParent == nullptr)
			{
				NavInterface = Cast<INavRelevantInterface>(OwnerActor);
				if (NavInterface && NavInterface->IsNavigationRelevant())
				{
					CachedNavParent = OwnerActor;
				}
			}
		}
	}

	// Mark cache as initialized (even if null) from this point so calls to GetNavigationParent can be validated.
	bNavParentCacheInitialized = true;

	FNavigationSystem::OnComponentRegistered(*this);
}

void UNavRelevantComponent::OnUnregister()
{
	Super::OnUnregister();

	FNavigationSystem::OnComponentUnregistered(*this);
}

FBox UNavRelevantComponent::GetNavigationBounds() const
{
	if (!bBoundsInitialized)
	{
		CalcAndCacheBounds();

		// Mark bounds as initialized after the actor has registered all its components
		// since some of them have dependencies.
		// (e.g., NavModifierComponent relies on registered primitive components)
		if (GetOwner() && GetOwner()->HasActorRegisteredAllComponents())
		{
			bBoundsInitialized = true;
		}
	}

	return Bounds;
}

bool UNavRelevantComponent::IsNavigationRelevant() const
{
	return bNavigationRelevant;
}

void UNavRelevantComponent::UpdateNavigationBounds()
{
	CalcAndCacheBounds();
	bBoundsInitialized = true;
}

UObject* UNavRelevantComponent::GetNavigationParent() const
{
	UE_CLOGF(!bNavParentCacheInitialized, LogNavigation, Error, 
		"%ls called before initialization of the navigation parent cache for [%ls]. This might cause improper registration in the NavOctree and must be fixed.", 
		ANSI_TO_TCHAR(__FUNCTION__),
		*GetFullName());

	return CachedNavParent;
}

void UNavRelevantComponent::CalcAndCacheBounds() const
{
	const AActor* OwnerActor = GetOwner();
	const FVector MyLocation = OwnerActor ? OwnerActor->GetActorLocation() : FVector::ZeroVector;

	Bounds = FBox::BuildAABB(MyLocation, FVector(100.0f, 100.0f, 100.0f));
}

void UNavRelevantComponent::ForceNavigationRelevancy(bool bForce)
{
	bAttachToOwnersRoot = !bForce;
	if (bForce)
	{
		bNavigationRelevant = true;
	}

	RefreshNavigationModifiers();
}

void UNavRelevantComponent::SetNavigationRelevancy(bool bRelevant)
{
	if (bNavigationRelevant != bRelevant)
	{
		bNavigationRelevant = bRelevant;
		RefreshNavigationModifiers();
	}
}

void UNavRelevantComponent::RefreshNavigationModifiers()
{
	// Only update after component registration since some required informations are initialized at that time (i.e. Cached Navigation Parent)
	if (bRegistered)
	{
		FNavigationSystem::UpdateComponentData(*this);
	}
}

