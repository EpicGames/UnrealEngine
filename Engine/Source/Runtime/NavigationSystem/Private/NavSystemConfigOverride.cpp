// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavSystemConfigOverride.h"
#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"
#include "AI/NavigationSystemBase.h"
#include "NavigationSystem.h"
#include "TimerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavSystemConfigOverride)

#if WITH_EDITORONLY_DATA
#include "UObject/ConstructorHelpers.h"
#include "Components/BillboardComponent.h"
#include "Engine/Texture2D.h"
#include "Editor.h"
#endif // WITH_EDITORONLY_DATA


ANavSystemConfigOverride::ANavSystemConfigOverride(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComp"));
	RootComponent = SceneComponent;
	RootComponent->Mobility = EComponentMobility::Static;

#if WITH_EDITORONLY_DATA
	bIsSpatiallyLoaded = false;
	
	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));

	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> NoteTextureObject;
			FName ID_Notes;
			FText NAME_Notes;
			FConstructorStatics()
				: NoteTextureObject(TEXT("/Engine/EditorResources/S_Note"))
				, ID_Notes(TEXT("Notes"))
				, NAME_Notes(NSLOCTEXT("SpriteCategory", "Notes", "Notes"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		if (SpriteComponent)
		{
			SpriteComponent->Sprite = ConstructorStatics.NoteTextureObject.Get();
			SpriteComponent->SetRelativeScale3D(FVector(0.5f, 0.5f, 0.5f));
			SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_Notes;
			SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Notes;
			SpriteComponent->SetupAttachment(RootComponent);
			SpriteComponent->Mobility = EComponentMobility::Static;
		}
	}
#endif // WITH_EDITORONLY_DATA

	SetHidden(true);
	SetCanBeDamaged(false);

	bNetLoadOnClient = false;
}

void ANavSystemConfigOverride::BeginPlay()
{
	Super::BeginPlay();
	ApplyConfig();
}

void ANavSystemConfigOverride::ApplyConfig()
{
	UWorld* World = GetWorld();
	if (World)
	{
		UNavigationSystemBase* PrevNavSys = World->GetNavigationSystem();

		if (PrevNavSys == nullptr || OverridePolicy == ENavSystemOverridePolicy::Override)
		{
			OverrideNavSystem();
		}
		// PrevNavSys != null at this point
		else if (OverridePolicy == ENavSystemOverridePolicy::Append)
		{
			// take the prev nav system and append data to it
			AppendToNavSystem(*PrevNavSys);
		}
		// else PrevNavSys != null AND OverridePolicy == ENavSystemOverridePolicy::Skip, so ignoring the override
	}
}

void ANavSystemConfigOverride::PostInitProperties()
{
	Super::PostInitProperties();
}

void ANavSystemConfigOverride::AppendToNavSystem(UNavigationSystemBase& PrevNavSys)
{
	if (NavigationSystemConfig)
	{
		PrevNavSys.AppendConfig(*NavigationSystemConfig);
	}
}

void ANavSystemConfigOverride::OverrideNavSystem()
{
	UWorld* World = GetWorld();
	if (World == nullptr)
	{
		return;
	}

	AWorldSettings* WorldSetting = World->GetWorldSettings();
	if (WorldSetting)
	{
		WorldSetting->SetNavigationSystemConfigOverride(NavigationSystemConfig);
	}

	if (World->bIsWorldInitialized
		&& NavigationSystemConfig)
	{
		const FNavigationSystemRunMode RunMode = World->WorldType == EWorldType::Editor
			? FNavigationSystemRunMode::EditorMode
			: (World->WorldType == EWorldType::PIE
				? FNavigationSystemRunMode::PIEMode
				: FNavigationSystemRunMode::GameMode)
			;

		if (FNavigationSystem::IsEditorRunMode(RunMode))
		{
			FNavigationSystem::AddNavigationSystemToWorld(*World, RunMode, NavigationSystemConfig, /*bInitializeForWorld=*/false, /*bOverridePreviousNavSys=*/true);
#if WITH_EDITOR
			UNavigationSystemBase* NewNavSys = World->GetNavigationSystem();
			if (NewNavSys)
			{
				GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateUObject(this
					, &ANavSystemConfigOverride::InitializeForWorld, NewNavSys, World, RunMode));
			}
#endif // WITH_EDITOR
		}
		else
		{
			FNavigationSystem::AddNavigationSystemToWorld(*World, RunMode, NavigationSystemConfig, /*bInitializeForWorld=*/true, /*bOverridePreviousNavSys=*/true);
		}
	}
}

#if WITH_EDITOR
void ANavSystemConfigOverride::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	// ApplyConfig in PostRegisterAllComponents only for Editor worlds; applied in BeginPlay for Game worlds.
	UWorld* World = GetWorld();
	if (World == nullptr || World->IsGameWorld() || World->WorldType == EWorldType::Inactive)
	{
		return;
	}

	// While refreshing streaming levels, components are unregistered then registered again
	// and we don't want to modify the configuration and recreate the NavigationSystem.
	if (World->IsRefreshingStreamingLevels())
	{
		return;
	}

	// Config override should not be applied during cooking.
	if (GIsCookerLoadingPackage)
	{
		return;
	}

	ApplyConfig();
}

void ANavSystemConfigOverride::PostUnregisterAllComponents()
{
	Super::PostUnregisterAllComponents();

	if (NavigationSystemConfig == nullptr)
	{
		return;
	}

	// ApplyConfig was performed in PostRegisterAllComponents for Editor worlds so nothing to unregister for Game worlds.
	UWorld* World = GetWorld();
	if (World == nullptr || World->IsGameWorld() || World->WorldType == EWorldType::Inactive || !World->IsInitialized() || World->IsBeingCleanedUp())
	{
		return;
	}

	// While refreshing streaming levels, components are unregistered then registered again
	// and we don't want to modify the configuration and recreate the NavigationSystem.
	if (World->IsRefreshingStreamingLevels())
	{
		return;
	}

	// Config override should not be applied during cooking.
	if (GIsCookerLoadingPackage)
	{
		return;
	}

	// If our override was used to create the navigation system, we remove the dependency
	// and recreate the navigation system (if needed after removing the override)
	AWorldSettings* WorldSetting = World->GetWorldSettings();
	if (WorldSetting && WorldSetting->GetNavigationSystemConfigOverride() == NavigationSystemConfig)
	{
		WorldSetting->SetNavigationSystemConfigOverride(nullptr);
		FNavigationSystem::AddNavigationSystemToWorld(*World, FNavigationSystemRunMode::EditorMode, nullptr, /*bInitializeForWorld=*/true, /*bOverridePreviousNavSys=*/true);
	}
}

void ANavSystemConfigOverride::InitializeForWorld(UNavigationSystemBase* NewNavSys, UWorld* World, const FNavigationSystemRunMode RunMode)
{
	if (NewNavSys && World)
	{
		NewNavSys->InitializeForWorld(*World, RunMode);
	}
}

void ANavSystemConfigOverride::ApplyChanges()
{
	UWorld* World = GetWorld();
	if (World)
	{
		AWorldSettings* WorldSetting = World->GetWorldSettings();
		if (WorldSetting)
		{
			WorldSetting->SetNavigationSystemConfigOverride(NavigationSystemConfig);
		}

		// recreate nav sys
		World->SetNavigationSystem(nullptr);
		FNavigationSystem::AddNavigationSystemToWorld(*World, FNavigationSystemRunMode::EditorMode, NavigationSystemConfig, /*bInitializeForWorld=*/true);
	}
}

void ANavSystemConfigOverride::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	bNetLoadOnClient = bLoadOnClient;
}
#endif // WITH_EDITOR

