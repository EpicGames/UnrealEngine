// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationData.h"

#include "AI/NavDataGenerator.h"
#include "AI/Navigation/NavAgentInterface.h"
#include "AI/Navigation/NavAreaBase.h"
#include "AI/Navigation/NavigationDirtyArea.h"
#include "AssetCompilingManager.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/Engine.h"
#include "NavAreas/NavArea.h"
#include "NavFilters/NavigationQueryFilter.h"
#include "NavigationSystem.h"
#include "VisualLogger/VisualLogger.h"
#include "Misc/ScopeLock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationData)

// set to NAVMESHVER_LANDSCAPE_HEIGHT at the moment of refactoring navigation
// code out of the engine module. No point in using RecastNavMesh versioning 
// for NavigationData
#define NAVDATAVER_LATEST 13	

namespace UE::Navigation::Private
{
	static int32 bDestroyNavDataInCleanUpAndMarkPendingKill = 1;
	static FAutoConsoleVariableRef CVarDestroyNavDataInCleanUpAndMarkPendingKill(
		TEXT("ai.DestroyNavDataInCleanUpAndMarkPendingKill"),
		bDestroyNavDataInCleanUpAndMarkPendingKill,
		TEXT("If set to 1 NavData will be destroyed in CleanUpAndMarkPendingKill rather than being marked as garbage.\n"),
		ECVF_Default);

	// This prevents accumulating SuspendedDirtyAreas indefinitely and eventually running out of memory. That could happen if SetRebuildSuspended
	// is enabled for long periods of time. The limit cannot be too low because moving complex geometry made of hundreds or thousands of Actors
	// can quickly create several thousands of DirtyAreas.
	// Careful: Reaching this limit causes a Rebuild All for this navigation data (if it ever resumes generation).
	static int32 MaxDirtyAreasCountWhileSuspended = 32768;
	static FAutoConsoleVariableRef CVarMaxDirtyAreasCountWhileSuspended(
		TEXT("ai.navigation.MaxDirtyAreasCountWhileSuspended"),
		MaxDirtyAreasCountWhileSuspended,
		TEXT("If a navigation data accumulates too many dirty areas because it's been indefinitely suspended, we'll stop accumulating them and rebuild the entire navigation data if it ever resumes building.\n"),
		ECVF_Default);
}

//----------------------------------------------------------------------//
// FPathFindingQuery
//----------------------------------------------------------------------//
FPathFindingQuery::FPathFindingQuery(const UObject* InOwner, const ANavigationData& InNavData, const FVector& Start, const FVector& End, FSharedConstNavQueryFilter SourceQueryFilter, FNavPathSharedPtr InPathInstanceToFill, const FVector::FReal CostLimit, const bool bInRequireNavigableEndLocation) :
	FPathFindingQueryData(InOwner, Start, End, SourceQueryFilter, 0 /*InNavDataFlags*/, true /*bInAllowPartialPaths*/, CostLimit, bInRequireNavigableEndLocation),
	NavData(&InNavData), PathInstanceToFill(InPathInstanceToFill), NavAgentProperties(InNavData.GetConfig())
{
	if (!QueryFilter.IsValid() && NavData.IsValid())
	{
		QueryFilter = NavData->GetDefaultQueryFilter();
	}
}

FPathFindingQuery::FPathFindingQuery(const INavAgentInterface& InNavAgent, const ANavigationData& InNavData, const FVector& Start, const FVector& End, FSharedConstNavQueryFilter SourceQueryFilter, FNavPathSharedPtr InPathInstanceToFill, const FVector::FReal CostLimit, const bool bInRequireNavigableEndLocation) :
	FPathFindingQueryData(Cast<UObject>(&InNavAgent), Start, End, SourceQueryFilter, 0 /*InNavDataFlags*/, true /*bInAllowPartialPaths*/, CostLimit, bInRequireNavigableEndLocation),
	NavData(&InNavData), PathInstanceToFill(InPathInstanceToFill), NavAgentProperties(InNavAgent.GetNavAgentPropertiesRef())
{
	if (!QueryFilter.IsValid() && NavData.IsValid())
	{
		QueryFilter = NavData->GetDefaultQueryFilter();
	}
}

FPathFindingQuery::FPathFindingQuery(FNavPathSharedRef PathToRecalculate, const ANavigationData* NavDataOverride) :
	FPathFindingQueryData(PathToRecalculate->GetQueryData()),
	NavData(NavDataOverride ? NavDataOverride : PathToRecalculate->GetNavigationDataUsed()), PathInstanceToFill(PathToRecalculate), NavAgentProperties(FNavAgentProperties::DefaultProperties)
{
	if (PathToRecalculate->ShouldUpdateStartPointOnRepath() && (PathToRecalculate->GetSourceActor() != nullptr))
	{
		const FVector NewStartLocation = PathToRecalculate->GetPathFindingStartLocation();
		if (FNavigationSystem::IsValidLocation(NewStartLocation))
		{
			StartLocation = NewStartLocation;
		}
	}

	if (PathToRecalculate->ShouldUpdateEndPointOnRepath() && (PathToRecalculate->GetGoalActor() != nullptr))
	{
		const FVector NewEndLocation = PathToRecalculate->GetGoalLocation();
		if (FNavigationSystem::IsValidLocation(NewEndLocation))
		{
			EndLocation = NewEndLocation;
		}
	}

	if (NavData.IsValid())
	{
		if (!QueryFilter.IsValid())
		{
			QueryFilter = NavData->GetDefaultQueryFilter();
		}

		NavAgentProperties = NavData->GetConfig();
	}
}

FVector::FReal FPathFindingQuery::ComputeCostLimitFromHeuristic(const FVector& StartPos, const FVector& EndPos, const FVector::FReal HeuristicScale, const FVector::FReal CostLimitFactor, const FVector::FReal MinimumCostLimit)
{
	if (CostLimitFactor == FLT_MAX)
	{
		return FLT_MAX;
	}
	else
	{
		const FVector::FReal OriginalHeuristicEstimate = HeuristicScale * FVector::Dist(StartPos, EndPos);
		return FMath::Clamp(CostLimitFactor * OriginalHeuristicEstimate, MinimumCostLimit, TNumericLimits<FVector::FReal>::Max());
	}
}

//----------------------------------------------------------------------//
// FAsyncPathFindingQuery
//----------------------------------------------------------------------//
uint32 FAsyncPathFindingQuery::LastPathFindingUniqueID = INVALID_NAVQUERYID;

FAsyncPathFindingQuery::FAsyncPathFindingQuery(const UObject* InOwner, const ANavigationData& InNavData, const FVector& Start, const FVector& End, const FNavPathQueryDelegate& Delegate, FSharedConstNavQueryFilter SourceQueryFilter, const FVector::FReal CostLimit)
: FPathFindingQuery(InOwner, InNavData, Start, End, SourceQueryFilter)
, QueryID(GetUniqueID())
, OnDoneDelegate(Delegate)
, Mode(EPathFindingMode::Regular)
{

}

FAsyncPathFindingQuery::FAsyncPathFindingQuery(const FPathFindingQuery& Query, const FNavPathQueryDelegate& Delegate, const EPathFindingMode::Type QueryMode)
: FPathFindingQuery(Query)
, QueryID(GetUniqueID())
, OnDoneDelegate(Delegate)
, Mode(QueryMode)
{

}

//----------------------------------------------------------------------//
// FSupportedAreaData
//----------------------------------------------------------------------//
FSupportedAreaData::FSupportedAreaData(TSubclassOf<UNavArea> NavAreaClass, int32 InAreaID)
	: AreaID(InAreaID), AreaClass(NavAreaClass)
{
	if (AreaClass != NULL)
	{
		AreaClassName = AreaClass->GetName();
	}
	else
	{
		AreaClassName = TEXT("Invalid");
	}
}

//----------------------------------------------------------------------//
// FNavigationPath
//----------------------------------------------------------------------//
void FNavigationPath::RemoveOverlappingPoints(const FVector& Tolerance)
{
	// Remove overlapping points according to the tolerance.
	for (int32 Index = 0; Index+1 < PathPoints.Num();)
	{
		const FVector Delta = PathPoints[Index+1].Location - PathPoints[Index].Location;
		const bool bSameLocation = FMath::Abs(Delta.X) <= Tolerance.X && FMath::Abs(Delta.Y) <= Tolerance.Y && FMath::Abs(Delta.Z) <= Tolerance.Z;
		if (bSameLocation)
		{
			PathPoints.RemoveAt(Index);
		}
		else
		{
			++Index;
		}
	}
}

//----------------------------------------------------------------------//
// ANavigationData                                                                
//----------------------------------------------------------------------//
ANavigationData::ANavigationData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bEnableDrawing(false)
	, bForceRebuildOnLoad(false)
	, bAutoDestroyWhenNoNavigation(false)
	, bCanBeMainNavData(true)
	, bCanSpawnOnRebuild(true)
	, RuntimeGeneration(ERuntimeGenerationType::LegacyGeneration) //TODO: set to a valid value once bRebuildAtRuntime_DEPRECATED is removed
	, bRequiresInitialRebuild(false)
	, DataVersion(NAVDATAVER_LATEST)
	, FindPathImplementation(NULL)
	, FindHierarchicalPathImplementation(NULL)
	, bRegistered(false)
	, bRebuildingSuspended(false)
#if WITH_EDITORONLY_DATA
	, bIsBuildingOnLoad(false)
#endif
	, NavDataUniqueID(GetNextUniqueID())
{
	PrimaryActorTick.bCanEverTick = true;
	bNetLoadOnClient = false;
	SetCanBeDamaged(false);
	DefaultQueryFilter = MakeShareable(new FNavigationQueryFilter());
	ObservedPathsTickInterval = 0.5;

	// by giving NavigationData a root component we can detect changes to 
	// actor's location and react to it (see ARecastNavMesh::PostRegisterAllComponents)
	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComp"));
	RootComponent = SceneComponent;
	RootComponent->Mobility = EComponentMobility::Static;

#if WITH_EDITORONLY_DATA
	bIsSpatiallyLoaded = false;
#endif
}

uint16 ANavigationData::GetNextUniqueID()
{
	static FThreadSafeCounter StaticID(INVALID_NAVDATA);
	return IntCastChecked<uint16>(StaticID.Increment());
}

void ANavigationData::PostInitProperties()
{
	Super::PostInitProperties();

	if (!IsValidChecked(this))
	{
		return;
	}

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		if (RuntimeGeneration == ERuntimeGenerationType::LegacyGeneration)
		{
			RuntimeGeneration = bRebuildAtRuntime_DEPRECATED ? ERuntimeGenerationType::Dynamic : ERuntimeGenerationType::Static;
		}
	}
	else
	{
		bNetLoadOnClient = FNavigationSystem::ShouldLoadNavigationOnClient(*this);
		RequestRegistration();
#if UE_ENABLE_DEBUG_DRAWING
		RenderingComp = ConstructRenderingComponent();
#endif // UE_ENABLE_DEBUG_DRAWING
	}
}

void ANavigationData::PostInitializeComponents()
{
	Super::PostInitializeComponents();
	
	UWorld* MyWorld = GetWorld();
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(MyWorld);

	if (bAutoDestroyWhenNoNavigation && (MyWorld == nullptr ||
		(MyWorld->GetNetMode() != NM_Client && NavSys == nullptr) ||
		(MyWorld->GetNetMode() == NM_Client && !bNetLoadOnClient)))
	{
		UE_VLOG_UELOG(this, LogNavigation, Log, TEXT("Marking %s as PendingKill due to %s"), *GetName()
			, !MyWorld ? TEXT("No World") : (MyWorld->GetNetMode() == NM_Client ? TEXT("not creating navigation on clients") : TEXT("missing navigation system")));
		CleanUpAndMarkPendingKill();
	}
}

void ANavigationData::PostLoad() 
{
	Super::PostLoad();

	InstantiateAndRegisterRenderingComponent();

	bNetLoadOnClient = FNavigationSystem::ShouldLoadNavigationOnClient(*this);
	RequestRegistration();
}

void ANavigationData::RequestRegistration()
{
	if (IsRegistered() == false
		&& HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
		if (NavSys)
		{
			NavSys->RequestRegistrationDeferred(*this);
		}
	}
}

void ANavigationData::TickActor(float DeltaTime, enum ELevelTick TickType, FActorTickFunction& ThisTickFunction)
{
	Super::TickActor(DeltaTime, TickType, ThisTickFunction);

	PurgeUnusedPaths();

	INC_DWORD_STAT_BY(STAT_Navigation_ObservedPathsCount, ObservedPaths.Num());

	if (NextObservedPathsTickInSeconds >= 0.f)
	{
		NextObservedPathsTickInSeconds -= DeltaTime;
		if (NextObservedPathsTickInSeconds <= 0.f)
		{
			RepathRequests.Reserve(ObservedPaths.Num());

			for (int32 PathIndex = ObservedPaths.Num() - 1; PathIndex >= 0; --PathIndex)
			{
				if (ObservedPaths[PathIndex].IsValid())
				{
					FNavPathSharedPtr SharedPath = ObservedPaths[PathIndex].Pin();
					FNavigationPath* Path = SharedPath.Get();
					EPathObservationResult::Type Result = Path->TickPathObservation();
					switch (Result)
					{
					case EPathObservationResult::NoLongerObserving:
						ObservedPaths.RemoveAtSwap(PathIndex, EAllowShrinking::No);
						break;

					case EPathObservationResult::NoChange:
						// do nothing
						break;

					case EPathObservationResult::RequestRepath:
						RepathRequests.Add(FNavPathRecalculationRequest(SharedPath, ENavPathUpdateType::GoalMoved));
						break;
					
					default:
						check(false && "unhandled EPathObservationResult::Type in ANavigationData::TickActor");
						break;
					}
				}
				else
				{
					ObservedPaths.RemoveAtSwap(PathIndex, EAllowShrinking::No);
				}
			}

			if (ObservedPaths.Num() > 0)
			{
				NextObservedPathsTickInSeconds = ObservedPathsTickInterval;
			}
		}
	}

	if (RepathRequests.Num() > 0)
	{
		double TimeStamp = GetWorldTimeStamp();
		const UWorld* World = GetWorld();

		// @todo batch-process it!

		const int32 MaxProcessedRequests = 1000;

		// make a copy of path requests and reset (remove up to MaxProcessedRequests) from navdata's array
		// this allows storing new requests in the middle of loop (e.g. used by meta path corrections)

		TArray<FNavPathRecalculationRequest> WorkQueue(RepathRequests);
		if (WorkQueue.Num() > MaxProcessedRequests)
		{
			UE_VLOG(this, LogNavigation, Error, TEXT("Too many repath requests! (%d/%d)"), WorkQueue.Num(), MaxProcessedRequests);

			WorkQueue.RemoveAt(MaxProcessedRequests, WorkQueue.Num() - MaxProcessedRequests);
			RepathRequests.RemoveAt(0, MaxProcessedRequests);
		}
		else
		{
			RepathRequests.Reset();
		}

		for (int32 Idx = 0; Idx < WorkQueue.Num(); Idx++)
		{
			FNavPathRecalculationRequest& RecalcRequest = WorkQueue[Idx];

			// check if it can be updated right now
			FNavPathSharedPtr PinnedPath = RecalcRequest.Path.Pin();
			if (PinnedPath.IsValid() == false)
			{
				continue;
			}

			const UObject* PathQuerier = PinnedPath->GetQuerier();
			const INavAgentInterface* PathNavAgent = Cast<const INavAgentInterface>(PathQuerier);
			if (PathNavAgent && PathNavAgent->ShouldPostponePathUpdates())
			{
				RepathRequests.Add(RecalcRequest);
				continue;
			}

			FPathFindingQuery Query(PinnedPath.ToSharedRef());
			const FPathFindingResult Result = FindPath(Query.NavAgentProperties, Query.SetPathInstanceToUpdate(PinnedPath));

			// update time stamp to give observers any means of telling if it has changed
			PinnedPath->SetTimeStamp(TimeStamp);

			// partial paths are still valid and can change to full path when moving goal gets back on navmesh
			if (Result.IsSuccessful() || Result.IsPartial())
			{
				PinnedPath->UpdateLastRepathGoalLocation();
				PinnedPath->DoneUpdating(RecalcRequest.Reason);
				if (RecalcRequest.Reason == ENavPathUpdateType::NavigationChanged)
				{
					RegisterActivePath(PinnedPath);
				}
			}
			else
			{
				PinnedPath->RePathFailed();
			}
		}
	}
}

#if WITH_EDITOR
void ANavigationData::RerunConstructionScripts()
{
	Super::RerunConstructionScripts();

	InstantiateAndRegisterRenderingComponent();
}
#endif

void ANavigationData::OnRegistered() 
{ 
	InstantiateAndRegisterRenderingComponent();

	bRegistered = true;
	ConditionalConstructGenerator();
}

void ANavigationData::OnUnregistered()
{
	bRegistered = false;
}

void ANavigationData::InstantiateAndRegisterRenderingComponent()
{
#if UE_ENABLE_DEBUG_DRAWING
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (IsThisNotNull(this, "ANavigationData::InstantiateAndRegisterRenderingComponent") && IsValidChecked(this) && !IsValid(RenderingComp))
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		const bool bRootIsRenderComp = (RenderingComp == RootComponent);
		if (RenderingComp)
		{
			// rename the old rendering component out of the way
			RenderingComp->Rename(NULL, NULL, REN_DontCreateRedirectors | REN_ForceGlobalUnique | REN_DoNotDirty | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
		}

		RenderingComp = ConstructRenderingComponent();

		UWorld* World = GetWorld();
		if (World && World->bIsWorldInitialized && RenderingComp)
		{
			RenderingComp->RegisterComponent();
		}

		if (bRootIsRenderComp)
		{
			RootComponent = RenderingComp;
		}
	}
#endif // UE_ENABLE_DEBUG_DRAWING
}

void ANavigationData::PurgeUnusedPaths()
{
	check(IsInGameThread());

	// Paths can be registered from async pathfinding thread 
	// while unused paths are purged in main thread (actor tick)
	UE::TScopeLock PathLock(ActivePathsLock);

	const int32 Count = ActivePaths.Num();
	for (int32 PathIndex = Count - 1; PathIndex >= 0; --PathIndex)
	{
		FNavPathWeakPtr* WeakPathPtr = &ActivePaths[PathIndex];
		if (WeakPathPtr->IsValid() == false)
		{
			ActivePaths.RemoveAtSwap(PathIndex, EAllowShrinking::No);
		}
	}
}

void ANavigationData::RegisterActivePath(FNavPathSharedPtr SharedPath)
{
	// Paths can be registered from main thread and async pathfinding thread
	UE::TScopeLock PathLock(ActivePathsLock);
	ActivePaths.Add(SharedPath);
}

#if WITH_EDITOR
void ANavigationData::PostEditUndo()
{
	// make sure that rendering component is not pending kill before trying to register all components
	InstantiateAndRegisterRenderingComponent();

	Super::PostEditUndo();

	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	if (NavSys)
	{
		if (IsPendingKillPending())
		{
			NavSys->UnregisterNavData(this);
		}
		else
		{
			NavSys->RequestRegistrationDeferred(*this);
		}
	}
}
#endif // WITH_EDITOR

bool ANavigationData::DoesSupportAgent(const FNavAgentProperties& AgentProps) const
{
	return NavDataConfig.IsEquivalent(AgentProps);
}

void ANavigationData::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	UnregisterAndCleanUp();
	Super::EndPlay(EndPlayReason);
}

void ANavigationData::Destroyed()
{
	UnregisterAndCleanUp();
	Super::Destroyed();
}
void ANavigationData::UnregisterAndCleanUp()
{
	if (bRegistered)
	{
		bRegistered = false;
		UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
		if (NavSys)
		{
			NavSys->UnregisterNavData(this);
		}
	}

	// Cleanup is not tied to the registration state.
	CleanUp();
}

void ANavigationData::CleanUp()
{
	bRegistered = false;
}

void ANavigationData::ApplyWorldOffset(const FVector& InOffset, bool bWorldShift)
{
	// do nothing, will be handled by NavigationSystem
}

void ANavigationData::DestroyActor()
{
	UWorld* World = GetWorld();
	if (World && World->IsEditorWorld() && !World->IsPlayInEditor())
	{
		// Editor-aware destroy (notifies editor, handles actors)
		World->EditorDestroyActor(this, /*bShouldModifyLevel*/true);
	}
	else if (World)
	{
		// Runtime destroy
		Destroy(/*bNetForce*/false, /*bShouldModifyLevel*/false);
	}
}

void ANavigationData::CleanUpAndMarkPendingKill()
{
	CleanUp();

	/* Need to check if the world is valid since, when this is called from Serialize, the World won't be set and Destroy will do nothing, 
	 * in which case it will crash when it tries to register with the NavSystem in UNavigationSystemV1::ProcessRegistrationCandidates. */
	if (UE::Navigation::Private::bDestroyNavDataInCleanUpAndMarkPendingKill && IsValid(GetWorld()))
	{
		Destroy();
	}
	else
	{
		SetActorHiddenInGame(true);

		if (UWorld* World = GetWorld())
		{
			// This part is not thread-safe and should only happen on the GT...
			check(IsInGameThread());
			World->RemoveNetworkActor(this);
		}
		MarkAsGarbage();
		MarkComponentsAsGarbage();
	}
}

bool ANavigationData::SupportsRuntimeGeneration() const
{
	return false;
}

bool ANavigationData::SupportsStreaming() const
{
	return false;
}

void ANavigationData::ConditionalConstructGenerator()
{
}

void ANavigationData::RebuildAll()
{
	const double LoadTime = FPlatformTime::Seconds();

	LoadBeforeGeneratorRebuild();
	FAssetCompilingManager::Get().FinishAllCompilation();
	
	if (UWorld* World = GetWorld())
	{
		UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);
		if (NavSys)
		{
			// This used to be in UNavigationSystemV1::RebuildAll but since we just loaded new assets it needs to be done here.
			NavSys->FlushPendingOperations();
		}
	}
	
	UE_LOGF(LogNavigationDataBuild, Display, "   %ls load time: %.2fs", ANSI_TO_TCHAR(__FUNCTION__), (FPlatformTime::Seconds() - LoadTime));

	PostLoadPreRebuild();
	
	ConditionalConstructGenerator(); //recreate generator
	
	if (NavDataGenerator.IsValid())
	{
#if WITH_EDITOR		
		if (!IsBuildingOnLoad())
		{
			MarkPackageDirty();
		}
#endif // WITH_EDITOR

		NavDataGenerator->RebuildAll();
	}
}

void ANavigationData::EnsureBuildCompletion()
{
	if (NavDataGenerator.IsValid())
	{
		NavDataGenerator->EnsureBuildCompletion();
	}
}

void ANavigationData::CancelBuild()
{
	if (NavDataGenerator.IsValid())
	{
		NavDataGenerator->CancelBuild();
	}
}

void ANavigationData::OnNavigationBoundsChanged()
{
	UE_LOGF(LogNavigation, Log, "%s %ls", __FUNCTION__, *GetNameSafe(this));
	
	// Create generator if it wasn't yet
	if (NavDataGenerator.Get() == nullptr)
	{
		ConditionalConstructGenerator();
	}
	
	if (NavDataGenerator.IsValid())
	{
		NavDataGenerator->OnNavigationBoundsChanged();
	}
}

void ANavigationData::TickAsyncBuild(float DeltaSeconds)
{
	if (NavDataGenerator.IsValid())
	{
		NavDataGenerator->TickAsyncBuild(DeltaSeconds);
	}
}

void ANavigationData::RebuildDirtyAreas(const TArray<FNavigationDirtyArea>& DirtyAreas)
{
	if (bRebuildingSuspended)
	{
		if (bReachedSuspendedAreasMaxCount)
		{
			return; // Already reached max count
		}

		const int32 TotalDirtyAreasCount = SuspendedDirtyAreas.Num() + DirtyAreas.Num();
		if (TotalDirtyAreasCount > UE::Navigation::Private::MaxDirtyAreasCountWhileSuspended)
		{
			// Rebuilding has been suspended for too long and we're accumulating too many dirty areas.
			// Let's just stop accumulating them and remember that the whole navigation data might now be invalid.
			UE_VLOG_UELOG(this, LogNavigation, Display, TEXT("%s reached ai.navigation.MaxDirtyAreasCountWhileSuspended (%d). Ignoring dirty areas until rebuilding is unsuspended."),
				*GetName(),
				UE::Navigation::Private::MaxDirtyAreasCountWhileSuspended);

			bReachedSuspendedAreasMaxCount = true;
			SuspendedDirtyAreas.Empty();
		}
		else
		{
			SuspendedDirtyAreas.Append(DirtyAreas);
		}
	}
	else
	{
		if (NavDataGenerator.IsValid())
		{
			NavDataGenerator->RebuildDirtyAreas(DirtyAreas);
		}
	}
}

void ANavigationData::SetRebuildingSuspended(const bool bNewSuspend)
{
	if (bRebuildingSuspended != bNewSuspend)
	{
		bRebuildingSuspended = bNewSuspend;
		UE_VLOG_UELOG(this, LogNavigation, Verbose, TEXT("%s nav generation %s")
			, *GetName(), bNewSuspend ? TEXT("SUSPENDED") : TEXT("ACTIVE"));

		if (bNewSuspend == false)
		{
			if (bReachedSuspendedAreasMaxCount)
			{
				UE_VLOG_UELOG(this, LogNavigation, Warning, TEXT("%s resuming nav generation after MaxDirtyAreasCountWhileSuspended was reached. This will rebuild the entire navigation data."), *GetName());

				// We stopped tracking dirty areas so we'll need to rebuild all.
				bReachedSuspendedAreasMaxCount = false;
				RebuildAll();
			}
			else if (SuspendedDirtyAreas.Num() > 0)
			{
				UE_VLOG_UELOG(this, LogNavigation, Verbose, TEXT("%s resuming nav generation with %d dirty areas")
					, *GetName(), SuspendedDirtyAreas.Num());

				// resuming the generation so we need to utilize SuspendedDirtyAreas and clean it
				RebuildDirtyAreas(SuspendedDirtyAreas);
				SuspendedDirtyAreas.Empty();
			}
		}
	}
}

TArray<FBox> ANavigationData::GetNavigableBounds() const
{
	TArray<FBox> Result;
	const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<const UNavigationSystemV1>(GetWorld());
	
	if (NavSys)
	{
		// @note this has been switched over from calling GetNavigationBounds
		// to get navigable bounds relevant to this one nav data instance
		// This implements the original intension of the function
		NavSys->GetNavigationBoundsForNavData(*this, Result);
	}
	
	return Result;
}
	
TArray<FBox> ANavigationData::GetNavigableBoundsInLevel(ULevel* InLevel) const
{
	TArray<FBox> Result;
	const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<const UNavigationSystemV1>(GetWorld());

	if (NavSys)
	{
		NavSys->GetNavigationBoundsForNavData(*this, Result, InLevel);
	}

	return Result;
}

void ANavigationData::DrawDebugPath(FNavigationPath* Path, const FColor PathColor, UCanvas* Canvas, const bool bPersistent, const float LifeTime, const uint32 NextPathPointIndex) const
{
	Path->DebugDraw(this, PathColor, Canvas, bPersistent, LifeTime, NextPathPointIndex);
}

double ANavigationData::GetWorldTimeStamp() const
{
	const UWorld* World = GetWorld();
	return World ? World->GetTimeSeconds() : 0.f;
}

void ANavigationData::OnNavAreaAdded(const UClass* NavAreaClass, int32 AgentIndex)
{
	// check if area can be added
	const UNavArea* DefArea = GetDefault<UNavArea>(const_cast<UClass*>(NavAreaClass));
	const bool bIsMetaArea = DefArea != nullptr && DefArea->IsMetaArea();
	if (!DefArea || bIsMetaArea || !DefArea->IsSupportingAgent(AgentIndex))
	{
		UE_VLOG_UELOG(this, LogNavigation, Verbose, TEXT("%s discarded area %s (valid:%s meta:%s validAgent[%d]:%s)"),
			*GetName(), *GetNameSafe(NavAreaClass),
			DefArea ? TEXT("yes") : TEXT("NO"),
			bIsMetaArea ? TEXT("YES") : TEXT("no"),
			AgentIndex, (DefArea && DefArea->IsSupportingAgent(AgentIndex)) ? TEXT("yes") : TEXT("NO"));
		return;
	}

	// check if area is already on supported list
	FString AreaClassName = NavAreaClass->GetName();
	for (int32 i = 0; i < SupportedAreas.Num(); i++)
	{
		if (SupportedAreas[i].AreaClassName == AreaClassName)
		{
			SupportedAreas[i].AreaClass = NavAreaClass;
			AreaClassToIdMap.Add(NavAreaClass, SupportedAreas[i].AreaID);
			UE_VLOG_UELOG(this, LogNavigation, VeryVerbose, TEXT("%s: updated area %s with ID %d"), *GetFullName(), *AreaClassName, SupportedAreas[i].AreaID);
			return;
		}
	}

	// try adding new one
	const int32 MaxSupported = GetMaxSupportedAreas();
	if (SupportedAreas.Num() >= MaxSupported)
	{
		UE_VLOG_UELOG(this, LogNavigation, Error, TEXT("%s can't support area %s - limit reached! (%d)"), *GetName(), *AreaClassName, MaxSupported);
		return;
	}

	FSupportedAreaData NewAgentData;
	NewAgentData.AreaClass = NavAreaClass;
	NewAgentData.AreaClassName = AreaClassName;
	NewAgentData.AreaID = GetNewAreaID(NavAreaClass);
	SupportedAreas.Add(NewAgentData);
	AreaClassToIdMap.Add(NavAreaClass, NewAgentData.AreaID);

	UE_VLOG_UELOG(this, LogNavigation, VeryVerbose, TEXT("  %s registered area %s with ID %d"), *GetPathName(), *AreaClassName, NewAgentData.AreaID);
}

void ANavigationData::OnNavAreaEvent(const UClass* NavAreaClass, ENavAreaEvent::Type Event)
{
	if (Event == ENavAreaEvent::Registered)
	{
		const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<const UNavigationSystemV1>(GetWorld());
		const int32 AgentIndex = NavSys->GetSupportedAgentIndex(this);

		OnNavAreaAdded(NavAreaClass, AgentIndex);
	}
	else // Unregistered
	{
		OnNavAreaRemoved(NavAreaClass);
	}

	OnNavAreaChanged();
}

void ANavigationData::OnNavAreaRemoved(const UClass* NavAreaClass)
{
	for (int32 i = 0; i < SupportedAreas.Num(); i++)
	{
		if (SupportedAreas[i].AreaClass == NavAreaClass)
		{
			AreaClassToIdMap.Remove(NavAreaClass);
			SupportedAreas.RemoveAt(i);
			break;
		}
	}
}

void ANavigationData::OnNavAreaChanged()
{
	// empty in base class
}

void ANavigationData::ProcessNavAreas(const TSet<const UClass*>& AreaClasses, int32 AgentIndex)
{
	for (const UClass* AreaClass : AreaClasses)
	{
		OnNavAreaAdded(AreaClass, AgentIndex);
	}

	OnNavAreaChanged();
}

int32 ANavigationData::GetNewAreaID(const UClass* AreaClass) const
{
	int TestId = 0;
	while (TestId < SupportedAreas.Num())
	{
		const bool bIsTaken = IsAreaAssigned(TestId);
		if (bIsTaken)
		{
			TestId++;
		}
		else
		{
			break;
		}
	}

	return TestId;
}

const UClass* ANavigationData::GetAreaClass(int32 AreaID) const
{
	for (int32 i = 0; i < SupportedAreas.Num(); i++)
	{
		if (SupportedAreas[i].AreaID == AreaID)
		{
			return SupportedAreas[i].AreaClass;
		}
	}

	return NULL;
}

bool ANavigationData::IsAreaAssigned(int32 AreaID) const
{
	for (int32 i = 0; i < SupportedAreas.Num(); i++)
	{
		if (SupportedAreas[i].AreaID == AreaID)
		{
			return true;
		}
	}

	return false;
}

int32 ANavigationData::GetAreaID(const UClass* AreaClass) const
{
	const int32* PtrId = AreaClassToIdMap.Find(AreaClass);
	return PtrId ? *PtrId : INDEX_NONE;
}

void ANavigationData::SetNavRenderingEnabled(bool bEnable)
{
	if (bEnableDrawing != bEnable)
	{
		bEnableDrawing = bEnable;
		MarkComponentsRenderStateDirty();
	}
}

void ANavigationData::UpdateCustomLink(const INavLinkCustomInterface* CustomLink)
{
	// no implementation for abstract class
}

FSharedConstNavQueryFilter ANavigationData::GetQueryFilter(TSubclassOf<UNavigationQueryFilter> FilterClass) const
{
	return QueryFilters.FindRef(FilterClass);
}

void ANavigationData::StoreQueryFilter(TSubclassOf<UNavigationQueryFilter> FilterClass, FSharedConstNavQueryFilter NavFilter)
{
	QueryFilters.Add(FilterClass, NavFilter);
}

void ANavigationData::RemoveQueryFilter(TSubclassOf<UNavigationQueryFilter> FilterClass)
{
	QueryFilters.Remove(FilterClass);
}

uint32 ANavigationData::LogMemUsed() const
{
	SIZE_T ActivePathsMemSize = 0;
	{
		// Paths can be registered from async pathfinding thread
		// while logging is requested on main thread (console command)
		UE::TScopeLock PathLock(ActivePathsLock);
		ActivePathsMemSize = ActivePaths.GetAllocatedSize();
	}

	const uint32 MemUsed = IntCastChecked<uint32>(ActivePathsMemSize + SupportedAreas.GetAllocatedSize() +
		QueryFilters.GetAllocatedSize() + AreaClassToIdMap.GetAllocatedSize());

	UE_VLOG_UELOG(this, LogNavigation, Display, TEXT("%s: ANavigationData: %u\n    self: %zu"), *GetName(), MemUsed, sizeof(ANavigationData));

	if (NavDataGenerator.IsValid())
	{
		NavDataGenerator->LogMemUsed();
	}

	return MemUsed;
}

void ANavigationData::SetConfig(const FNavDataConfig& Src)
{
	SetNavAgentProperties(Src);
	NavDataConfig = Src;
}