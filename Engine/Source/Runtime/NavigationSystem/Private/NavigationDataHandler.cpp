// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationDataHandler.h"
#include "AI/Navigation/NavigationDirtyElement.h"
#include "AI/Navigation/NavRelevantInterface.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"
#include "NavAreas/NavArea.h"
#include "NavMesh/RecastGeometryExport.h"
#include "VisualLogger/VisualLogger.h"

#if WITH_RECAST
#include "DetourCrowd/DetourCrowd.h"
#endif // WITH_RECAST

DEFINE_LOG_CATEGORY_STATIC(LogNavOctree, Warning, All);

namespace UE::NavigationHelper::Private
{
	ENavigationDirtyFlag GetDirtyFlag(const int32 UpdateFlags, const ENavigationDirtyFlag DefaultValue)
	{
		return ((UpdateFlags & FNavigationOctreeController::OctreeUpdate_Geometry) != 0) ? ENavigationDirtyFlag::All :
			((UpdateFlags & FNavigationOctreeController::OctreeUpdate_Modifiers) != 0) ? ENavigationDirtyFlag::DynamicModifier :
			DefaultValue;
	}
}
	
FNavigationDataHandler::FNavigationDataHandler(FNavigationOctreeController& InOctreeController, FNavigationDirtyAreasController& InDirtyAreasController)
		: OctreeController(InOctreeController), DirtyAreasController(InDirtyAreasController)
{}

void FNavigationDataHandler::ConstructNavOctree(const FVector& Origin, const double Radius, const ENavDataGatheringModeConfig DataGatheringMode, const float GatheringNavModifiersWarningLimitTime)
{
	UE_LOGF(LogNavOctree, Log, "CREATE (Origin:%ls Radius:%.2f)", *Origin.ToString(), Radius);

	OctreeController.Reset();
	OctreeController.NavOctree = MakeShareable(new FNavigationOctree(Origin, Radius));
	OctreeController.NavOctree->SetDataGatheringMode(DataGatheringMode);
#if !UE_BUILD_SHIPPING
	OctreeController.NavOctree->SetGatheringNavModifiersTimeLimitWarning(GatheringNavModifiersWarningLimitTime);
#endif // !UE_BUILD_SHIPPING
}

void FNavigationDataHandler::RemoveFromNavOctree(const FOctreeElementId2& ElementId, const int32 UpdateFlags)
{
	if (ensure(OctreeController.IsValidElement(ElementId)))
	{
		const FNavigationOctreeElement& ElementData = OctreeController.NavOctree->GetElementById(ElementId);
		// mark area occupied by given element as dirty except if explicitly set to skip this default behavior
		if (!ElementData.Data->bShouldSkipDirtyAreaOnAddOrRemove)
		{
			const ENavigationDirtyFlag DirtyFlag = UE::NavigationHelper::Private::GetDirtyFlag(UpdateFlags, ElementData.Data->GetDirtyFlag());
			DirtyAreasController.AddArea(
				ElementData.Bounds.GetBox(),
				DirtyFlag,
				[&ElementData]
				{
					return ElementData.Data->SourceElement;
				},
				/*DirtyElement*/nullptr,
				"Remove from navoctree");
		}

		OctreeController.RemoveNode(ElementId, ElementData.Data.Get().SourceElement.Get().GetHandle());
	}
}

FSetElementId FNavigationDataHandler::RegisterElementWithNavOctree(const TSharedRef<const FNavigationElement>& ElementRef, const int32 UpdateFlags)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RegisterNavOctreeElement);

	FSetElementId SetId;
	const FNavigationElement& NavigationElement = ElementRef.Get();

	if (OctreeController.IsValid() == false)
	{
		UE_LOGF(LogNavOctree, VeryVerbose, "IGNORE(%s) %ls: octree not created yet",
			__FUNCTION__, *NavigationElement.GetPathName());
		return SetId;
	}

	if (OctreeController.IsNavigationOctreeLocked())
	{
		UE_LOGF(LogNavOctree, Log, "IGNORE(%s) %ls: navigation octree locked",
			__FUNCTION__, *NavigationElement.GetPathName());
		return SetId;
	}

	UE_LOGF(LogNavOctree, Log, "REG %ls", *NavigationElement.GetPathName());

	bool bCanAdd = false;
	if (const TWeakObjectPtr<const UObject>& NavigationParent = NavigationElement.GetNavigationParent(); !NavigationParent.IsExplicitlyNull())
	{
		OctreeController.AddChild(FNavigationElementHandle(NavigationParent), ElementRef);
		bCanAdd = true;
	}
	else
	{
		bCanAdd = (OctreeController.HasElementNavOctreeId(NavigationElement.GetHandle()) == false);
	}

	if (bCanAdd)
	{
		FNavigationDirtyElement UpdateInfo(ElementRef, UE::NavigationHelper::Private::GetDirtyFlag(UpdateFlags, ENavigationDirtyFlag::None), DirtyAreasController.bUseWorldPartitionedDynamicMode);

		SetId = OctreeController.PendingUpdates.FindId(NavigationElement.GetHandle());
		if (SetId.IsValidId())
		{
			// make sure this request stays, in case it has been invalidated already and keep any dirty areas
			UpdateInfo.ExplicitAreasToDirty = OctreeController.PendingUpdates[SetId].ExplicitAreasToDirty;
			OctreeController.PendingUpdates[SetId] = UpdateInfo;
		}
		else
		{
			SetId = OctreeController.PendingUpdates.Add(UpdateInfo);
		}
	}

	return SetId;
}

void FNavigationDataHandler::AddElementToNavOctree(const FNavigationDirtyElement& DirtyElement)
{
	check(OctreeController.IsValid());
	LLM_SCOPE_BYTAG(NavigationOctree);

	// handle invalidated requests first
	if (DirtyElement.bInvalidRequest)
	{
		if (DirtyElement.bHasPrevData)
		{
			DirtyAreasController.AddArea(DirtyElement.PrevBounds,
				DirtyElement.PrevFlags,
				[&DirtyElement]
				{
					return DirtyElement.NavigationElement;
				},
				&DirtyElement,
				"Addition to navoctree (invalid request)");
		}

		return;
	}

	FNavigationOctreeElement OctreeElement(DirtyElement.NavigationElement);
	const FNavigationElement& NavigationElement = DirtyElement.NavigationElement.Get();

	const TWeakObjectPtr<const UObject> ElementWeakUObject = NavigationElement.GetWeakUObject();
	if (!ElementWeakUObject.IsExplicitlyNull())
	{
		UE_VLOG_UELOG(ElementWeakUObject.Get(), LogNavOctree, Verbose, TEXT("Create FNavigationOctreeElement for %s"),
			*NavigationElement.GetPathName());
	}

	// In WP dynamic mode, store if this is loaded data.
	if (DirtyAreasController.bUseWorldPartitionedDynamicMode)
	{
		OctreeElement.Data->bLoadedData = DirtyElement.bIsFromVisibilityChange || NavigationElement.IsFromLevelVisibilityChange();
	}

	const FBox ElementBounds = NavigationElement.GetBounds();

	if (const TWeakObjectPtr<const UObject>& NavigationParent = NavigationElement.GetNavigationParent(); !NavigationParent.IsExplicitlyNull())
	{
		const FNavigationElementHandle ParentKey(NavigationParent);

		// check if parent node is waiting in queue
		const FSetElementId ParentRequestId = OctreeController.PendingUpdates.FindId(ParentKey);
		const FOctreeElementId2* ParentId = OctreeController.GetNavOctreeIdForElement(ParentKey);
		if (ParentRequestId.IsValidId() && ParentId == nullptr)
		{
			FNavigationDirtyElement& ParentDirtyElement = OctreeController.PendingUpdates[ParentRequestId];
			AddElementToNavOctree(ParentDirtyElement);

			// mark as invalid so it won't be processed twice
			ParentDirtyElement.bInvalidRequest = true;
		}

		const FOctreeElementId2* ElementId = ParentId ? ParentId : OctreeController.GetNavOctreeIdForElement(ParentKey);
		if (ElementId && ensure(OctreeController.IsValidElement(*ElementId)))
		{
			UE_LOGF(LogNavOctree, Log, "ADD %ls to %ls", *NavigationElement.GetPathName(), *GetNameSafe(NavigationParent.Get()));
			OctreeController.NavOctree->AppendToNode(*ElementId, DirtyElement.NavigationElement, ElementBounds, OctreeElement);
		}
		else
		{
			UE_LOGF(LogNavOctree, Warning, "Can't add node [%ls] - parent [%ls] not found in octree!",
				*NavigationElement.GetPathName(),
				*GetNameSafe(NavigationParent.Get()));
		}
	}
	else
	{
		OctreeController.NavOctree->AddNode(ElementBounds, OctreeElement);
		UE_SUPPRESS(LogNavOctree, Verbose,
			{
				const FOctreeElementId2* ElementId = OctreeController.GetNavOctreeIdForElement(DirtyElement.NavigationElement->GetHandle());
				UE_VLOG_UELOG(NavigationElement.GetWeakUObject().Get(), LogNavOctree, Log, TEXT("ADD %s - %s"),
					*NavigationElement.GetPathName(),
					ElementId ? *LexToString(*ElementId) : TEXT("No element"));
			});
	}

	// mark area occupied by given element as dirty except if explicitly set to skip this default behavior
	const ENavigationDirtyFlag DirtyFlag = DirtyElement.FlagsOverride != ENavigationDirtyFlag::None ? DirtyElement.FlagsOverride : OctreeElement.Data->GetDirtyFlag();
	if (OctreeElement.Data->bShouldSkipDirtyAreaOnAddOrRemove)
	{
		if (DirtyElement.ExplicitAreasToDirty.Num() > 0)
		{
			DirtyAreasController.AddAreas(
				DirtyElement.ExplicitAreasToDirty,
				DirtyFlag, 
				[ElementOwner = DirtyElement.NavigationElement]
				{
					return ElementOwner;
				},
				&DirtyElement,
				"Addition to navoctree");
		}
	}
	else if (!OctreeElement.IsEmpty())
	{
		DirtyAreasController.AddArea(
			OctreeElement.Bounds.GetBox(),
			DirtyFlag,
			[ElementOwner = DirtyElement.NavigationElement]
			{
				return ElementOwner;
			},
			&DirtyElement,
			"Addition to navoctree");
	}
}

bool FNavigationDataHandler::UnregisterElementWithNavOctree(const TSharedRef<const FNavigationElement>& ElementRef, const int32 UpdateFlags)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_UnregisterNavOctreeElement);

	const FNavigationElement& NavRelevantElement = ElementRef.Get();
	if (OctreeController.IsValid() == false)
	{
		UE_LOGF(LogNavOctree, VeryVerbose, "IGNORE(%s) %ls: octree not created yet",
			__FUNCTION__, *NavRelevantElement.GetPathName());
		return false;
	}

	if (OctreeController.IsNavigationOctreeLocked())
	{
		UE_LOGF(LogNavOctree, Log, "IGNORE(%s) %ls: octree locked",
			__FUNCTION__, *ElementRef.Get().GetPathName());
		return false;
	}

	const FNavigationElementHandle NavRelevantElementHandle = NavRelevantElement.GetHandle();

	const FOctreeElementId2* OctreeElementId = OctreeController.GetNavOctreeIdForElement(NavRelevantElementHandle);
	UE_VLOG_UELOG(NavRelevantElement.GetWeakUObject().Get(), LogNavOctree, Log, TEXT("UNREG %s %s"),
		*NavRelevantElement.GetPathName(),
		OctreeElementId ? *FString::Printf(TEXT("[exists %s]"), *LexToString(*OctreeElementId)) : TEXT("[doesn\'t exist]"));

	bool bUnregistered = false;

	if (OctreeElementId != nullptr)
	{
		RemoveFromNavOctree(*OctreeElementId, UpdateFlags);
		bUnregistered = true;
	}
	else if (const bool bCanRemoveChildNode = (UpdateFlags & FNavigationOctreeController::OctreeUpdate_ParentChain) == 0)
	{
		if (const TWeakObjectPtr<const UObject>& NavigationParent = NavRelevantElement.GetNavigationParent(); !NavigationParent.IsExplicitlyNull())
		{
			// if node has navigation parent (= doesn't exist in octree on its own)
			// and it's not part of parent chain update
			// remove it from map and force update on parent to rebuild octree element
			const FNavigationElementHandle ParentKey(NavigationParent);
			OctreeController.RemoveChild(ParentKey, ElementRef);

			if (const FNavigationRelevantData* NavigationData = OctreeController.GetDataForElement(ParentKey))
			{
				UpdateNavOctreeParentChain(NavigationData->SourceElement);
			}
		}
	}

	// mark pending update as invalid, it will be dirtied according to currently active settings
	if (const bool bCanInvalidateQueue = (UpdateFlags & FNavigationOctreeController::OctreeUpdate_Refresh) == 0)
	{
		const FSetElementId RequestId = OctreeController.PendingUpdates.FindId(NavRelevantElementHandle);
		if (RequestId.IsValidId())
		{
			FNavigationDirtyElement& DirtyElement = OctreeController.PendingUpdates[RequestId];

			// Only consider as unregistered when pending update was not already invalidated since return value must indicate
			// that ElementOwner was fully added or about to be added (valid pending update).
			bUnregistered |= (DirtyElement.bInvalidRequest == false);

			DirtyElement.bInvalidRequest = true;
		}
	}

	return bUnregistered;
}

void FNavigationDataHandler::UpdateNavOctreeElement(FNavigationElementHandle ElementHandle, const TSharedRef<const FNavigationElement>& UpdatedElement, int32 UpdateFlags)
{
	INC_DWORD_STAT(STAT_Navigation_UpdateNavOctree);

	if (OctreeController.IsValid() == false)
	{
		UE_LOGF(LogNavOctree, VeryVerbose, "IGNORE(%s) %ls: octree not created yet",
			__FUNCTION__, *UpdatedElement.Get().GetPathName());
		return;
	}

	if (OctreeController.IsNavigationOctreeLocked())
	{
		UE_LOGF(LogNavOctree, Log, "IGNORE(%s) %ls: octree locked",
			__FUNCTION__, *UpdatedElement.Get().GetPathName());
		return;
	}

	// grab existing octree data
	FBox CurrentBounds;
	ENavigationDirtyFlag CurrentFlags;
	const bool bAlreadyExists = OctreeController.GetNavOctreeElementData(ElementHandle, CurrentFlags, CurrentBounds);

	// don't invalidate pending requests
	UpdateFlags |= FNavigationOctreeController::OctreeUpdate_Refresh;

	// Use local shared reference to make sure to keep element alive to register back
	// since unregistering might remove the only reference.
	TSharedRef<const FNavigationElement> LocalElementRef = UpdatedElement;

	// Always try to unregister, even if element owner doesn't exist in octree (parent nodes).
	// This is also why we need to provide the new element and not only the handle, so we can access the parent (expected to be always the same for an update).
	UnregisterElementWithNavOctree(LocalElementRef, UpdateFlags);

	const FSetElementId RequestId = RegisterElementWithNavOctree(LocalElementRef, UpdateFlags);

	// add original data to pending registration request
	// so it could be dirtied properly when system receive unregister request while actor is still queued
	if (RequestId.IsValidId())
	{
		FNavigationDirtyElement& UpdateInfo = OctreeController.PendingUpdates[RequestId];
		UpdateInfo.PrevFlags = CurrentFlags;
		if (UpdateInfo.PrevBounds.IsValid)
		{
			// If we have something stored already we want to 
			// sum it up, since we care about the whole bounding
			// box of changes that potentially took place
			UpdateInfo.PrevBounds += CurrentBounds;
		}
		else
		{
			UpdateInfo.PrevBounds = CurrentBounds;
		}
		UpdateInfo.bHasPrevData = bAlreadyExists;
	}

	UpdateNavOctreeParentChain(UpdatedElement, /*bSkipElementOwnerUpdate=*/ true);
}

void FNavigationDataHandler::UpdateNavOctreeParentChain(const TSharedRef<const FNavigationElement>& Element, const bool bSkipElementOwnerUpdate)
{
	constexpr int32 UpdateFlags = FNavigationOctreeController::OctreeUpdate_ParentChain | FNavigationOctreeController::OctreeUpdate_Refresh;

	TArray<const TSharedRef<const FNavigationElement>> ChildNodes;
	OctreeController.GetChildren(Element->GetHandle(), ChildNodes);

	auto ElementOwnerUpdateFunc = [&]()->bool
	{
		bool bShouldRegisterChildren = true;
		if (bSkipElementOwnerUpdate == false)
		{
			// Use local shared reference to make sure to keep element alive to register back
			// since unregistering might remove the only reference.
			TSharedRef<const FNavigationElement> LocalElementRef = Element;

			// We don't want to register NavOctreeElement if owner was not already registered or queued
			// so we use Unregister/Register combo instead of UpdateNavOctreeElement
			if (UnregisterElementWithNavOctree(LocalElementRef, UpdateFlags))
			{
				const FSetElementId NewId = RegisterElementWithNavOctree(LocalElementRef, UpdateFlags);
				bShouldRegisterChildren = NewId.IsValidId();
			}
			else
			{
				bShouldRegisterChildren = false;
			}
		}
		return bShouldRegisterChildren;
	};

	if (ChildNodes.Num() == 0)
	{
		// Last child was removed, only need to rebuild owner's NavOctreeElement
		ElementOwnerUpdateFunc();
		return;
	}

	for (const TSharedRef<const FNavigationElement>& ChildNode : ChildNodes)
	{
		UnregisterElementWithNavOctree(ChildNode, UpdateFlags);
	}

	if (const bool bShouldRegisterChildren = ElementOwnerUpdateFunc())
	{
		for (const TSharedRef<const FNavigationElement>& ChildNode : ChildNodes)
		{
			RegisterElementWithNavOctree(ChildNode, UpdateFlags);
		}
	}
}

bool FNavigationDataHandler::UpdateNavOctreeElementBounds(const FNavigationElementHandle ElementHandle, const FBox& NewBounds, const TConstArrayView<FBox> DirtyAreas)
{
	const FOctreeElementId2* ElementId = OctreeController.GetNavOctreeIdForElement(ElementHandle);
	if (ElementId != nullptr && ensure(OctreeController.IsValidElement(*ElementId)))
	{
		OctreeController.NavOctree->UpdateNode(*ElementId, NewBounds);

		// Dirty areas
		if (DirtyAreas.Num() > 0)
		{
			// Refresh ElementId since object may be stored in a different node after updating bounds
			ElementId = OctreeController.GetNavOctreeIdForElement(ElementHandle);
			if (ElementId != nullptr && ensure(OctreeController.IsValidElement(*ElementId)))
			{
				const FNavigationOctreeElement& ElementData = OctreeController.NavOctree->GetElementById(*ElementId);
				DirtyAreasController.AddAreas(
					DirtyAreas,
					ElementData.Data->GetDirtyFlag(),
					[SourceElement = ElementData.Data->SourceElement]
					{
						return SourceElement;
					},
					/*DirtyElement*/ nullptr,
					"Bounds change");
			}
		}

		return true;
	}

	// Update bounds and to append dirty areas to a pending update since the element is not added yet.
	if (const FSetElementId PendingElementId = OctreeController.PendingUpdates.FindId(ElementHandle); PendingElementId.IsValidId())
	{
		if (FNavigationDirtyElement& DirtyElement = OctreeController.PendingUpdates[PendingElementId]; !DirtyElement.bInvalidRequest)
		{
			const TSharedRef<FNavigationElement> Updated = MakeShared<FNavigationElement>(DirtyElement.NavigationElement.Get());
			Updated->SetBounds(NewBounds);
			DirtyElement.NavigationElement = Updated;
			DirtyElement.ExplicitAreasToDirty.Append(DirtyAreas);
			return true;
		}
	}

	return false;
}

void FNavigationDataHandler::FindElementsInNavOctree(const FBox& QueryBox, const FNavigationOctreeFilter& Filter, TArray<FNavigationOctreeElement>& Elements)
{
	if (OctreeController.IsValid() == false)
	{
		UE_LOGF(LogNavOctree, Warning, "FNavigationDataHandler::FindElementsInNavOctree gets called while NavOctree is null");
		return;
	}

	OctreeController.NavOctree->FindElementsWithBoundsTest(QueryBox, [&Elements, &Filter](const FNavigationOctreeElement& Element)
	{
		if (Element.IsMatchingFilter(Filter))
		{
			Elements.Add(Element);
		}
	});
}

bool FNavigationDataHandler::ReplaceAreaInOctreeData(
	const FNavigationElementHandle Element,
	const TSubclassOf<UNavArea> OldArea,
	const TSubclassOf<UNavArea> NewArea,
	const bool bReplaceChildClasses) const
{
	FNavigationRelevantData* Data = OctreeController.GetMutableDataForElement(Element);

	if (Data == nullptr || Data->HasModifiers() == false)
	{
		return false;
	}

	for (FAreaNavModifier& AreaModifier : Data->Modifiers.GetMutableAreas())
	{
		if (AreaModifier.GetAreaClass() == OldArea
			|| (bReplaceChildClasses && AreaModifier.GetAreaClass()->IsChildOf(OldArea)))
		{
			AreaModifier.SetAreaClass(NewArea);
		}
	}
	for (FSimpleLinkNavModifier& SimpleLink : Data->Modifiers.GetSimpleLinks())
	{
		for (FNavigationLink& Link : SimpleLink.Links)
		{
			if (Link.GetAreaClass() == OldArea
				|| (bReplaceChildClasses && Link.GetAreaClass()->IsChildOf(OldArea)))
			{
				Link.SetAreaClass(NewArea);
			}
		}
		for (FNavigationSegmentLink& Link : SimpleLink.SegmentLinks)
		{
			if (Link.GetAreaClass() == OldArea
				|| (bReplaceChildClasses && Link.GetAreaClass()->IsChildOf(OldArea)))
			{
				Link.SetAreaClass(NewArea);
			}
		}
	}

	ensureMsgf(Data->Modifiers.GetCustomLinks().IsEmpty(), TEXT("Not implemented yet"));

	return true;
}

void FNavigationDataHandler::AddLevelCollisionToOctree(ULevel& Level)
{
#if WITH_RECAST
	if (OctreeController.IsValid() &&
		OctreeController.NavOctree->GetNavGeometryStoringMode() == FNavigationOctree::StoreNavGeometry)
	{
		const FNavigationElementHandle ElementKey(&Level);
		const TArray<FVector>* LevelGeom = Level.GetStaticNavigableGeometry();
		const FOctreeElementId2* ElementId = OctreeController.GetNavOctreeIdForElement(ElementKey);

		if (!ElementId && LevelGeom && LevelGeom->Num() > 0)
		{
			TSharedRef<const FNavigationElement> NavigationElement = MakeShared<const FNavigationElement>(Level, INDEX_NONE);
			FNavigationOctreeElement BSPElem(NavigationElement);

			// In WP dynamic mode, store if this is loaded data.
			if (DirtyAreasController.bUseWorldPartitionedDynamicMode)
			{
				BSPElem.Data->bLoadedData = Level.HasVisibilityChangeRequestPending();
			}
			
			FRecastGeometryExport::ExportVertexSoupGeometry(*LevelGeom, *BSPElem.Data);

			const FBox& Bounds = BSPElem.Data->Bounds;
			if (!Bounds.GetExtent().IsNearlyZero())
			{
				OctreeController.NavOctree->AddNode(Bounds, BSPElem);
				DirtyAreasController.AddArea(
					Bounds, 
					ENavigationDirtyFlag::All,
					[SourceElement = NavigationElement]
					{
						return SourceElement;
					},
					/*DirtyElement*/ nullptr,
					"Add level");

				UE_LOGF(LogNavOctree, Log, "ADD %ls", *NavigationElement.Get().GetPathName());
			}
		}
	}
#endif// WITH_RECAST
}

void FNavigationDataHandler::RemoveLevelCollisionFromOctree(ULevel& Level)
{
	if (OctreeController.IsValid())
	{
		const FNavigationElementHandle NavigationElementHandle(&Level);
		if (const FOctreeElementId2* OctreeElementId = OctreeController.GetNavOctreeIdForElement(NavigationElementHandle))
		{
			UE_LOGF(LogNavOctree, Log, "UNREG %ls %ls", *Level.GetPathName(), OctreeElementId ? TEXT("[exists]") : TEXT(""));
			RemoveFromNavOctree(*OctreeElementId, FNavigationOctreeController::OctreeUpdate_Geometry);
		}
	}
}

void FNavigationDataHandler::ProcessPendingOctreeUpdates()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Navigation_ProcessPendingOctreeUpdates);

	if (OctreeController.NavOctree)
	{
		// AddElementToNavOctree (through some of its resulting function calls) modifies PendingUpdates so invalidates the iterators,
		// (via WaitUntilAsyncPropertyReleased() / UpdateComponentInNavOctree() / RegisterElementWithNavOctree()). This means we can't iterate
		// through this set in the normal way. Previously the code iterated through this which also left us open to other potential bugs
		// in that we may have tried to modify elements we had already processed.
		while (TSet<FNavigationDirtyElement, FNavigationDirtyElementKeyFunctions>::TIterator It = OctreeController.PendingUpdates.CreateIterator())
		{
			FNavigationDirtyElement DirtyElement = *It;
			It.RemoveCurrent();
			AddElementToNavOctree(DirtyElement);
		}
	}
	OctreeController.PendingUpdates.Empty(32);
}

void FNavigationDataHandler::DemandLazyDataGathering(FNavigationRelevantData& ElementData)
{
	// Do the lazy gathering on the element
	OctreeController.NavOctree->DemandLazyDataGathering(ElementData);

    // Check if any child asked for some lazy gathering
	if (ElementData.IsPendingChildLazyModifiersGathering())
	{
		TArray<const TSharedRef<const FNavigationElement>> ChildNodes;
		OctreeController.GetChildren(ElementData.SourceElement->GetHandle(), ChildNodes);

		for (const TSharedRef<const FNavigationElement>& ChildNode : ChildNodes)
		{
			OctreeController.NavOctree->DemandChildLazyDataGathering(ElementData, ChildNode.Get());
		}
		ElementData.bPendingChildLazyModifiersGathering = false;
	}
}