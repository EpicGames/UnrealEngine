// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationDirtyAreasController.h"
#include "AI/Navigation/NavigationDirtyArea.h"
#include "AI/Navigation/NavigationDirtyElement.h"
#include "NavigationData.h"
#include "NavigationSystem.h"
#include "VisualLogger/VisualLogger.h"

DEFINE_LOG_CATEGORY(LogNavigationDirtyArea);

//----------------------------------------------------------------------//
// FNavigationDirtyAreasController
//----------------------------------------------------------------------//
FNavigationDirtyAreasController::FNavigationDirtyAreasController()
	: bCanAccumulateDirtyAreas(true)
	, bUseWorldPartitionedDynamicMode(false)
#if !UE_BUILD_SHIPPING
	, bDirtyAreasReportedWhileAccumulationLocked(false)
	, bCanReportOversizedDirtyArea(false)
	, bNavigationBuildLocked(false)
#endif // !UE_BUILD_SHIPPING
{

}

void FNavigationDirtyAreasController::ForceRebuildOnNextTick()
{
	float MinTimeForUpdate = (DirtyAreasUpdateFreq != 0.f ? (1.0f / DirtyAreasUpdateFreq) : 0.f);
	DirtyAreasUpdateTime = FMath::Max(DirtyAreasUpdateTime, MinTimeForUpdate);
}

namespace UE::Navigation::Private
{
	const UNavigationSystemV1* FindNavigationSystem(const TArray<ANavigationData*>& NavDataSet)
	{
		const UNavigationSystemV1* NavSys = nullptr;
		for (const ANavigationData* NavData : NavDataSet)
		{
			if (NavData)
			{
				NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(NavData->GetWorld());
				if (NavSys)
				{
					return NavSys;
				}
			}
		}

		return NavSys;
	}
}

void FNavigationDirtyAreasController::Tick(const float DeltaSeconds, const TArray<ANavigationData*>& NavDataSet, bool bForceRebuilding /*= false*/)
{
	DirtyAreasUpdateTime += DeltaSeconds;
	const bool bCanRebuildNow = bForceRebuilding || (DirtyAreasUpdateFreq != 0.f && DirtyAreasUpdateTime >= (1.0f / DirtyAreasUpdateFreq));

	if (DirtyAreas.Num() > 0 && bCanRebuildNow)
	{
		bool bIsUsingActiveTileGeneration = false;
		TArray<FNavigationDirtyArea> SubAreaArray;
		SubAreaArray.Reserve(DirtyAreas.Num());
		
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_MakingSubAreas);

			// Find the relevant navigation system
			const UNavigationSystemV1* NavSys = UE::Navigation::Private::FindNavigationSystem(NavDataSet);
			
			const TArray<FBox>* SeedsBoundsArrayPtr = nullptr;
			bIsUsingActiveTileGeneration = NavSys && NavSys->IsActiveTilesGenerationEnabled(); 
			if (bIsUsingActiveTileGeneration)
			{
				SeedsBoundsArrayPtr = &NavSys->GetInvokersSeedBounds();
			}

			for (const FNavigationDirtyArea& DirtyArea : DirtyAreas)
			{
				const FBox& AreaBound = DirtyArea.Bounds;
				if (!ensureMsgf(AreaBound.IsValid, TEXT("%hs Attempting to use DirtyArea.Bounds which are not valid. SourceObject: %s"),
					__FUNCTION__, *DirtyArea.GetSourceDescription()))
				{
					continue;
				}

				if (SeedsBoundsArrayPtr != nullptr && SeedsBoundsArrayPtr->Num() > 0)
				{
					for (const FBox& SeedBounds : *SeedsBoundsArrayPtr)
					{
						// Compute sub area bound
						const FBox OverlapBox = AreaBound.Overlap(SeedBounds);
						if (OverlapBox.IsValid)
						{
							SubAreaArray.Emplace(OverlapBox, DirtyArea.Flags, DirtyArea.OptionalSourceElement);
						}
					}
				}
				else
				{
					SubAreaArray.Emplace(DirtyArea);
				}
			}
		}
		
		for (ANavigationData* NavData : NavDataSet)
		{
			if (NavData)
			{
				NavData->RebuildDirtyAreas(bIsUsingActiveTileGeneration ? SubAreaArray : DirtyAreas);
			}
		}

		DirtyAreasUpdateTime = 0.f;
		DirtyAreas.Reset();
	}
}

void FNavigationDirtyAreasController::AddArea(const FBox& NewArea, const ENavigationDirtyFlag Flags, const TFunction<const TSharedPtr<const FNavigationElement>()>& ElementProviderFunc /*= nullptr*/,
	const FNavigationDirtyElement* DirtyElement /*= nullptr*/, const FName& DebugReason /*= NAME_None*/)
{
	AddAreas({NewArea}, Flags, ElementProviderFunc, DirtyElement, DebugReason);
}

void FNavigationDirtyAreasController::AddAreas(const TConstArrayView<FBox> NewAreas, const ENavigationDirtyFlag Flags, const TFunction<const TSharedPtr<const FNavigationElement>()>& ElementProviderFunc, const FNavigationDirtyElement* DirtyElement, const FName& DebugReason)
{
#if !UE_BUILD_SHIPPING
	// always keep track of reported areas even when filtered out by invalid area as long as flags are valid
	bDirtyAreasReportedWhileAccumulationLocked |= (Flags != ENavigationDirtyFlag::None) && !bCanAccumulateDirtyAreas;

	checkf(NewAreas.Num() > 0, TEXT("All callers of this method are expected to provide at least one area."));
#endif // !UE_BUILD_SHIPPING

	const TSharedPtr<const FNavigationElement> SourceElement = ElementProviderFunc ? ElementProviderFunc() : nullptr;
	if (bUseWorldPartitionedDynamicMode)
	{
		// Both conditions must be true to ignore dirtiness.
		//  If it's only a visibility change and it's not in the base navmesh, it's the case created by loading a data layer (dirtiness must be applied)
		//  If there is no visibility change, the change is not from loading/unloading a cell (dirtiness must be applied)
		
		// ElementProviderFunc() is not always providing a valid element.
		if (const bool bIsFromVisibilityChange = (DirtyElement && DirtyElement->bIsFromVisibilityChange) || (SourceElement && SourceElement->IsFromLevelVisibilityChange()))
		{
			// If the area is from the addition or removal of elements caused by level loading/unloading and it's already in the base navmesh ignore the dirtiness.
			if (const bool bIsIncludedInBaseNavmesh = (DirtyElement && DirtyElement->bIsInBaseNavmesh) || (SourceElement && SourceElement->IsInBaseNavigationData()))
			{
				UE_LOGF(LogNavigationDirtyArea, VeryVerbose, "Ignoring dirtyness (visibility changed and in base navmesh). (element: %ls from: %ls)",
					*GetFullNameSafe(SourceElement.Get()), *DebugReason.ToString());
				return;
			}
		}
	}

	if (ShouldSkipObjectPredicate.IsBound())
	{
		const UObject* SourceObject = SourceElement ? SourceElement->GetWeakUObject().Get() : nullptr;
		if (SourceObject && ShouldSkipObjectPredicate.Execute(*SourceObject))
		{
			return;
		}
	}

	int32 NumInvalidBounds = 0;
	int32 NumEmptyBounds = 0;
	for (const FBox& NewArea : NewAreas)
	{
		if (!NewArea.IsValid)
		{
			NumInvalidBounds++;
			continue;
		}

		const FVector2D BoundsSize(NewArea.GetSize());
		if (BoundsSize.IsNearlyZero())
		{
			NumEmptyBounds++;
			continue;
		}

#if !UE_BUILD_SHIPPING
		auto DumpExtraInfo = [SourceElement, DebugReason, BoundsSize, NewArea]()
			{
				const UObject* SourceObject = SourceElement ? SourceElement->GetWeakUObject().Get() : nullptr;
				FString ObjectInfo;
				if (const UObject* ObjectOwner = (SourceObject != nullptr ? SourceObject->GetOuter() : nullptr))
				{
					UE_VLOG_BOX(ObjectOwner, LogNavigationDirtyArea, Log, NewArea, FColor::Red, TEXT(""));
					ObjectInfo = FString::Printf(TEXT(" | Element's owner: %s"), *GetFullNameSafe(ObjectOwner));
				}

				// attempt to find the actor which is/contains SourceObject
				const AActor* Actor = Cast<AActor>(SourceObject);
				{
					const UObject* CurrentObject = SourceObject;
					while (!Actor && CurrentObject)
					{
						CurrentObject = CurrentObject->GetOuter();
						Actor = Cast<AActor>(CurrentObject);
					}
				}
				FString ActorInfo;
				if (Actor)
				{
					// useful to have actor label for those placed in editor
					ActorInfo = FString::Printf(TEXT("| Actor: %s | Actor label: %s"), *GetFullNameSafe(Actor), *Actor->GetActorNameOrLabel());
				}

				return FString::Printf(TEXT("From: %s | Object: %s %s | Bounds: %s %s"),
					*DebugReason.ToString(),
					*GetFullNameSafe(SourceObject),
					*ObjectInfo,
					*BoundsSize.ToString(),
					*ActorInfo);
			};

		if (ShouldReportOversizedDirtyArea() && BoundsSize.GetMax() > DirtyAreaWarningSizeThreshold)
		{
			UE_LOGF(LogNavigationDirtyArea, Warning, "Adding an oversized dirty area: %ls | Threshold: %.2f", *DumpExtraInfo(), DirtyAreaWarningSizeThreshold);
		}
		else
		{
			UE_LOGF(LogNavigationDirtyArea, VeryVerbose, "Adding dirty area object: %ls", *DumpExtraInfo());
		}
#endif // !UE_BUILD_SHIPPING

		if (Flags != ENavigationDirtyFlag::None && bCanAccumulateDirtyAreas)
		{
			DirtyAreas.Add(FNavigationDirtyArea(NewArea, Flags, SourceElement));
		}
	}
	
	UE_CLOGF(NumInvalidBounds > 0 || NumEmptyBounds > 0, LogNavigationDirtyArea, Warning,
		"Skipped some dirty area creation due to: %d invalid bounds, %d empty bounds (element: %ls, from: %ls)",
		NumInvalidBounds, NumEmptyBounds, *GetFullNameSafe(SourceElement.Get()), *DebugReason.ToString());
}

void FNavigationDirtyAreasController::OnNavigationBuildLocked()
{
#if !UE_BUILD_SHIPPING
	bNavigationBuildLocked = true;
#endif // !UE_BUILD_SHIPPING
}

void FNavigationDirtyAreasController::OnNavigationBuildUnlocked()
{
#if !UE_BUILD_SHIPPING
	bNavigationBuildLocked = false;
#endif // !UE_BUILD_SHIPPING
}

void FNavigationDirtyAreasController::SetDirtyAreaWarningSizeThreshold(const float Threshold)
{
#if !UE_BUILD_SHIPPING
	DirtyAreaWarningSizeThreshold = Threshold;
#endif // !UE_BUILD_SHIPPING
}

void FNavigationDirtyAreasController::SetUseWorldPartitionedDynamicMode(bool bIsWPDynamic)
{
	bUseWorldPartitionedDynamicMode = bIsWPDynamic;
}

void FNavigationDirtyAreasController::SetCanReportOversizedDirtyArea(bool bCanReport)
{
#if !UE_BUILD_SHIPPING
	bCanReportOversizedDirtyArea = bCanReport;
#endif // !UE_BUILD_SHIPPING
}

#if !UE_BUILD_SHIPPING
bool FNavigationDirtyAreasController::ShouldReportOversizedDirtyArea() const
{ 
	return bNavigationBuildLocked == false && bCanReportOversizedDirtyArea && DirtyAreaWarningSizeThreshold >= 0.0f;
}
#endif // !UE_BUILD_SHIPPING


void FNavigationDirtyAreasController::Reset()
{
	// discard all pending dirty areas, we are going to rebuild navmesh anyway 
	UE_LOGF(LogNavigationDirtyArea, VeryVerbose, "%s: Reseting All Dirty Areas. DirtyAreas.Num = [%d]",__FUNCTION__, DirtyAreas.Num());

	DirtyAreas.Reset();
#if !UE_BUILD_SHIPPING
	bDirtyAreasReportedWhileAccumulationLocked = false;
#endif // !UE_BUILD_SHIPPING
}
