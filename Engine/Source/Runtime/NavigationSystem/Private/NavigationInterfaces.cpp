// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavNodeInterface.h"


UNavNodeInterface::UNavNodeInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

