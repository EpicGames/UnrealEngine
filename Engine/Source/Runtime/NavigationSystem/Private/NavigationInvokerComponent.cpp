// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationInvokerComponent.h"
#include "NavigationSystem.h"
#include "AI/Navigation/NavigationInvokerPriority.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationInvokerComponent)

UNavigationInvokerComponent::UNavigationInvokerComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, TileGenerationRadius(3000)
	, TileRemovalRadius(5000)
	, Priority(ENavigationInvokerPriority::Default)
{
	bAutoActivate = true;
	SupportedAgents.MarkInitialized();
}

void UNavigationInvokerComponent::Activate(bool bReset)
{
	Super::Activate(bReset);

	UNavigationSystemBase::OnNavigationInitStartStaticDelegate().AddUObject(this, &UNavigationInvokerComponent::OnNavigationInitStart);

	AActor* Owner = GetOwner();
	if (Owner)
	{
		UNavigationSystemV1::RegisterNavigationInvoker(*Owner, TileGenerationRadius, TileRemovalRadius, SupportedAgents, Priority);
	}
}

void UNavigationInvokerComponent::Deactivate()
{
	Super::Deactivate();

	AActor* Owner = GetOwner();
	if (Owner)
	{
		UNavigationSystemV1::UnregisterNavigationInvoker(*Owner);
	}

	UNavigationSystemBase::OnNavigationInitStartStaticDelegate().RemoveAll(this);
}

void UNavigationInvokerComponent::PostInitProperties()
{
	Super::PostInitProperties();
	SupportedAgents.MarkInitialized();
}

void UNavigationInvokerComponent::RegisterWithNavigationSystem(UNavigationSystemV1& NavSys)
{
	if (IsActive())
	{
		AActor* Owner = GetOwner();
		if (Owner)
		{
			NavSys.RegisterInvoker(*Owner, TileGenerationRadius, TileRemovalRadius, SupportedAgents, Priority);
		}
	}
}

void UNavigationInvokerComponent::SetGenerationRadii(const float GenerationRadius, const float RemovalRadius)
{
	TileGenerationRadius = GenerationRadius;
	TileRemovalRadius = RemovalRadius;
}

void UNavigationInvokerComponent::OnNavigationInitStart(const UNavigationSystemBase& NavSys)
{
	if (NavSys.GetWorld() != GetWorld())
	{
		return;
	}

	AActor* Owner = GetOwner();
	if (Owner)
	{
		UNavigationSystemV1::UnregisterNavigationInvoker(*Owner);
		UNavigationSystemV1::RegisterNavigationInvoker(*Owner, TileGenerationRadius, TileRemovalRadius, SupportedAgents, Priority);
	}
}
