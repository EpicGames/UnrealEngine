// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationObjectRepository.h"
#include "Misc/OutputDevice.h"
#include "NavigationSystem.h"
#include "NavLinkCustomInterface.h"
#include "AI/NavigationSystemBase.h"
#include "AI/Navigation/NavigationElement.h"
#include "AI/Navigation/NavRelevantInterface.h"
#include "UObject/ObjectKey.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationObjectRepository)

namespace UE::Navigation::Private
{
static FAutoConsoleCommandWithWorldArgsAndOutputDevice CmdDumpRepositoryElements(
	TEXT("ai.debug.nav.DumpRepositoryElements"),
	TEXT("Logs details about each element stored in the navigation repository to the output device."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, const UWorld* World, FOutputDevice& OutputDevice)
		{
			if (const UNavigationObjectRepository* Repository = World->GetSubsystem<UNavigationObjectRepository>())
			{
				int32 NumElements = 0;

				Repository->ForEachNavigationElement([&OutputDevice, &NumElements](const TSharedRef<const FNavigationElement>& Element)
					{
						NumElements++;
						OutputDevice.Logf(ELogVerbosity::Log, TEXT("%s bounds: [%s] parent:'%s'"),
							*Element->GetPathName(),
							*Element->GetBounds().ToString(),
							*GetNameSafe(Element->GetNavigationParent().Get()));
					});

				OutputDevice.Logf(ELogVerbosity::Log, TEXT("Total: %d elements"), NumElements);
			}
			else
			{
				OutputDevice.Log(ELogVerbosity::Error, TEXT("Command failed since it was unable to find the navigation repository"));
			}
		})
	);
} // UE::Navigation::Private

TSharedPtr<const FNavigationElement> UNavigationObjectRepository::AddNavigationElement(FNavigationElement&& Element, const ENotifyOnSuccess NotifyOnSuccess /*= ENotifyOnSuccess::Yes*/)
{
#if DO_ENSURE // We don't want to execute the Find at all for targets where ensures are disabled
	{
		UE_MT_SCOPED_READ_ACCESS(NavElementAccessDetector);

		if (!ensureMsgf(NavRelevantElements.Find(Element.GetHandle()) == nullptr, TEXT("Same element can't be registered twice.")))
		{
			return nullptr;
		}
	}
#endif

	const TSharedRef SharedElement(MakeShared<FNavigationElement>(MoveTemp(Element)));
	{
		UE_MT_SCOPED_WRITE_ACCESS(NavElementAccessDetector);
		NavRelevantElements.Emplace(Element.GetHandle(), SharedElement);
	}

	if (NotifyOnSuccess == ENotifyOnSuccess::Yes)
	{
		(void)OnNavigationElementAddedDelegate.ExecuteIfBound(SharedElement);
	}

	return SharedElement.ToSharedPtr();
}

void UNavigationObjectRepository::RemoveNavigationElement(const FNavigationElementHandle Handle)
{
	UE_MT_SCOPED_WRITE_ACCESS(NavElementAccessDetector);

	TSharedPtr<FNavigationElement> Element;
	if (ensureMsgf(NavRelevantElements.RemoveAndCopyValue(Handle, Element),
		TEXT("Navigation element can't be removed since it was not registered or already unregistered)")))
	{
		(void)OnNavigationElementRemovedDelegate.ExecuteIfBound(Element.ToSharedRef());
	}
}

void UNavigationObjectRepository::ForEachNavigationElement(TFunctionRef<void(const TSharedRef<const FNavigationElement>&)> PerElementCallback) const
{
	UE_MT_SCOPED_READ_ACCESS(NavElementAccessDetector);

	for (auto It = NavRelevantElements.CreateConstIterator(); It; ++It)
	{
		if (const TSharedPtr<const FNavigationElement>& Element = It.Value())
		{
			PerElementCallback(Element.ToSharedRef());
		}
	}
}

TSharedPtr<const FNavigationElement> UNavigationObjectRepository::RegisterNavRelevantObject(const INavRelevantInterface& NavRelevantObject)
{
	return RegisterNavRelevantObjectInternal(NavRelevantObject, *Cast<UObject>(&NavRelevantObject), ENotifyOnSuccess::Yes);
}

bool UNavigationObjectRepository::ShouldCreateSubsystem(UObject* Outer) const
{
	return (Super::ShouldCreateSubsystem(Outer))
		&& GetDefault<UNavigationSystemV1>()->ShouldCreateNavigationSystemInstance(Cast<UWorld>(Outer));
}

TSharedPtr<const FNavigationElement> UNavigationObjectRepository::RegisterNavRelevantObjectInternal(
	const INavRelevantInterface& NavRelevantInterface,
	const UObject& NavRelevantObject,
	const ENotifyOnSuccess NotifyOnSuccess)
{
	// In AActor/UActorComponent code paths it is possible that a component registration is performed more than once
	// (i.e., Actor registering its component, then individual component registers too)
	// In such case we update with the latest.
	if (const TSharedPtr<FNavigationElement> ExistingElement = GetMutableNavigationElementForUObject(&NavRelevantObject))
	{
		const FBox PreviousBounds = ExistingElement->GetBounds();
		const FNavigationElementHandle Handle = ExistingElement->GetHandle();

		// Create a new FNavigationElement instance rather than overwriting the existing one in-place.
		// The old instance may still be referenced by the navigation octree or an in-flight tile generator
		// (via FNavigationRelevantData::SourceElement) and must remain valid until those holders release it.
		const TSharedRef<FNavigationElement> NewElement = MakeShared<FNavigationElement>(NavRelevantInterface);
		{
			UE_MT_SCOPED_WRITE_ACCESS(NavElementAccessDetector);
			NavRelevantElements.Emplace(Handle, NewElement);
		}

		if (NotifyOnSuccess == ENotifyOnSuccess::Yes)
		{
			(void)OnNavigationElementAddedDelegate.ExecuteIfBound(NewElement);
		}

		UE_LOGF(LogNavigation, VeryVerbose, "%s [already registered - updating] (%ls:%ls) Bounds: [%ls]->[%ls]", __FUNCTION__,
			*GetNameSafe(NavRelevantObject.GetOuter()), *GetNameSafe(&NavRelevantObject),
			*PreviousBounds.ToString(), *NewElement->GetBounds().ToString());

		return NewElement;
	}

	if (NavRelevantInterface.IsNavigationRelevant())
	{
		if (const TSharedPtr<const FNavigationElement> SharedElement = AddNavigationElement(FNavigationElement(NavRelevantInterface), NotifyOnSuccess))
		{
			UE_MT_SCOPED_WRITE_ACCESS(NavElementAccessDetector);
			ObjectsToHandleMap.Emplace(FObjectKey(&NavRelevantObject), SharedElement->GetHandle());

			UE_LOGF(LogNavigation, VeryVerbose, "%s [registered] (%ls:%ls) Bounds: [%ls]", __FUNCTION__,
				*GetNameSafe(NavRelevantObject.GetOuter()), *GetNameSafe(&NavRelevantObject),
				*NavRelevantInterface.GetNavigationBounds().ToString());

			return SharedElement;
		}

		return nullptr;
	}

	UE_LOGF(LogNavigation, VeryVerbose, "%s [skipped: not relevant] (%ls:%ls)", __FUNCTION__,
		*GetNameSafe(NavRelevantObject.GetOuter()), *GetNameSafe(&NavRelevantObject));
	return nullptr;
}

void UNavigationObjectRepository::UnregisterNavRelevantObject(const INavRelevantInterface& NavRelevantObject)
{
	UnregisterNavRelevantObject(Cast<UObject>(&NavRelevantObject));
}

void UNavigationObjectRepository::UnregisterNavRelevantObject(const UObject* NavRelevantObject)
{
	UE_LOGF(LogNavigation, VeryVerbose, "%s (%ls:%ls)", __FUNCTION__,
		NavRelevantObject ? *GetNameSafe(NavRelevantObject->GetOuter()) : TEXT("null outer"),
		*GetNameSafe(NavRelevantObject));

	FNavigationElementHandle Handle;
	{
		UE_MT_SCOPED_WRITE_ACCESS(NavElementAccessDetector);
		ObjectsToHandleMap.RemoveAndCopyValue(FObjectKey(NavRelevantObject), Handle);
	}

	if (Handle)
	{
		RemoveNavigationElement(Handle);
	}
}

TSharedPtr<const FNavigationElement> UNavigationObjectRepository::GetNavigationElementForHandle(const FNavigationElementHandle Handle) const
{
	return GetMutableNavigationElementForHandle(Handle);
}

TSharedPtr<FNavigationElement> UNavigationObjectRepository::GetMutableNavigationElementForHandle(const FNavigationElementHandle Handle) const
{
	UE_MT_SCOPED_READ_ACCESS(NavElementAccessDetector);

	if (const TSharedPtr<FNavigationElement>* Element = NavRelevantElements.Find(Handle))
	{
		return *Element;
	}

	return nullptr;
}

FNavigationElementHandle UNavigationObjectRepository::GetNavigationElementHandleForUObject(const UObject* NavRelevantObject) const
{
	UE_MT_SCOPED_READ_ACCESS(NavElementAccessDetector);

	if (const FNavigationElementHandle* Handle = ObjectsToHandleMap.Find(FObjectKey(Cast<UObject>(NavRelevantObject))))
	{
		return *Handle;
	}

	return FNavigationElementHandle::Invalid;
}

TSharedPtr<const FNavigationElement> UNavigationObjectRepository::GetNavigationElementForUObject(const UObject* NavRelevantObject) const
{
	return GetMutableNavigationElementForUObject(NavRelevantObject);
}

TSharedPtr<FNavigationElement> UNavigationObjectRepository::GetMutableNavigationElementForUObject(const UObject* NavRelevantObject) const
{
	UE_MT_SCOPED_READ_ACCESS(NavElementAccessDetector);

	if (const FNavigationElementHandle* Handle = ObjectsToHandleMap.Find(FObjectKey(NavRelevantObject)))
	{
		if (const TSharedPtr<FNavigationElement>* Element = NavRelevantElements.Find(*Handle))
		{
			return *Element;
		}
	}

	return nullptr;
}

TSharedPtr<const FNavigationElement> UNavigationObjectRepository::UpdateNavigationElementForUObject(
	const INavRelevantInterface& NavRelevantInterface,
	const UObject& NavRelevantObject)
{
	// This method is called by the navigation system to make sure an up-to-date navigation element exists for a
	// given navigation relevant UObject.
	// In this case we only need to create, or update, the navigation element without sending
	// notification (i.e. ENotifyOnSuccess::No) since the caller (NavigationSystem) is already in the process of updating.
	return RegisterNavRelevantObjectInternal(NavRelevantInterface, NavRelevantObject, ENotifyOnSuccess::No);
}

void UNavigationObjectRepository::RegisterCustomNavLinkObject(INavLinkCustomInterface& CustomNavLinkObject)
{
	{
		UE_MT_SCOPED_WRITE_ACCESS(NavElementAccessDetector);

#if DO_ENSURE // We don't want to execute the Find at all for targets where ensures are disabled
		if (!ensureMsgf(CustomLinkObjects.Find(&CustomNavLinkObject) == INDEX_NONE, TEXT("Same interface pointer can't be registered twice.")))
		{
			return;
		}
#endif

		CustomLinkObjects.Emplace(&CustomNavLinkObject);
	}

	OnCustomNavLinkObjectRegistered.ExecuteIfBound(CustomNavLinkObject);
}

void UNavigationObjectRepository::UnregisterCustomNavLinkObject(INavLinkCustomInterface& CustomNavLinkObject)
{
	{
		UE_MT_SCOPED_WRITE_ACCESS(NavElementAccessDetector);
		ensureMsgf(CustomLinkObjects.Remove(&CustomNavLinkObject) > 0, TEXT("Interface can't be removed since it was not registered or already unregistered)"));
	}

	OnCustomNavLinkObjectUnregistered.ExecuteIfBound(CustomNavLinkObject);
}
