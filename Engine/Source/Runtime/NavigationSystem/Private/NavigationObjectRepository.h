// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AI/Navigation/NavigationElement.h" // required since we can't fwd declare types used as TMap keys
#include "Misc/MTTransactionallySafeAccessDetector.h"
#include "Subsystems/WorldSubsystem.h"
#include "UObject/ObjectKey.h"
#include "UObject/WeakInterfacePtr.h"
#include "NavigationObjectRepository.generated.h"

class INavLinkCustomInterface;
class INavRelevantInterface;

UE_DEPRECATED(5.5, "Use the overloaded version with FNavigationElement instead.")
DECLARE_DELEGATE_OneParam(FOnNavRelevantObjectRegistrationEvent, INavRelevantInterface&);

DECLARE_DELEGATE_OneParam(FOnNavigationElementRegistrationEvent, const TSharedRef<const FNavigationElement>&);
DECLARE_DELEGATE_OneParam(FOnCustomNavLinkObjectRegistrationEvent, INavLinkCustomInterface&);

/**
 * World subsystem dedicated to store different types of navigation related elements that the
 * NavigationSystem needs to access.
 */
UCLASS()
class UNavigationObjectRepository : public UWorldSubsystem
{
	GENERATED_BODY()

public:

	// Specifies whether OnNavigationElementAddedDelegate must be broadcast on successful registration.
	enum class ENotifyOnSuccess : uint8
	{
		No,
		Yes
	};

	/**
	 * Adds the provided navigation element to the list of registered elements.
	 * @param Element Reference to the navigation element to register.
	 * @param NotifyOnSuccess Indicates if OnNavigationElementRegisteredDelegate must be broadcast on successful registration.
	 * @return Handle to the registered element.
	 * @note Method will assert if the same element is registered twice.
	 */
	TSharedPtr<const FNavigationElement> AddNavigationElement(FNavigationElement&& Element, ENotifyOnSuccess NotifyOnSuccess = ENotifyOnSuccess::Yes);

	/**
	 * Removes the navigation element associated with the provided handle from the list of registered elements.
	 * @param Handle Handle to the element to unregister.
	 * @note Method will assert if the element can't be removed (i.e., not registered or already unregistered).
	 */
	void RemoveNavigationElement(FNavigationElementHandle Handle);

	/**
	 * Iterates through all registered navigation elements and call provided function with the element as parameter.
	 */
	void ForEachNavigationElement(TFunctionRef<void(const TSharedRef<const FNavigationElement>&)>) const;

	/**
	 * Adds the provided object implementing INavRelevantInterface to the list of registered navigation relevant objects.
	 * @param NavRelevantObject INavRelevantInterface of the object to register.
	 * @note Method will also assert if same interface pointers is registered twice.
	 */
	TSharedPtr<const FNavigationElement> RegisterNavRelevantObject(const INavRelevantInterface& NavRelevantObject);

	/**
	 * Removes the provided interface from the list of registered navigation relevant objects.
	 * @param NavRelevantObject INavRelevantInterface of the object to unregister.
	 * @note Method will also assert if interface can't be removed (i.e. not registered or already unregistered).
	 */
	void UnregisterNavRelevantObject(const INavRelevantInterface& NavRelevantObject);

	/**
	 * Removes the provided object from the list of registered navigation relevant objects.
	 * @param NavRelevantObject INavRelevantInterface of the object to unregister.
	 * @note Method will also assert if interface can't be removed (i.e. not registered or already unregistered).
	 */
	void UnregisterNavRelevantObject(const UObject* NavRelevantObject);

	/**
	 * Returns the handle associated with the specific UObject if it is registered in the repository.
	 * @param NavRelevantObject The UObject for which the handle must be retrieved.
	 * @return The handle associated with the specified UObject if already registered, an invalid handle otherwise.
	 */
	FNavigationElementHandle GetNavigationElementHandleForUObject(const UObject* NavRelevantObject) const;

	/**
	 * Returns a shared pointer to the FNavigationElement created for a registered UObject.
	 * @param NavRelevantObject The UObject for which the FNavigationElement must be retrieved.
	 * @return A valid shared pointer if the specified UObject is registered and an associated FNavigationElement was created,
	 * null pointer otherwise.
	 */
	TSharedPtr<const FNavigationElement> GetNavigationElementForUObject(const UObject* NavRelevantObject) const;

	/**
	 * Returns a shared pointer to the mutable FNavigationElement created for a registered UObject.
	 * @param NavRelevantObject The UObject for which the FNavigationElement must be retrieved.
	 * @return A valid shared pointer if the specified UObject is registered and an associated FNavigationElement was created,
	 * null pointer otherwise.
	 */
	TSharedPtr<FNavigationElement> GetMutableNavigationElementForUObject(const UObject* NavRelevantObject) const;

	/**
	 * Returns a shared pointer to the FNavigationElement created for a registered UObject.
	 * This method will not handle other navigation structure updates (e.g., NavigationOctree)
	 * and should only be used by the NavigationSystem.
	 * @param NavRelevantInterface The interface to use to update an existing element
	 * @param NavRelevantObject The UObject for which the FNavigationElement must be retrieved.
	 * @return A valid shared pointer if the specified UObject is registered and an associated FNavigationElement was created,
	 * null pointer otherwise.
	 */
	TSharedPtr<const FNavigationElement> UpdateNavigationElementForUObject(const INavRelevantInterface& NavRelevantInterface, const UObject& NavRelevantObject);

	/**
	 * Returns a shared pointer to the FNavigationElement associated with the provided handle.
	 * @param Handle The handle for which the FNavigationElement must be retrieved.
	 * @return A valid shared pointer if a FNavigationElement associated with the provided handle was registered, null pointer otherwise.
	 */
	TSharedPtr<const FNavigationElement> GetNavigationElementForHandle(FNavigationElementHandle Handle) const;

	/**
	 * Returns a shared pointer to the mutable FNavigationElement associated with the provided handle.
	 * @param Handle The handle for which the FNavigationElement must be retrieved.
	 * @return A valid shared pointer if a FNavigationElement associated with the provided handle was registered, null pointer otherwise.
	 */
	TSharedPtr<FNavigationElement> GetMutableNavigationElementForHandle(FNavigationElementHandle Handle) const;

	/**
	 * Adds the provided interface to the list of registered custom navigation links.
	 * @param CustomNavLinkObject INavLinkCustomInterface of the object to register.
	 * @note Method will also assert if same interface pointers is registered twice.
	 */
	void RegisterCustomNavLinkObject(INavLinkCustomInterface& CustomNavLinkObject);

	/**
	 * Removes the provided interface from the list of registered custom navigation links.
	 * @param CustomNavLinkObject INavLinkCustomInterface of the object to unregister.
	 * @note Method will also assert if interface can't be removed (i.e. not registered or already unregistered).
	 */
	void UnregisterCustomNavLinkObject(INavLinkCustomInterface& CustomNavLinkObject);

	/**
	 * Returns the list of registered custom navigation links.
	 * @return Const view on the list of all registered custom navigation links.
	 */
	TConstArrayView<TWeakInterfacePtr<INavLinkCustomInterface>> GetCustomLinks() const
	{
		return CustomLinkObjects;
	}

	/** Returns the number of navigation elements registered in the repository */
	int32 GetNumRegisteredElements() const
	{
		return NavRelevantElements.Num();
	}

	/** Returns the number of UObject registered in the repository for which a FNavigationElement has been created and registered. */
	int32 GetNumRegisteredUObjects() const
	{
		return ObjectsToHandleMap.Num();
	}

	/** Returns the number of UObjects implementing INavLinkCustomInterface registered in the repository */
	int32 GetNumRegisteredCustomLinks() const
	{
		return CustomLinkObjects.Num();
	}

	/** Delegate executed when a navigation element is added in the repository. */
	FOnNavigationElementRegistrationEvent OnNavigationElementAddedDelegate;

	/** Delegate executed when a navigation element is removed from the repository. */
	FOnNavigationElementRegistrationEvent OnNavigationElementRemovedDelegate;

	/** Delegate executed when a custom navigation link is registered with the repository. */
	FOnCustomNavLinkObjectRegistrationEvent OnCustomNavLinkObjectRegistered;

	/** Delegate executed when a custom navigation link is unregistered with the repository. */
	FOnCustomNavLinkObjectRegistrationEvent OnCustomNavLinkObjectUnregistered;

protected:
	virtual bool ShouldCreateSubsystem(UObject* Outer) const override;

private:

	TSharedPtr<const FNavigationElement> RegisterNavRelevantObjectInternal(const INavRelevantInterface& NavRelevantInterface, const UObject& NavRelevantObject, ENotifyOnSuccess NotifyOnSuccess);

	/** For legacy object registration path (i.e., Actor/ActorComponent) */
	TMap<FObjectKey, FNavigationElementHandle> ObjectsToHandleMap;

	/** List of registered navigation elements. */
	TMap<FNavigationElementHandle, TSharedPtr<FNavigationElement>> NavRelevantElements;

	/** List of registered custom navigation link objects. */
	TArray<TWeakInterfacePtr<INavLinkCustomInterface>> CustomLinkObjects;

	/** Multi thread access detector used to validate accesses to the maps of registered UObjects and FNavigationElement */
	UE_MT_DECLARE_TS_RW_ACCESS_DETECTOR(NavElementAccessDetector);

	//----------------------------------------------------------------------//
	// DEPRECATED
	//----------------------------------------------------------------------//
public:

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "Use OnNavigationElementAddedDelegate instead.")
	FOnNavRelevantObjectRegistrationEvent OnNavRelevantObjectRegistered;

	UE_DEPRECATED(5.5, "Use OnNavigationElementRemovedDelegate instead.")
	FOnNavRelevantObjectRegistrationEvent OnNavRelevantObjectUnregistered;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
};
