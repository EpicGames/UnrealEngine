// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationOctree.h"
#include "AI/Navigation/NavRelevantInterface.h"
#include "AI/Navigation/NavigationElement.h"
#include "Interfaces/Interface_AsyncCompilation.h"
#include "NavigationSystem.h"
#include "UObject/Package.h"

LLM_DEFINE_TAG(NavigationOctree);

namespace UE::NavigationOctree::Private
{
#if !UE_BUILD_SHIPPING
bool bValidateConsistencyWhenAddingNode = false;

FAutoConsoleVariableRef ConsoleVariables[] =
{
	FAutoConsoleVariableRef(
		TEXT("ai.debug.nav.validateConsistencyWhenAddingOctreeNode"),
		bValidateConsistencyWhenAddingNode,
		TEXT("Used to validate that registered FNavigationElement matches the values "
			"returned by INavRelevantInterface when processing pending updates to add elements to the octree."))
};
#endif // !UE_BUILD_SHIPPING

} // UE::NavigationOctree::Private

//----------------------------------------------------------------------//
// FNavigationOctree
//----------------------------------------------------------------------//
FNavigationOctree::FNavigationOctree(const FVector& Origin, FVector::FReal Radius)
	: TOctree2<FNavigationOctreeElement, FNavigationOctreeSemantics>(Origin, Radius)
	, DefaultGeometryGatheringMode(ENavDataGatheringMode::Instant)
	, bGatherGeometry(false)
	, NodesMemory(0)
#if !UE_BUILD_SHIPPING
	, GatheringNavModifiersTimeLimitWarning(-1.0f)
#endif // !UE_BUILD_SHIPPING
{
	INC_DWORD_STAT_BY( STAT_NavigationMemory, sizeof(*this) );
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FNavigationOctree::~FNavigationOctree()
{
	DEC_DWORD_STAT_BY( STAT_NavigationMemory, sizeof(*this) );
	DEC_MEMORY_STAT_BY(STAT_Navigation_CollisionTreeMemory, NodesMemory);
	
	ElementToOctreeId.Empty();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FNavigationOctree::SetDataGatheringMode(ENavDataGatheringModeConfig Mode)
{
	check(Mode != ENavDataGatheringModeConfig::Invalid);
	DefaultGeometryGatheringMode = ENavDataGatheringMode(Mode);
}

void FNavigationOctree::SetNavigableGeometryStoringMode(ENavGeometryStoringMode NavGeometryMode)
{
	bGatherGeometry = (NavGeometryMode == FNavigationOctree::StoreNavGeometry);
}

void FNavigationOctree::DemandLazyDataGathering(FNavigationRelevantData& ElementData)
{
	LLM_SCOPE_BYTAG(NavigationOctree);

	bool bShrink = false;
	const int32 OrgElementMemory = IntCastChecked<int32>(ElementData.GetGeometryAllocatedSize());

	if (ElementData.IsPendingLazyGeometryGathering() == true && ElementData.SupportsGatheringGeometrySlices() == false)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_LazyGeometryExport);

		GeometryExportDelegate.ExecuteIfBound(ElementData.SourceElement.Get(), ElementData);
		bShrink = true;

		// mark this element as no longer needing geometry gathering
		ElementData.bPendingLazyGeometryGathering = false;
	}

	if (ElementData.IsPendingLazyModifiersGathering())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RecastNavMeshGenerator_LazyModifiersExport);

#if !UE_BUILD_SHIPPING
		const bool bCanOutputDurationWarning = GatheringNavModifiersTimeLimitWarning >= 0.0f;
		const double StartTime = bCanOutputDurationWarning ? FPlatformTime::Seconds() : 0.0f;
#endif //!UE_BUILD_SHIPPING

		ElementData.SourceElement->NavigationDataExportDelegate.ExecuteIfBound(ElementData.SourceElement.Get(), ElementData);
		ElementData.bPendingLazyModifiersGathering = false;
		bShrink = true;

#if !UE_BUILD_SHIPPING
		// If GatheringNavModifiersWarningLimitTime is positive, it will print a Warning if the time taken to call GetNavigationData is more than GatheringNavModifiersWarningLimitTime
		if (bCanOutputDurationWarning)
		{
			const double DeltaTime = FPlatformTime::Seconds() - StartTime;
			if (DeltaTime > GatheringNavModifiersTimeLimitWarning)
			{
				UE_LOGF(LogNavigation, Warning, "The time (%f sec) for gathering navigation data on a navigation element exceeded the time limit (%f sec) | Element = %ls",
					DeltaTime,
					GatheringNavModifiersTimeLimitWarning,
					*ElementData.SourceElement->GetName());
			}
		}
#endif //!UE_BUILD_SHIPPING
	}

	if (bShrink)
	{
		// validate exported data
		// shrink arrays before counting memory
		// it will be reallocated when adding to octree and RemoveNode will have different value returned by GetAllocatedSize()
		ElementData.ValidateAndShrink();
	}

	const int32 ElementMemoryChange = IntCastChecked<int32>(ElementData.GetGeometryAllocatedSize()) - OrgElementMemory;
	NodesMemory += ElementMemoryChange;
	INC_MEMORY_STAT_BY(STAT_Navigation_CollisionTreeMemory, ElementMemoryChange);
}

void FNavigationOctree::DemandChildLazyDataGathering(FNavigationRelevantData& ElementData, const FNavigationElement& ChildElement) const
{
	LLM_SCOPE_BYTAG(NavigationOctree);

	if (IsLazyGathering(ChildElement))
	{
		if (ChildElement.NavigationDataExportDelegate.ExecuteIfBound(ChildElement, ElementData))
		{
			ElementData.ValidateAndShrink();
		}
	}
}

#if !UE_BUILD_SHIPPING
void FNavigationOctree::SetGatheringNavModifiersTimeLimitWarning(const float Threshold)
{
	GatheringNavModifiersTimeLimitWarning = Threshold;
}
#endif // !UE_BUILD_SHIPPING

bool FNavigationOctree::IsLazyGathering(const FNavigationElement& NavigationElement) const
{
	const ENavDataGatheringMode GatheringMode = NavigationElement.GetGeometryGatheringMode();
	const bool bDoInstantGathering = ((GatheringMode == ENavDataGatheringMode::Default && DefaultGeometryGatheringMode == ENavDataGatheringMode::Instant)
		|| GatheringMode == ENavDataGatheringMode::Instant);

	return !bDoInstantGathering;
}

void FNavigationOctree::AddNode(const FBox& Bounds, FNavigationOctreeElement& OctreeElement)
{
	LLM_SCOPE_BYTAG(NavigationOctree);

	const TSharedRef<const FNavigationElement>& ElementRef = OctreeElement.Data->SourceElement;
	const FNavigationElement& SourceElement = ElementRef.Get();

	UE_LOGF(LogNavigation, VeryVerbose, "%s: '%ls' bounds: [%ls]", __FUNCTION__, *SourceElement.GetName(), *Bounds.ToString());

#if !UE_BUILD_SHIPPING
	if (UE::NavigationOctree::Private::bValidateConsistencyWhenAddingNode)
	{
		if (const INavRelevantInterface* NavRelevantInterface = Cast<INavRelevantInterface>(SourceElement.GetWeakUObject().Get()))
		{
			// The following validations are used to detect scenarios where properties of a NavigationElement created from
			// a navigation relevant UObject (implementing INavRelevantInterface and returning 'true' to IsNavigationRelevant())
			// differs from the values that are provided by that same UObject when the pending registration of the element is processed.
			// These indicate that the values were not up-to-date when the element was added to a pending FNavigationDirtyElement,
			// or during that frame since an update would have refreshed the pending DirtyElement.
			if (const FBox NewBounds = NavRelevantInterface->GetNavigationBounds(); !Bounds.Equals(NewBounds))
			{
				UE_LOGF(LogNavigation, Warning, "%s: '%ls' bounds changed between element's creation and its addition to the octree: [%ls] --> [%ls]",
					__FUNCTION__, *SourceElement.GetName(), *Bounds.ToString(), *NewBounds.ToString());
			}

			if (const UObject* NewParent = NavRelevantInterface->GetNavigationParent(); NewParent != SourceElement.GetNavigationParent())
			{
				UE_LOGF(LogNavigation, Warning, "%s: '%ls' parent changed between element's creation and its addition to the octree: [%ls] --> [%ls]",
					__FUNCTION__, *SourceElement.GetName(), *GetNameSafe(SourceElement.GetNavigationParent().Get()), *GetNameSafe(NewParent));
			}
		}
	}
#endif // !UE_BUILD_SHIPPING

	if (UNLIKELY(!Bounds.IsValid || Bounds.GetSize().IsNearlyZero()))
	{
		UE_LOGF(LogNavigation, Warning, "%s: %ls bounds, ignoring %ls.", __FUNCTION__, !Bounds.IsValid ? TEXT("Invalid") : TEXT("Empty"), *SourceElement.GetFullName());
		return;
	}

	OctreeElement.Bounds = Bounds;
	OctreeElement.Data->bShouldSkipDirtyAreaOnAddOrRemove = !SourceElement.GetDirtyAreaOnRegistration();

	// Only gather geometry and navigation data if not already provided.
	// We don't want to use the default geometry export since it will clear the navigation data.
	if (OctreeElement.Data->IsEmpty())
	{
		const bool bDoInstantGathering = !IsLazyGathering(SourceElement);

		if (bGatherGeometry)
		{
			if (bDoInstantGathering)
			{
				GeometryExportDelegate.ExecuteIfBound(SourceElement, *OctreeElement.Data);
			}
			else
			{
				OctreeElement.Data->bPendingLazyGeometryGathering = true;
				OctreeElement.Data->bSupportsGatheringGeometrySlices = SourceElement.GeometrySliceExportDelegate.IsBound();
			}
		}

		SCOPE_CYCLE_COUNTER(STAT_Navigation_GatheringNavigationModifiersSync);
		if (bDoInstantGathering)
		{
#if !UE_BUILD_SHIPPING
			const bool bCanOutputDurationWarning = GatheringNavModifiersTimeLimitWarning >= 0.0f;
			const double StartTime = bCanOutputDurationWarning ? FPlatformTime::Seconds() : 0.0f;
#endif //!UE_BUILD_SHIPPING

			SourceElement.NavigationDataExportDelegate.ExecuteIfBound(SourceElement, *OctreeElement.Data);

#if !UE_BUILD_SHIPPING
			// If GatheringNavModifiersWarningLimitTime is positive, it will print a Warning if the time taken to call GetNavigationData is more than GatheringNavModifiersWarningLimitTime			
			if (bCanOutputDurationWarning)
			{
				if (const double DeltaTime = FPlatformTime::Seconds() - StartTime; DeltaTime > GatheringNavModifiersTimeLimitWarning)
				{
					UE_LOGF(LogNavigation, Warning, "The time (%f sec) for gathering navigation data on a navigation element exceeded the time limit (%f sec) | Element = %ls",
						DeltaTime,
						GatheringNavModifiersTimeLimitWarning,
						*SourceElement.GetName());
				}
			}
#endif //!UE_BUILD_SHIPPING
		}
		else
		{
			OctreeElement.Data->bPendingLazyModifiersGathering = true;
		}

		// validate exported data
		// shrink arrays before counting memory
		// it will be reallocated when adding to octree and RemoveNode will have different value returned by GetAllocatedSize()
		OctreeElement.ValidateAndShrink();
	}

	const int32 ElementMemory = OctreeElement.GetAllocatedSize();
	NodesMemory += ElementMemory;
	INC_MEMORY_STAT_BY(STAT_Navigation_CollisionTreeMemory, ElementMemory);

	AddElement(OctreeElement);
}

void FNavigationOctree::AppendToNode(const FOctreeElementId2& Id, const TSharedRef<const FNavigationElement>& ElementRef, const FBox& Bounds, FNavigationOctreeElement& Element)
{
	LLM_SCOPE_BYTAG(NavigationOctree);

	const FNavigationOctreeElement OrgData = GetElementById(Id);

	Element = OrgData;
	Element.Bounds = Bounds + OrgData.Bounds.GetBox();

	SCOPE_CYCLE_COUNTER(STAT_Navigation_GatheringNavigationModifiersSync);
	if (IsLazyGathering(ElementRef.Get()))
	{
		Element.Data->bPendingChildLazyModifiersGathering = true;
	}
	else
	{
		ElementRef->NavigationDataExportDelegate.ExecuteIfBound(ElementRef.Get(), *Element.Data);
	}

	// validate exported data
	// shrink arrays before counting memory
	// it will be reallocated when adding to octree and RemoveNode will have different value returned by GetAllocatedSize()
	Element.ValidateAndShrink();

	const int32 OrgElementMemory = OrgData.GetAllocatedSize();
	const int32 NewElementMemory = Element.GetAllocatedSize();
	const int32 MemoryDelta = NewElementMemory - OrgElementMemory;

	NodesMemory += MemoryDelta;
	INC_MEMORY_STAT_BY(STAT_Navigation_CollisionTreeMemory, MemoryDelta);

	RemoveElement(Id);
	AddElement(Element);
}

void FNavigationOctree::UpdateNode(const FOctreeElementId2& Id, const FBox& NewBounds)
{
	FNavigationOctreeElement ElementCopy = GetElementById(Id);
	RemoveElement(Id);
	ElementCopy.Bounds = NewBounds;
	AddElement(ElementCopy);
}

void FNavigationOctree::RemoveNode(const FOctreeElementId2& Id)
{
	const FNavigationOctreeElement& Element = GetElementById(Id);
	const int32 ElementMemory = Element.GetAllocatedSize();
	NodesMemory -= ElementMemory;
	DEC_MEMORY_STAT_BY(STAT_Navigation_CollisionTreeMemory, ElementMemory);

	RemoveElement(Id);
}

const FNavigationRelevantData* FNavigationOctree::GetDataForID(const FOctreeElementId2& Id) const
{
	return Id.IsValidId() ? &*GetElementById(Id).Data : nullptr;
}

FNavigationRelevantData* FNavigationOctree::GetMutableDataForID(const FOctreeElementId2& Id)
{
	return Id.IsValidId() ? &*GetElementById(Id).Data : nullptr;
}

void FNavigationOctree::SetElementIdImpl(const FNavigationElementHandle ElementHandle, const FOctreeElementId2 Id)
{
	ElementToOctreeId.Add(ElementHandle, Id);
}

//----------------------------------------------------------------------//
// FNavigationOctreeSemantics
//----------------------------------------------------------------------//
#if NAVSYS_DEBUG
FORCENOINLINE
#endif // NAVSYS_DEBUG
void FNavigationOctreeSemantics::SetElementId(FOctree& OctreeOwner, const FNavigationOctreeElement& Element, const FOctreeElementId2 Id)
{
	static_cast<FNavigationOctree&>(OctreeOwner).SetElementIdImpl(Element.Data.Get().SourceElement.Get().GetHandle(), Id);
}