// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationOctreeController.h"
#include "NavigationSystem.h"


//----------------------------------------------------------------------//
// FNavigationOctreeController
//----------------------------------------------------------------------//
void FNavigationOctreeController::Reset()
{
	if (NavOctree.IsValid())
	{
		NavOctree->Destroy();
		NavOctree = nullptr;
	}
	PendingUpdates.Empty(32);
}

bool FNavigationOctreeController::HasPendingUpdateForElement(const FNavigationElementHandle Element) const
{ 
	return PendingUpdates.Contains(Element);
}

void FNavigationOctreeController::SetNavigableGeometryStoringMode(FNavigationOctree::ENavGeometryStoringMode NavGeometryMode)
{
	check(NavOctree.IsValid());
	NavOctree->SetNavigableGeometryStoringMode(NavGeometryMode);
}

bool FNavigationOctreeController::GetNavOctreeElementData(const FNavigationElementHandle Element, ENavigationDirtyFlag& OutDirtyFlags, FBox& OutDirtyBounds)
{
	const FOctreeElementId2* ElementId = GetNavOctreeIdForElement(Element);
	if (ElementId != nullptr && IsValidElement(*ElementId))
	{
		// mark area occupied by given actor as dirty
		const FNavigationOctreeElement& ElementData = NavOctree->GetElementById(*ElementId);
		OutDirtyFlags = ElementData.Data->GetDirtyFlag();
		OutDirtyBounds = ElementData.Bounds.GetBox();
		return true;
	}

	return false;
}

const FNavigationRelevantData* FNavigationOctreeController::GetDataForElement(const FNavigationElementHandle Element) const
{
	if (const FOctreeElementId2* ElementId = GetNavOctreeIdForElement(Element); IsValidElement(ElementId))
	{
		return NavOctree->GetDataForID(*ElementId);
	}

	return nullptr;
}

FNavigationRelevantData* FNavigationOctreeController::GetMutableDataForElement(const FNavigationElementHandle Element)
{
	if (const FOctreeElementId2* ElementId = GetNavOctreeIdForElement(Element); IsValidElement(ElementId))
	{
		return NavOctree->GetMutableDataForID(*ElementId);
	}

	return nullptr;
}
