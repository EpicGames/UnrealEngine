// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationPath.h"
#include "EngineStats.h"
#include "EngineGlobals.h"
#include "AI/Navigation/NavAgentInterface.h"
#include "NavigationSystem.h"
#include "Engine/Engine.h"
#include "Engine/Canvas.h"
#include "DrawDebugHelpers.h"
#include "VisualLogger/VisualLoggerTypes.h"
#include "NavAreas/NavArea.h"
#include "Debug/DebugDrawService.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationPath)

#define DEBUG_DRAW_OFFSET 0
#define PATH_OFFSET_KEEP_VISIBLE_POINTS 1


//----------------------------------------------------------------------//
// FNavigationPath
//----------------------------------------------------------------------//
const FNavPathType FNavigationPath::Type;

FNavigationPath::FNavigationPath()
	: GoalActorAsNavAgent(nullptr)
	, SourceActorAsNavAgent(nullptr)
	, PathType(FNavigationPath::Type)
	, bDoAutoUpdateOnInvalidation(true)
	, bIgnoreInvalidation(false)
	, bUpdateStartPointOnRepath(true)
	, bUpdateEndPointOnRepath(true)
	, bWaitingForRepath(false)
	, bUseOnPathUpdatedNotify(false)
	, LastUpdateTimeStamp(-1.f)	// indicates that it has not been set
	, GoalActorLocationTetherDistanceSq(-1.f)
	, GoalActorLastLocation(FVector::ZeroVector)
{
	InternalResetNavigationPath();
}

FNavigationPath::FNavigationPath(const TArray<FVector>& Points, AActor* InBase)
	: GoalActorAsNavAgent(nullptr)
	, SourceActorAsNavAgent(nullptr)
	, PathType(FNavigationPath::Type)
	, bDoAutoUpdateOnInvalidation(true)
	, bIgnoreInvalidation(false)
	, bUpdateStartPointOnRepath(true)
	, bUpdateEndPointOnRepath(true)
	, bWaitingForRepath(false)
	, bUseOnPathUpdatedNotify(false)
	, LastUpdateTimeStamp(-1.f)	// indicates that it has not been set
	, GoalActorLocationTetherDistanceSq(-1.f)
	, GoalActorLastLocation(FVector::ZeroVector)
{
	InternalResetNavigationPath();
	MarkReady();

	Base = InBase;

	PathPoints.AddZeroed(Points.Num());
	for (int32 i = 0; i < Points.Num(); i++)
	{
		FBasedPosition BasedPoint(InBase, Points[i]);
		PathPoints[i] = FNavPathPoint(*BasedPoint);
	}
}

FNavigationPath::~FNavigationPath() = default;

void FNavigationPath::InternalResetNavigationPath()
{
	ShortcutNodeRefs.Reset();
	PathPoints.Reset();
	Base.Reset();

	bUpToDate = true;
	bIsReady = false;
	bIsPartial = false;
	bReachedSearchLimit = false;
	bObservingGoalActor = GoalActor.IsValid();

	// keep:
	// - GoalActor
	// - GoalActorAsNavAgent
	// - SourceActor
	// - SourceActorAsNavAgent
	// - Querier
	// - Filter
	// - PathType
	// - ObserverDelegate
	// - bDoAutoUpdateOnInvalidation
	// - bIgnoreInvalidation
	// - bUpdateStartPointOnRepath
	// - bUpdateEndPointOnRepath
	// - bWaitingForRepath
	// - NavigationDataUsed
	// - LastUpdateTimeStamp
	// - GoalActorLocationTetherDistanceSq
	// - GoalActorLastLocation
}

FVector FNavigationPath::GetGoalLocation() const
{
	return GoalActor != NULL ? (GoalActorAsNavAgent != NULL ? GoalActorAsNavAgent->GetNavAgentLocation() : GoalActor->GetActorLocation()) : GetEndLocation();
}

FVector FNavigationPath::GetPathFindingStartLocation() const
{
	return SourceActor != NULL ? (SourceActorAsNavAgent != NULL ? SourceActorAsNavAgent->GetNavAgentLocation() : SourceActor->GetActorLocation()) : GetStartLocation();
}

void FNavigationPath::SetGoalActorObservation(const AActor& ActorToObserve, float TetherDistance)
{
	if (NavigationDataUsed.IsValid() == false)
	{
		// this mechanism is available only for navigation-generated paths
		UE_LOGF(LogNavigation, Warning, "Updating navigation path on goal actor's location change is available only for navigation-generated paths. Called for %ls"
			, *GetNameSafe(&ActorToObserve));
		return;
	}

	// register for path observing only if we weren't registered already
	const bool RegisterForPathUpdates = (GoalActor.IsValid() == false);
	GoalActor = &ActorToObserve;
	checkSlow(GoalActor.IsValid());
	GoalActorAsNavAgent = Cast<INavAgentInterface>(&ActorToObserve);
	GoalActorLocationTetherDistanceSq = FMath::Square(TetherDistance);
	bObservingGoalActor = true;
	UpdateLastRepathGoalLocation();

	if (RegisterForPathUpdates)
	{
		NavigationDataUsed->RegisterObservedPath(AsShared());
	}
}

void FNavigationPath::SetSourceActor(const AActor& InSourceActor)
{
	SourceActor = &InSourceActor;
	SourceActorAsNavAgent = Cast<INavAgentInterface>(&InSourceActor);
}

void FNavigationPath::UpdateLastRepathGoalLocation()
{
	if (GoalActor.IsValid())
	{
		GoalActorLastLocation = GoalActorAsNavAgent ? GoalActorAsNavAgent->GetNavAgentLocation() : GoalActor->GetActorLocation();
	}
}

EPathObservationResult::Type FNavigationPath::TickPathObservation()
{
	if (bObservingGoalActor == false || GoalActor.IsValid() == false)
	{
		return EPathObservationResult::NoLongerObserving;
	}

	const FVector GoalLocation = GoalActorAsNavAgent != NULL ? GoalActorAsNavAgent->GetNavAgentLocation() : GoalActor->GetActorLocation();
	return FVector::DistSquared(GoalLocation, GoalActorLastLocation) <= GoalActorLocationTetherDistanceSq ? EPathObservationResult::NoChange : EPathObservationResult::RequestRepath;
}

void FNavigationPath::DisableGoalActorObservation()
{
	GoalActor = NULL;
	GoalActorAsNavAgent = NULL;
	GoalActorLocationTetherDistanceSq = -1.f;
	bObservingGoalActor = false;
}

void FNavigationPath::Invalidate()
{
	if (!bIgnoreInvalidation)
	{
		bUpToDate = false;
		ObserverDelegate.Broadcast(this, ENavPathEvent::Invalidated);
		if (bDoAutoUpdateOnInvalidation && NavigationDataUsed.IsValid())
		{
			bWaitingForRepath = true;
			NavigationDataUsed->RequestRePath(AsShared(), ENavPathUpdateType::NavigationChanged);
		}
	}
}

void FNavigationPath::RePathFailed()
{
	ObserverDelegate.Broadcast(this, ENavPathEvent::RePathFailed);
	bWaitingForRepath = false;
}

void FNavigationPath::ResetForRepath()
{
	InternalResetNavigationPath();
}

void FNavigationPath::DebugDraw(const ANavigationData* NavData, FColor PathColor, UCanvas* Canvas, bool bPersistent, float LifeTime, const uint32 NextPathPointIndex) const
{
#if ENABLE_DRAW_DEBUG

	static const FColor Grey(100,100,100);
	const int32 NumPathVerts = PathPoints.Num();

	UWorld* World = NavData->GetWorld();

	for (int32 VertIdx = 0; VertIdx < NumPathVerts-1; ++VertIdx)
	{
		// draw box at vert
		FVector const VertLoc = PathPoints[VertIdx].Location + NavigationDebugDrawing::PathOffset;
		DrawDebugSolidBox(World, VertLoc, NavigationDebugDrawing::PathNodeBoxExtent, VertIdx < int32(NextPathPointIndex) ? Grey : PathColor, bPersistent, LifeTime);

		// draw line to next loc
		FVector const NextVertLoc = PathPoints[VertIdx+1].Location + NavigationDebugDrawing::PathOffset;
		DrawDebugLine(World, VertLoc, NextVertLoc, VertIdx < int32(NextPathPointIndex)-1 ? Grey : PathColor, bPersistent
			, LifeTime, /*DepthPriority*/0
			, /*Thickness*/NavigationDebugDrawing::PathLineThickness);
	}

	// draw last vert
	if (NumPathVerts > 0)
	{
		DrawDebugBox(World, PathPoints[NumPathVerts-1].Location + NavigationDebugDrawing::PathOffset, FVector(15.), PathColor, bPersistent, LifeTime);
	}

	// if observing goal actor draw a radius and a line to the goal
	if (GoalActor.IsValid())
	{
		const FVector GoalLocation = GetGoalLocation() + NavigationDebugDrawing::PathOffset;
		const FVector EndLocation = GetEndLocation() + NavigationDebugDrawing::PathOffset;
		static const FVector CylinderHalfHeight = FVector::UpVector * 10.;
		DrawDebugCylinder(World, EndLocation - CylinderHalfHeight, EndLocation + CylinderHalfHeight, FMath::Sqrt(GoalActorLocationTetherDistanceSq), 16, PathColor, bPersistent, LifeTime);
		DrawDebugLine(World, EndLocation, GoalLocation, Grey, bPersistent, LifeTime);
	}

#endif
}

bool FNavigationPath::ContainsNode(NavNodeRef NodeRef) const
{
	for (int32 Index = 0; Index < PathPoints.Num(); Index++)
	{
		if (PathPoints[Index].NodeRef == NodeRef)
		{
			return true;
		}
	}

	return ShortcutNodeRefs.Find(NodeRef) != INDEX_NONE;
}

FVector::FReal FNavigationPath::GetLengthFromPosition(FVector SegmentStart, uint32 NextPathPointIndex) const
{
	if (NextPathPointIndex >= (uint32)PathPoints.Num())
	{
		return 0;
	}
	
	const uint32 PathPointsCount = PathPoints.Num();
	FVector::FReal PathDistance = 0.;

	for (uint32 PathIndex = NextPathPointIndex; PathIndex < PathPointsCount; ++PathIndex)
	{
		const FVector SegmentEnd = PathPoints[PathIndex].Location;
		PathDistance += FVector::Dist(SegmentStart, SegmentEnd);
		SegmentStart = SegmentEnd;
	}

	return PathDistance;
}

bool FNavigationPath::ContainsCustomLink(FNavLinkId LinkUniqueId) const
{
	if (LinkUniqueId == FNavLinkId::Invalid)
	{
		return false;
	}

	for (int32 i = 0; i < PathPoints.Num(); i++)
	{
		if (PathPoints[i].CustomNavLinkId == LinkUniqueId)
		{
			return true;
		}
	}

	return false;
}

bool FNavigationPath::ContainsAnyCustomLink() const
{
	for (int32 i = 0; i < PathPoints.Num(); i++)
	{
		if (PathPoints[i].CustomNavLinkId != FNavLinkId::Invalid)
		{
			return true;
		}
	}

	return false;
}

FORCEINLINE bool FNavigationPath::DoesPathIntersectBoxImplementation(const FBox& Box, const FVector& StartLocation, uint32 StartingIndex, int32* IntersectingSegmentIndex, FVector* AgentExtent) const
{	
	bool bIntersects = false;

	FVector Start = StartLocation;
	for (int32 PathPointIndex = int32(StartingIndex); PathPointIndex < PathPoints.Num(); ++PathPointIndex)
	{
		const FVector End = PathPoints[PathPointIndex].Location;
		if (FVector::DistSquared(Start, End) > SMALL_NUMBER)
		{
			const FVector Direction = (End - Start);

			FVector HitLocation, HitNormal;
			float HitTime;

			// If we have a valid AgentExtent, then we use an extent box to represent the path
			// Otherwise we use a line to represent the path
			if ((AgentExtent && FMath::LineExtentBoxIntersection(Box, Start, End, *AgentExtent, HitLocation, HitNormal, HitTime)) ||
				(!AgentExtent && FMath::LineBoxIntersection(Box, Start, End, Direction)))
			{
				bIntersects = true;
				if (IntersectingSegmentIndex != NULL)
				{
					*IntersectingSegmentIndex = PathPointIndex;
				}
				break;
			}
		}

		Start = End;
	}

	return bIntersects;
}

bool FNavigationPath::DoesIntersectBox(const FBox& Box, uint32 StartingIndex, int32* IntersectingSegmentIndex, FVector* AgentExtent) const
{
	// iterate over all segments and check if any intersects with given box
	bool bIntersects = false;
	int32 PathPointIndex = INDEX_NONE;

	if (PathPoints.Num() > 1 && PathPoints.IsValidIndex(int32(StartingIndex)))
	{
		bIntersects = DoesPathIntersectBoxImplementation(Box, PathPoints[StartingIndex].Location, StartingIndex + 1, IntersectingSegmentIndex, AgentExtent);
	}

	return bIntersects;
}

bool FNavigationPath::DoesIntersectBox(const FBox& Box, const FVector& AgentLocation, uint32 StartingIndex, int32* IntersectingSegmentIndex, FVector* AgentExtent) const
{
	// iterate over all segments and check if any intersects with given box
	bool bIntersects = false;
	int32 PathPointIndex = INDEX_NONE;

	if (PathPoints.Num() > 1 && PathPoints.IsValidIndex(int32(StartingIndex)))
	{
		bIntersects = DoesPathIntersectBoxImplementation(Box, AgentLocation, StartingIndex, IntersectingSegmentIndex, AgentExtent);
	}

	return bIntersects;
}

FVector FNavigationPath::GetSegmentDirection(uint32 SegmentEndIndex) const
{
	FVector Result = FNavigationSystem::InvalidLocation;

	// require at least two points
	if (PathPoints.Num() > 1)
	{
		if (PathPoints.IsValidIndex(SegmentEndIndex))
		{
			if (SegmentEndIndex > 0)
			{
				Result = (PathPoints[SegmentEndIndex].Location - PathPoints[SegmentEndIndex - 1].Location).GetSafeNormal();
			}
			else
			{
				// for '0'-th segment returns same as for 1st segment 
				Result = (PathPoints[1].Location - PathPoints[0].Location).GetSafeNormal();
			}
		}
		else if (SegmentEndIndex >= uint32(GetPathPoints().Num()))
		{
			// in this special case return direction of last segment
			Result = (PathPoints[PathPoints.Num() - 1].Location - PathPoints[PathPoints.Num() - 2].Location).GetSafeNormal();
		}
	}

	return Result;
}

FBasedPosition FNavigationPath::GetPathPointLocation(uint32 Index) const
{
	FBasedPosition BasedPt;
	if (PathPoints.IsValidIndex(Index))
	{
		BasedPt.Base = Base.Get();
		BasedPt.Position = PathPoints[Index].Location;
	}

	return BasedPt;
}

#if ENABLE_VISUAL_LOG

void FNavigationPath::DescribeSelfToVisLog(FVisualLogEntry* Snapshot) const 
{
	if (Snapshot == nullptr)
	{
		return;
	}

	const int32 NumPathVerts = PathPoints.Num();
	FVisualLogShapeElement Element(EVisualLoggerShapeElement::Path);
	Element.Category = LogNavigation.GetCategoryName();
	Element.SetColor(FColorList::Green);
	Element.Points.Reserve(NumPathVerts);
	Element.Thickness = 3;
	
	for (int32 VertIdx = 0; VertIdx < NumPathVerts; ++VertIdx)
	{
		Element.Points.Add(PathPoints[VertIdx].Location + NavigationDebugDrawing::PathOffset);
	}

	Snapshot->ElementsToDraw.Add(Element);
}

FString FNavigationPath::GetDescription() const
{
	return FString::Printf(TEXT("NotifyPathUpdate points:%d valid:%s")
		, PathPoints.Num()
		, IsValid() ? TEXT("yes") : TEXT("no"));
}

#endif // ENABLE_VISUAL_LOG

//----------------------------------------------------------------------//
// UNavigationPath
//----------------------------------------------------------------------//

UNavigationPath::UNavigationPath(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bIsValid(false)
	, bDebugDrawingEnabled(false)
	, DebugDrawingColor(FColor::White)
	, SharedPath(NULL)
{	
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		PathObserver = FNavigationPath::FPathObserverDelegate::FDelegate::CreateUObject(this, &UNavigationPath::OnPathEvent);
	}
}

void UNavigationPath::BeginDestroy()
{
	if (SharedPath.IsValid())
	{
		SharedPath->RemoveObserver(PathObserverDelegateHandle);
	}
	Super::BeginDestroy();
}

void UNavigationPath::OnPathEvent(FNavigationPath* UpdatedPath, ENavPathEvent::Type PathEvent)
{
	if (UpdatedPath == SharedPath.Get())
	{
		PathUpdatedNotifier.Broadcast(this, PathEvent);
		if (SharedPath.IsValid() && SharedPath->IsValid())
		{
			bIsValid = true;
			SetPathPointsFromPath(*UpdatedPath);
		}
		else
		{
			bIsValid = false;
		}
	}
}

FString UNavigationPath::GetDebugString() const
{
	check((SharedPath.IsValid() && SharedPath->IsValid()) == !!bIsValid);
	if (!bIsValid)
	{
		return TEXT("Invalid path");
	}

	return FString::Printf(TEXT("Path: points %d%s%s"), SharedPath->GetPathPoints().Num()
		, SharedPath->IsPartial() ? TEXT(", partial") : TEXT("")
		, SharedPath->IsUpToDate() ? TEXT("") : TEXT(", OUT OF DATE!")
		);
}

void UNavigationPath::DrawDebug(UCanvas* Canvas, APlayerController*)
{
	if (SharedPath.IsValid())
	{
		SharedPath->DebugDraw(SharedPath->GetNavigationDataUsed(), DebugDrawingColor, Canvas, /*bPersistent=*/false, -1.f);
	}
}

void UNavigationPath::EnableDebugDrawing(bool bShouldDrawDebugData, FLinearColor PathColor)
{
	DebugDrawingColor = PathColor.ToFColor(true);

	if (bDebugDrawingEnabled == bShouldDrawDebugData)
	{
		return;
	}

	bDebugDrawingEnabled = bShouldDrawDebugData;
	if (bShouldDrawDebugData)
	{
		DrawDebugDelegateHandle = UDebugDrawService::Register(TEXT("Navigation"), FDebugDrawDelegate::CreateUObject(this, &UNavigationPath::DrawDebug));
	}
	else
	{
		UDebugDrawService::Unregister(DrawDebugDelegateHandle);
	}
}

void UNavigationPath::EnableRecalculationOnInvalidation(const ENavigationOptionFlag DoRecalculation)
{
	if (DoRecalculation != RecalculateOnInvalidation)
	{
		RecalculateOnInvalidation = DoRecalculation;
		if (!!bIsValid && RecalculateOnInvalidation != ENavigationOptionFlag::Default)
		{
			SharedPath->EnableRecalculationOnInvalidation(RecalculateOnInvalidation == ENavigationOptionFlag::Enable);
		}
	}
}

double UNavigationPath::GetPathLength() const
{
	check((SharedPath.IsValid() && SharedPath->IsValid()) == !!bIsValid);
	return !!bIsValid ? SharedPath->GetLength() : -1.;
}

double UNavigationPath::GetLengthFromPosition(FVector Position, int32 NextPathPointIndex) const
{
	check((SharedPath.IsValid() && SharedPath->IsValid()) == !!bIsValid);

	if (!bIsValid)
	{
		return -1.;
	}

	if (NextPathPointIndex < 0)
	{
		UE_LOGF(LogNavigation, Warning, "Negative PathPointIndex passed in (%d).", NextPathPointIndex);
		return 0.;
	}

	return SharedPath->GetLengthFromPosition(Position, NextPathPointIndex);
}

double UNavigationPath::GetPathCost() const
{
	check((SharedPath.IsValid() && SharedPath->IsValid()) == !!bIsValid);
	return !!bIsValid ? SharedPath->GetCost() : -1.;
}

bool UNavigationPath::IsPartial() const
{
	check((SharedPath.IsValid() && SharedPath->IsValid()) == !!bIsValid);
	return !!bIsValid && SharedPath->IsPartial();
}

bool UNavigationPath::IsValid() const
{
	check((SharedPath.IsValid() && SharedPath->IsValid()) == !!bIsValid);
	return !!bIsValid;
}

bool UNavigationPath::IsStringPulled() const
{
	return false;
}

void UNavigationPath::SetPath(FNavPathSharedPtr NewSharedPath)
{
	FNavigationPath* NewPath = NewSharedPath.Get();
	if (SharedPath.Get() != NewPath)
	{
		if (SharedPath.IsValid())
		{
			SharedPath->RemoveObserver(PathObserverDelegateHandle);
		}
		SharedPath = NewSharedPath;
		if (NewPath != NULL)
		{
			PathObserverDelegateHandle = NewPath->AddObserver(PathObserver);

			if (RecalculateOnInvalidation != ENavigationOptionFlag::Default)
			{
				NewPath->EnableRecalculationOnInvalidation(RecalculateOnInvalidation == ENavigationOptionFlag::Enable);
			}
			
			SetPathPointsFromPath(*NewPath);
		}
		else
		{
			PathPoints.Reset();
		}

		OnPathEvent(NewPath, NewPath != NULL ? ENavPathEvent::NewPath : ENavPathEvent::Cleared);
	}
}

void UNavigationPath::SetPathPointsFromPath(FNavigationPath& NativePath)
{
	PathPoints.Reset(NativePath.GetPathPoints().Num());
	for (const auto& PathPoint : NativePath.GetPathPoints())
	{
		PathPoints.Add(PathPoint.Location);
	}
}