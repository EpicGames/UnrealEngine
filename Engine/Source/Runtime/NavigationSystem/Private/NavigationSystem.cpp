// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationSystem.h"
#include "AbstractNavData.h"
#include "AI/NavDataGenerator.h"
#include "AI/Navigation/NavAgentInterface.h"
#include "AI/Navigation/NavigationDataChunk.h"
#include "AI/Navigation/NavigationDirtyArea.h"
#include "AI/Navigation/NavigationDirtyElement.h"
#include "AI/Navigation/NavigationInvokerInterface.h"
#include "AI/Navigation/NavigationInvokerPriority.h"
#include "AI/Navigation/NavigationElement.h"
#include "AI/Navigation/NavRelevantInterface.h"
#include "AI/NavigationModifier.h"
#include "Components/PrimitiveComponent.h"
#include "CrowdManagerBase.h"
#include "Engine/Engine.h"
#include "Engine/LocalPlayer.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "GameFramework/Controller.h"
#include "GameFramework/Pawn.h"
#include "Logging/MessageLog.h"
#include "Misc/ScopeLock.h"
#include "Modules/ModuleManager.h"
#include "NavAreas/NavArea.h"
#include "NavAreas/NavArea_Default.h"
#include "NavAreas/NavArea_Obstacle.h"
#include "NavAreas/NavAreaMeta_SwitchByAgent.h"
#include "NavFilters/NavigationQueryFilter.h"
#include "NavigationDataHandler.h"
#include "NavigationInvokerComponent.h"
#include "NavigationObjectRepository.h"
#include "NavigationOctree.h"
#include "NavigationPath.h"
#include "NavLinkCustomInterface.h"
#include "NavMesh/NavMeshBoundsVolume.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Stats/StatsMisc.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UObjectThreadContext.h"
#include "VisualLogger/VisualLogger.h"

#if WITH_RECAST
#include "NavMesh/RecastGeometryExport.h"
#include "NavMesh/RecastHelpers.h"
#include "NavMesh/RecastNavMesh.h"
#endif // WITH_RECAST

#if WITH_EDITOR
#include "Editor.h"
#include "AssetCompilingManager.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "LevelEditor.h"
#include "Misc/MessageDialog.h"
#include "WorldPartition/WorldPartition.h"
#include "SourceControlHelpers.h"
#include "HAL/PlatformFileManager.h"
#endif // WITH_EDITOR


#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationSystem)


static const uint32 INITIAL_ASYNC_QUERIES_SIZE = 32;
static const uint32 REGISTRATION_QUEUE_SIZE = 16;	// and we'll not reallocate

#define LOCTEXT_NAMESPACE "Navigation"

DECLARE_CYCLE_STAT(TEXT("Nav Tick: mark dirty"), STAT_Navigation_TickMarkDirty, STATGROUP_Navigation);
DECLARE_CYCLE_STAT(TEXT("Nav Tick: async build"), STAT_Navigation_TickAsyncBuild, STATGROUP_Navigation);
DECLARE_CYCLE_STAT(TEXT("Nav Tick: dispatch async pathfinding results"), STAT_Navigation_DispatchAsyncPathfindingResults, STATGROUP_Navigation);
DECLARE_CYCLE_STAT(TEXT("Nav Tick: async pathfinding"), STAT_Navigation_TickAsyncPathfinding, STATGROUP_Navigation);
DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("NavOctree bookkeeping"), STAT_NavOctreeBookkeeping, STATGROUP_Navigation, EStatFlags::Verbose);

//----------------------------------------------------------------------//
// Stats
//----------------------------------------------------------------------//

DEFINE_STAT(STAT_Navigation_QueriesTimeSync);
DEFINE_STAT(STAT_Navigation_RequestingAsyncPathfinding);
DEFINE_STAT(STAT_Navigation_PathfindingSync);
DEFINE_STAT(STAT_Navigation_PathfindingAsync);
DEFINE_STAT(STAT_Navigation_TileNavAreaSorting);
DEFINE_STAT(STAT_Navigation_TileGeometryExportToObjAsync);
DEFINE_STAT(STAT_Navigation_TileVoxelFilteringAsync);
DEFINE_STAT(STAT_Navigation_TileBuildAsync);
DEFINE_STAT(STAT_Navigation_TileBuildPreparationSync);
DEFINE_STAT(STAT_Navigation_BSPExportSync);
DEFINE_STAT(STAT_Navigation_GatheringNavigationModifiersSync);
DEFINE_STAT(STAT_Navigation_ActorsGeometryExportSync);
DEFINE_STAT(STAT_Navigation_ProcessingActorsForNavMeshBuilding);
DEFINE_STAT(STAT_Navigation_AdjustingNavLinks);
DEFINE_STAT(STAT_Navigation_RegisterNavOctreeElement);
DEFINE_STAT(STAT_Navigation_UnregisterNavOctreeElement);
DEFINE_STAT(STAT_Navigation_AddingActorsToNavOctree);
DEFINE_STAT(STAT_Navigation_RecastAddGeneratedTiles);
DEFINE_STAT(STAT_Navigation_RecastAddGeneratedTileLayer);
DEFINE_STAT(STAT_Navigation_RecastTick);
DEFINE_STAT(STAT_Navigation_RecastPathfinding);
DEFINE_STAT(STAT_Navigation_RecastTestPath);
DEFINE_STAT(STAT_Navigation_StoringCompressedLayers);
DEFINE_STAT(STAT_Navigation_CreateTileGenerator);
DEFINE_STAT(STAT_Navigation_DoWork);
DEFINE_STAT(STAT_Navigation_RemoveLayers);
DEFINE_STAT(STAT_Navigation_RecastBuildCompressedLayers);
DEFINE_STAT(STAT_Navigation_RecastCreateHeightField);
DEFINE_STAT(STAT_Navigation_RecastComputeRasterizationMasks);
DEFINE_STAT(STAT_Navigation_RecastRasterizeTriangles);
DEFINE_STAT(STAT_Navigation_RecastVoxelFilter);
DEFINE_STAT(STAT_Navigation_RecastFilter);
DEFINE_STAT(STAT_Navigation_FilterLedgeSpans);
DEFINE_STAT(STAT_Navigation_RecastBuildCompactHeightField);
DEFINE_STAT(STAT_Navigation_RecastErodeWalkable);
DEFINE_STAT(STAT_Navigation_RecastBuildLayers);
DEFINE_STAT(STAT_Navigation_RecastBuildTileCache);
DEFINE_STAT(STAT_Navigation_RecastBuildPolyMesh);
DEFINE_STAT(STAT_Navigation_RecastBuildPolyDetail);
DEFINE_STAT(STAT_Navigation_RecastGatherOffMeshData);
DEFINE_STAT(STAT_Navigation_RecastCreateNavMeshData);
DEFINE_STAT(STAT_Navigation_RecastMarkAreas);
DEFINE_STAT(STAT_Navigation_RecastBuildContours);
DEFINE_STAT(STAT_Navigation_RecastBuildNavigation);
DEFINE_STAT(STAT_Navigation_GenerateNavigationDataLayer);
DEFINE_STAT(STAT_Navigation_RecastBuildLinks);
DEFINE_STAT(STAT_Navigation_RecastBuildLinks_FindEdges);
DEFINE_STAT(STAT_Navigation_RecastBuildLinks_Sample);
DEFINE_STAT(STAT_Navigation_RecastBuildRegions);
DEFINE_STAT(STAT_Navigation_UpdateNavOctree);
DEFINE_STAT(STAT_Navigation_CollisionTreeMemory);
DEFINE_STAT(STAT_Navigation_NavDataMemory);
DEFINE_STAT(STAT_Navigation_TileCacheMemory);
DEFINE_STAT(STAT_Navigation_OutOfNodesPath);
DEFINE_STAT(STAT_Navigation_PartialPath);
DEFINE_STAT(STAT_Navigation_CumulativeBuildTime);
DEFINE_STAT(STAT_Navigation_BuildTime);
DEFINE_STAT(STAT_Navigation_OffsetFromCorners);
DEFINE_STAT(STAT_Navigation_PathVisibilityOptimisation);
DEFINE_STAT(STAT_Navigation_ObservedPathsCount);
DEFINE_STAT(STAT_Navigation_RecastMemory);

DEFINE_STAT(STAT_Navigation_DetourTEMP);
DEFINE_STAT(STAT_Navigation_DetourPERM);
DEFINE_STAT(STAT_Navigation_DetourPERM_AVOIDANCE);
DEFINE_STAT(STAT_Navigation_DetourPERM_CROWD);
DEFINE_STAT(STAT_Navigation_DetourPERM_LOOKUP);
DEFINE_STAT(STAT_Navigation_DetourPERM_NAVQUERY);
DEFINE_STAT(STAT_Navigation_DetourPERM_NAVMESH);
DEFINE_STAT(STAT_Navigation_DetourPERM_NODE_POOL);
DEFINE_STAT(STAT_Navigation_DetourPERM_PATH_CORRIDOR);
DEFINE_STAT(STAT_Navigation_DetourPERM_PATH_QUEUE);
DEFINE_STAT(STAT_Navigation_DetourPERM_PROXY_GRID);
DEFINE_STAT(STAT_Navigation_DetourPERM_TILE_DATA);
DEFINE_STAT(STAT_Navigation_DetourPERM_TILE_DYNLINK);
DEFINE_STAT(STAT_Navigation_DetourPERM_TILE_DYNLINK_CLUSTER);
DEFINE_STAT(STAT_Navigation_DetourPERM_TILES);
DEFINE_STAT(STAT_Navigation_DetourPERM_TILE_LINK_BUILDER);

DEFINE_STAT(STAT_DetourTileMemory);
DEFINE_STAT(STAT_DetourTileMeshHeaderMemory);
DEFINE_STAT(STAT_DetourTileNavVertsMemory);
DEFINE_STAT(STAT_DetourTileNavPolysMemory);
DEFINE_STAT(STAT_DetourTileLinksMemory);
DEFINE_STAT(STAT_DetourTileDetailMeshesMemory);
DEFINE_STAT(STAT_DetourTileDetailVertsMemory);
DEFINE_STAT(STAT_DetourTileDetailTrisMemory);
DEFINE_STAT(STAT_DetourTileBVTreeMemory);
DEFINE_STAT(STAT_DetourTileOffMeshConsMemory);
DEFINE_STAT(STAT_DetourTileOffMeshSegsMemory);
DEFINE_STAT(STAT_DetourTileClustersMemory);
DEFINE_STAT(STAT_DetourTilePolyClustersMemory);

CSV_DEFINE_CATEGORY(NavigationSystem, false);
CSV_DEFINE_CATEGORY(NavigationBuildDetailed, true);
CSV_DEFINE_CATEGORY(NavTasksDelays, true);
CSV_DEFINE_CATEGORY(NavTasks, true);
CSV_DEFINE_CATEGORY(NavInvokers, true);

namespace FNavigationSystem
{	
#if ALLOW_TIME_SLICE_DEBUG
	const FName DebugTimeSliceDefaultSectionName("DefaultSection");
#endif // ALLOW_TIME_SLICE_DEBUG
}

namespace UE::Navigation::Private
{

static FAutoConsoleCommandWithWorldArgsAndOutputDevice CmdNavDirtyAreaAroundPlayer(
	TEXT("ai.debug.nav.DirtyAreaAroundPlayer"),
	TEXT("Dirty all tiles in a square area around the player using provided value as extent (in cm), using 10 meters if not specified."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, const UWorld* World, FOutputDevice& OutputDevice)
		{
			if (const APlayerController* Player = World->GetFirstPlayerController())
			{
				APawn* Pawn = Player->GetPawn();
				if (!Pawn)
				{
					return;
				}

				const FVector Center = Pawn->GetActorLocation();

				FVector::FReal Extent = 1000;
				if (Args.Num() > 0)
				{
					if (FCString::IsNumeric(*Args[0]))
					{
						Extent = FCString::Atod(*Args[0]);
					}
					else
					{
						OutputDevice.Log(ELogVerbosity::Error, TEXT("Command failed since first parameter is not a valid numerical value"));
						return;
					}
				}

				UNavigationSystemV1::NavigationDirtyEvent.Broadcast(FBox(Center - FVector(Extent), Center + FVector(Extent)));
			}
			else
			{
				OutputDevice.Log(ELogVerbosity::Error, TEXT("Command failed since it was unable to find a local player"));
			}
		}
	));


static FAutoConsoleCommandWithWorldArgsAndOutputDevice CmdDumpOctreeElements(
	TEXT("ai.debug.nav.DumpOctreeElements"),
	TEXT("Iterates through all nodes of the navigation octree and log details about each element to the output device."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, const UWorld* World, FOutputDevice& OutputDevice)
		{
			if (const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World))
			{
				if (const FNavigationOctree* Octree = NavSys->GetNavOctree())
				{
					int32 NumElements = 0;
					Octree->FindNodesWithPredicate(
						[&NumElements](FNavigationOctree::FNodeIndex /*ParentNodeIndex*/, FNavigationOctree::FNodeIndex /*NodeIndex*/, const FBoxCenterAndExtent&) { return true; },
						[&NumElements, &OutputDevice, &Octree](FNavigationOctree::FNodeIndex /*ParentNodeIndex*/, const FNavigationOctree::FNodeIndex NodeIndex, const FBoxCenterAndExtent&)
						{
							for (const FNavigationOctreeElement& OctreeElement : Octree->GetElementsForNode(NodeIndex))
							{
								NumElements++;
								OutputDevice.Logf(ELogVerbosity::Type::Log, TEXT("%s bounds: [%s] parent:'%s'"),
									*OctreeElement.GetSourceElement()->GetPathName(),
									*OctreeElement.Bounds.ToString(),
									*GetNameSafe(OctreeElement.GetSourceElement().Get().GetNavigationParent().Get()));
							};
						});

					OutputDevice.Logf(ELogVerbosity::Type::Log, TEXT("Total: %d elements"), NumElements);
				}
				else
				{
					OutputDevice.Log(ELogVerbosity::Error, TEXT("Octree not used in the current configuration"));
				}
			}
			else
			{
				OutputDevice.Log(ELogVerbosity::Error, TEXT("Command failed since it was unable to find the navigation system"));
			}
		}
	));

static FAutoConsoleCommandWithWorldArgsAndOutputDevice CmdLogInvokers(
	TEXT("ai.debug.nav.LogInvokers"),
	TEXT("Iterate through all the navigation invokers and log details about each of them to the output device."),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, const UWorld* World, FOutputDevice& OutputDevice)
		{
#if !UE_BUILD_SHIPPING
			if (const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World))
			{
				NavSys->DebugLogInvokers(OutputDevice);
			}
#endif // !UE_BUILD_SHIPPING
		}
	));

const FNavDataConfig& GetFallbackNavDataConfig()
{
	static FNavDataConfig FallbackNavDataConfig(FNavigationSystem::FallbackAgentRadius, FNavigationSystem::FallbackAgentHeight);
	return FallbackNavDataConfig;
}

FORCEINLINE bool IsValidExtent(const FVector& Extent)
{
	return Extent != INVALID_NAVEXTENT;
}

static bool bComponentShouldWaitForActorToRegister = true;
static FAutoConsoleVariableRef CVarRollbackNavigationComponentShouldWaitForActorToRegister(
	TEXT("UE.Rollback.Navigation.ComponentShouldWaitForActorToRegister"), bComponentShouldWaitForActorToRegister,
	TEXT("Components registration to navigation octree will be postponed until owning actor is registered to the octree."
		"\nCategory: [Navigation]"),
	ECVF_Default);

bool ShouldComponentWaitForActorToRegister(const UActorComponent* Comp)
{
	if (bComponentShouldWaitForActorToRegister)
	{
		// Ignore operations on components until the actor has registered all its components to the scene.
		// Then, Actor registration to the navigation octree will also registers its components to the octree.
		if (const AActor* Owner = Comp->GetOwner())
		{
			if (!Owner->HasActorRegisteredAllComponents())
			{
				return true;
			}
		}
	}

	return false;
}

} // UE::Navigation::Private

//----------------------------------------------------------------------//
// FNavigationSystem
//----------------------------------------------------------------------//
namespace FNavigationSystem
{
FCustomLinkOwnerInfo::FCustomLinkOwnerInfo(INavLinkCustomInterface* Link)
{
	LinkInterface = Link;
	LinkOwner = Link->GetLinkOwner();
}

bool ShouldLoadNavigationOnClient(ANavigationData& NavData)
{
	const UWorld* World = NavData.GetWorld();

	if (World && World->GetNavigationSystem())
	{
		const UNavigationSystemV1* NavSys = Cast<UNavigationSystemV1>(World->GetNavigationSystem());
		return NavSys && NavSys->ShouldLoadNavigationOnClient(&NavData);
	}
	else if (GEngine->NavigationSystemClass && GEngine->NavigationSystemClass->IsChildOf<UNavigationSystemV1>())
	{
		const UNavigationSystemV1* NavSysCDO = GEngine->NavigationSystemClass->GetDefaultObject<const UNavigationSystemV1>();
		return NavSysCDO && NavSysCDO->ShouldLoadNavigationOnClient(&NavData);
	}
	return false;
}

void MakeAllComponentsNeverAffectNav(AActor& Actor)
{
	const TSet<UActorComponent*> Components = Actor.GetComponents();
	for (UActorComponent* ActorComp : Components)
	{
		ActorComp->SetCanEverAffectNavigation(false);
	}
}

} // namespace FNavigationSystem

//----------------------------------------------------------------------//
// NavigationDebugDrawing
//----------------------------------------------------------------------//
namespace NavigationDebugDrawing
{
	const float PathLineThickness = 3.f;
	const FVector PathOffset(0,0,15);
	const FVector PathNodeBoxExtent(16.f);
}

//----------------------------------------------------------------------//
// FNavigationInvokerRaw
//----------------------------------------------------------------------//
FNavigationInvokerRaw::FNavigationInvokerRaw(const FVector& InLocation, float Min, float Max, const FNavAgentSelector& InSupportedAgents, ENavigationInvokerPriority InPriority)
: Location(InLocation)
, RadiusMin(Min)
, RadiusMax(Max)
, SupportedAgents(InSupportedAgents)
, Priority(InPriority)
{
}

//----------------------------------------------------------------------//
// FNavigationInvoker
//----------------------------------------------------------------------//
FNavigationInvoker::FNavigationInvoker()
: Actor(nullptr)
, Object(nullptr)
, GenerationRadius(0)
, RemovalRadius(0)
, Priority(ENavigationInvokerPriority::Default)
{
	SupportedAgents.MarkInitialized();
}

FNavigationInvoker::FNavigationInvoker(AActor& InActor, float InGenerationRadius, float InRemovalRadius, const FNavAgentSelector& InSupportedAgents, ENavigationInvokerPriority InPriority)
: Actor(&InActor)
, Object(nullptr)
, GenerationRadius(InGenerationRadius)
, RemovalRadius(InRemovalRadius)
, SupportedAgents(InSupportedAgents)
, Priority(InPriority)
{
	SupportedAgents.MarkInitialized();
}

FNavigationInvoker::FNavigationInvoker(INavigationInvokerInterface& InObject, float InGenerationRadius, float InRemovalRadius, const FNavAgentSelector& InSupportedAgents, ENavigationInvokerPriority InPriority)
: Actor(nullptr)
, Object(&InObject)
, GenerationRadius(InGenerationRadius)
, RemovalRadius(InRemovalRadius)
, SupportedAgents(InSupportedAgents)
, Priority(InPriority)
{
}

FString FNavigationInvoker::GetName() const
{
	/** We are using IsExplicitlyNull to know which one of the Actor or the Object was set at construction */
	if (!Actor.IsExplicitlyNull())
	{
		return GetNameSafe(Actor.Get());
	}
	else
	{
		return GetNameSafe(Object.GetObject());
	}
}

bool FNavigationInvoker::GetLocation(FVector& OutLocation) const
{
	/** We are using IsExplicitlyNull to know which one of the Actor or the Object was set at construction */
	if (!Actor.IsExplicitlyNull())
	{
		if (const AActor* ActorPtr = Actor.Get())
		{
			OutLocation = ActorPtr->GetActorLocation();
			return true;
		}
	}
	else
	{
		if (const INavigationInvokerInterface* InvokerInterface = Object.Get())
		{
			OutLocation = InvokerInterface->GetNavigationInvokerLocation();
			return true;
		}
	}

	return false;
}

//----------------------------------------------------------------------//
// helpers
//----------------------------------------------------------------------//
namespace
{
#if ENABLE_VISUAL_LOG
	void NavigationDataDump(const UObject* Object, const FName& CategoryName, const ELogVerbosity::Type Verbosity, const FBox& Box, const UWorld& World, FVisualLogEntry& CurrentEntry)
	{
		const ANavigationData* MainNavData = FNavigationSystem::GetCurrent<UNavigationSystemV1>(&World)->GetDefaultNavDataInstance();
		const FNavDataGenerator* Generator = MainNavData ? MainNavData->GetGenerator() : nullptr;
		if (Generator)
		{
			Generator->GrabDebugSnapshot(&CurrentEntry, (!Box.IsValid || FMath::IsNearlyZero(Box.GetVolume())) ? MainNavData->GetBounds().ExpandBy(FVector(20, 20, 20)) : Box, CategoryName, Verbosity);
		}
	}
#endif // ENABLE_VISUAL_LOG
}

void FNavRegenTimeSlicer::SetupTimeSlice(double SliceDuration)
{
	RemainingDuration = OriginalDuration = SliceDuration; 
	StartTime = TimeLastTested = 0.;
	bTimeSliceFinishedCached = false; 
}

void FNavRegenTimeSlicer::StartTimeSlice()
{
	ensureMsgf(!bTimeSliceFinishedCached, TEXT("Starting a time slice that has already been tested as finished! Call SetupTimeSlice() before calling StartTimeSlice() again!"));
	ensureMsgf(RemainingDuration > 0., TEXT("Attempting to start a time slice that has zero duration!"));

	TimeLastTested = StartTime = FPlatformTime::Seconds();
}
 
void FNavRegenTimeSlicer::EndTimeSliceAndAdjustDuration()
{
	RemainingDuration = FMath::Max(RemainingDuration - (TimeLastTested - StartTime), 0.);
}

#if ALLOW_TIME_SLICE_DEBUG
void FNavRegenTimeSlicer::DebugSetLongTimeSliceData(TFunction<void(FName, double)> LongTimeSliceFunction, double LongTimeSliceDuration) const
{
	DebugLongTimeSliceFunction = LongTimeSliceFunction;
	DebugLongTimeSliceDuration = LongTimeSliceDuration;
}

void FNavRegenTimeSlicer::DebugResetLongTimeSliceFunction() const
{
	DebugLongTimeSliceFunction.Reset();
}

#endif // ALLOW_TIME_SLICE_DEBUG

bool FNavRegenTimeSlicer::TestTimeSliceFinished() const
{
	ensureMsgf(!bTimeSliceFinishedCached, TEXT("Testing time slice is finished when we have already confirmed that!"));

	const double Time = FPlatformTime::Seconds();

#if ALLOW_TIME_SLICE_DEBUG
	const double TimeSinceLastTested = Time - TimeLastTested;
	if (TimeSinceLastTested >= DebugLongTimeSliceDuration)
	{
		if (ensureMsgf(DebugLongTimeSliceFunction, TEXT("DebugLongTimeSliceFunction should be setup! Call DebugSetLongTimeSliceData() prior to TestTimeSliceFinished()!")))
		{
			DebugLongTimeSliceFunction(DebugSectionName, TimeSinceLastTested);
		}
	}

	// Reset SectionDebugName
	DebugSectionName = FNavigationSystem::DebugTimeSliceDefaultSectionName;
#endif // ALLOW_TIME_SLICE_DEBUG

	TimeLastTested = Time;

	bTimeSliceFinishedCached = (TimeLastTested - StartTime) >= RemainingDuration;
	return bTimeSliceFinishedCached;
}

void FNavRegenTimeSliceManager::ResetTileWaitTimeArrays(const TArray<TObjectPtr<ANavigationData>>& NavDataSet)
{
	TileWaitTimes.SetNum(NavDataSet.Num());
	for (TArray<double>& Array : TileWaitTimes)
	{
		Array.Empty();
	}
}

void FNavRegenTimeSliceManager::PushTileWaitTime(const int32 NavDataIndex, const double NewTime)
{
	if (TileWaitTimes.IsValidIndex(NavDataIndex))
	{
		TileWaitTimes[NavDataIndex].Add(NewTime);	
	}
}

#if !UE_BUILD_SHIPPING
void FNavRegenTimeSliceManager::ResetTileHistoryData(const TArray<TObjectPtr<ANavigationData>>& NavDataSet)
{
	TileHistoryData.SetNum(NavDataSet.Num());
	for (TArray<FTileHistoryData>& HistoryData : TileHistoryData)
	{
		HistoryData.Empty();
	}
	TileHistoryStartTime = FPlatformTime::Seconds();
}

void FNavRegenTimeSliceManager::PushTileHistoryData(const int32 NavDataIndex, const FTileHistoryData& TileData)
{
	if (TileHistoryData.IsValidIndex(NavDataIndex))
	{
		TileHistoryData[NavDataIndex].Add(TileData);	
	}
}
#endif // UE_BUILD_SHIPPING

double FNavRegenTimeSliceManager::GetAverageTileWaitTime(const int32 NavDataIndex) const
{
	if (!TileWaitTimes.IsValidIndex(NavDataIndex))
	{
		return 0.;
	}
	
	double Total = 0.;
	const TArray<double>& TimeArray = TileWaitTimes[NavDataIndex];
	if (TimeArray.Num() == 0)
	{
		return 0.;			
	}
		
	for (const double Time : TimeArray)
	{
		Total += Time;
	}
	return Total / TimeArray.Num();
}

void FNavRegenTimeSliceManager::ResetTileWaitTime(const int32 NavDataIndex)
{
	if (TileWaitTimes.IsValidIndex(NavDataIndex))
	{
		TileWaitTimes[NavDataIndex].Reset();			
	}
}

FNavRegenTimeSliceManager::FNavRegenTimeSliceManager()
	: MinTimeSliceDuration(0.00075)
	, MaxTimeSliceDuration(0.004)
	, FrameNumOld(TNumericLimits<int64>::Max() - 1)
	, MaxDesiredTileRegenDuration(0.7f)
	, TimeLastCall(-1.f)
	, NavDataIdx(0)
#if WITH_RECAST && TIME_SLICE_NAV_REGEN
	, bDoTimeSlicedUpdate(true)
#else
	, bDoTimeSlicedUpdate(false)
#endif
{
}

void FNavRegenTimeSliceManager::CalcAverageDeltaTime(uint64 FrameNum)
{
	const double CurTime = FPlatformTime::Seconds();

	if ((FrameNumOld + 1) == FrameNum)
	{
		const double DeltaTime = CurTime - TimeLastCall;
		MovingWindowDeltaTime.PushValue(DeltaTime);
	}
	TimeLastCall = CurTime;
	FrameNumOld = FrameNum;
}

void FNavRegenTimeSliceManager::CalcTimeSliceDuration(const TArray<TObjectPtr<ANavigationData>>& NavDataSet, int32 NumTilesToRegen, const TArray<double>& CurrentTileRegenDurations)
{
	const float RawDeltaTimesAverage = FloatCastChecked<float>(MovingWindowDeltaTime.GetAverage(), UE::LWC::DefaultFloatPrecision);
	const float DeltaTimesAverage = (RawDeltaTimesAverage > 0.f) ? RawDeltaTimesAverage : (1.f / 30.f); //use default 33 ms

	const double TileRegenTimesAverage = (MovingWindowTileRegenTime.GetAverage() > 0.) ? MovingWindowTileRegenTime.GetAverage() : 0.0025; //use default of 2.5 milli secs to regen a full tile

	//calculate the max desired frames to regen all the tiles in PendingDirtyTiles
	const float MaxDesiredFramesToRegen = FMath::FloorToFloat(MaxDesiredTileRegenDuration / DeltaTimesAverage);

	//tiles to add to PendingDirtyTiles if the current tiles are taking longer than average to regen
	//we add 1 tile for however many times longer the current tile is taking compared with the moving window average
	int32 TilesToAddForLongCurrentTileRegen = 0;
	for (const double RegenDuration : CurrentTileRegenDurations)
	{
		TilesToAddForLongCurrentTileRegen += (RegenDuration > 0.) ? (static_cast<int32>(RegenDuration / TileRegenTimesAverage)) : 0;
	}

	//calculate the total processing time to regen all the tiles based on the moving window average
	const double TotalRegenTime = TileRegenTimesAverage * static_cast<double>(NumTilesToRegen + TilesToAddForLongCurrentTileRegen);

	//calculate the time slice per frame required to regen all the tiles clamped between MinTimeSliceDuration and MaxTimeSliceDuration
	const double NextRegenTimeSliceTime = FMath::Clamp(TotalRegenTime / static_cast<double>(MaxDesiredFramesToRegen), MinTimeSliceDuration, MaxTimeSliceDuration);

	TimeSlicer.SetupTimeSlice(NextRegenTimeSliceTime);

#if !UE_BUILD_SHIPPING
	CSV_CUSTOM_STAT(NavigationSystem, NavTileRegenTimeSliceTimeMs, static_cast<float>(NextRegenTimeSliceTime * 1000.), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(NavigationSystem, NavTileNumTilesToRegen, NumTilesToRegen, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(NavigationSystem, NavTilesToAddForLongCurrentTileRegen, TilesToAddForLongCurrentTileRegen, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(NavigationSystem, NavTileAvRegenTimeMs, static_cast<float>(MovingWindowTileRegenTime.GetAverage() * 1000.), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(NavigationSystem, NavTileAvRegenDeltaTimeMs, static_cast<float>(MovingWindowDeltaTime.GetAverage() * 1000.), ECsvCustomStatOp::Set);

	for (int32 NavDataIndex = 0; NavDataIndex < NavDataSet.Num(); ++NavDataIndex)
	{
		if (TileWaitTimes.IsValidIndex(NavDataIndex))
		{
#if CSV_PROFILER_STATS			
			const float WaitTime = static_cast<float>(GetAverageTileWaitTime(NavDataIndex) * 1000.);

			const FString StatName = FString::Printf(TEXT("NavTileAvTileWaitTimeMs_%s"), *GetNameSafe(NavDataSet[NavDataIndex])); 
			FCsvProfiler::RecordCustomStat(*StatName, CSV_CATEGORY_INDEX(NavTasksDelays), WaitTime, ECsvCustomStatOp::Set);
#endif // CSV_PROFILER_STATS

			ResetTileWaitTime(NavDataIndex);
		}
	}
#endif // !UE_BUILD_SHIPPING
}

void FNavRegenTimeSliceManager::SetMinTimeSliceDuration(double NewMinTimeSliceDuration)
{
	MinTimeSliceDuration = NewMinTimeSliceDuration;

	UE_LOGF(LogNavigationDataBuild, Verbose, "Navigation System: MinTimeSliceDuration = %f", MinTimeSliceDuration);
}

void FNavRegenTimeSliceManager::SetMaxTimeSliceDuration(double NewMaxTimeSliceDuration)
{
	MaxTimeSliceDuration = NewMaxTimeSliceDuration;

	UE_LOGF(LogNavigationDataBuild, Verbose, "Navigation System: MaxTimeSliceDuration = %f", MaxTimeSliceDuration);
}

void FNavRegenTimeSliceManager::SetMaxDesiredTileRegenDuration(float NewMaxDesiredTileRegenDuration)
{
	MaxDesiredTileRegenDuration = NewMaxDesiredTileRegenDuration;

	UE_LOGF(LogNavigationDataBuild, Verbose, "Navigation System: MaxDesiredTileRegenDuration = %f", MaxDesiredTileRegenDuration);
}

#if !UE_BUILD_SHIPPING
void FNavRegenTimeSliceManager::LogTileStatistics(const TArray<TObjectPtr<ANavigationData>>& NavDataSet) const
{
	UE_SUPPRESS(LogNavigationHistory, Log,
	{
		// Log median tile processing time every 60 frames.
		const bool bLog = GFrameCounter % 60 == 0;
		const double HistoryDuration = FPlatformTime::Seconds() - TileHistoryStartTime;
		for (int32 NavDataIndex = 0; bLog && NavDataIndex < NavDataSet.Num(); ++NavDataIndex)
		{
			if (TileHistoryData.IsValidIndex(NavDataIndex))
			{
				TArray<FTileHistoryData> HistoryData = TileHistoryData[NavDataIndex];
				if (HistoryData.Num() > 0)
				{
					const int32 MedianIndex = HistoryData.Num()/2;
					const int32 HighIndex = int(HistoryData.Num()*0.9);

					HistoryData.Sort([](const FTileHistoryData& A, const FTileHistoryData& B){ return A.TileRegenTime < B.TileRegenTime; });
					const double MedianRegenTimeMs = HistoryData[MedianIndex].TileRegenTime * 1000.f;
					const double HighRegenTimeMs = HistoryData[HighIndex].TileRegenTime * 1000.f;
					const int64 MedianRegenFrames = HistoryData[MedianIndex].EndRegenFrame - HistoryData[MedianIndex].StartRegenFrame;

					HistoryData.Sort([](const FTileHistoryData& A, const FTileHistoryData& B){ return A.TileWaitTime < B.TileWaitTime; });
					const double MedianWaitTimeMs = HistoryData[MedianIndex].TileWaitTime * 1000.f;
					const double HighWaitTimeMs = HistoryData[HighIndex].TileWaitTime * 1000.f;
					
					UE_LOGF(LogNavigationHistory, Log, "%-35ls Median tile stats: regen time: %2.2f ms, regen frames %lld, wait time: %4.f ms (high regen time: %2.2f ms, high wait time: %4.f ms) regen count: %i, regen/s: %0.2f",
						*GetNameSafe(NavDataSet[NavDataIndex]), MedianRegenTimeMs, MedianRegenFrames, MedianWaitTimeMs, HighRegenTimeMs, HighWaitTimeMs,
						HistoryData.Num(), HistoryData.Num()/HistoryDuration);
				}
			}
		}
	});
}
#endif // !UE_BUILD_SHIPPING

//----------------------------------------------------------------------//
// UNavigationSystemV1
//----------------------------------------------------------------------//
bool UNavigationSystemV1::bNavigationAutoUpdateEnabled = true;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
FNavigationSystemExec UNavigationSystemV1::ExecHandler;
#endif // !UE_BUILD_SHIPPING

/** called after navigation influencing event takes place*/
UNavigationSystemV1::FOnNavigationDirty UNavigationSystemV1::NavigationDirtyEvent;

bool UNavigationSystemV1::bUpdateNavOctreeOnComponentChange = true;
bool UNavigationSystemV1::bStaticRuntimeNavigation = false;
bool UNavigationSystemV1::bIsPIEActive = false;
//----------------------------------------------------------------------//
// life cycle stuff
//----------------------------------------------------------------------//

UNavigationSystemV1::UNavigationSystemV1(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bTickWhilePaused(false)
	, bDirtyNavDataPackageOnNavigationGenerationFinished(false)
	, bWholeWorldNavigable(false)
	, bSkipAgentHeightCheckWhenPickingNavData(false)
	, DirtyAreaWarningSizeThreshold(-1.0f)
	, GatheringNavModifiersWarningLimitTime(-1.0f)
	, BuildBounds(EForceInit::ForceInit)
	, OperationMode(FNavigationSystemRunMode::InvalidMode)
	, bAbortAsyncQueriesRequested(false)
	, NavBuildingLockFlags(0)
	, InitialNavBuildingLockFlags(0)
	, bInitialSetupHasBeenPerformed(false)
	, bInitialLevelsAdded(false)
	, bWorldInitDone(false)
	, bCleanUpDone(false)
	, CurrentlyDrawnNavDataIndex(0)
{
#if WITH_EDITOR
	NavUpdateLockFlags = 0;
#endif
	struct FDelegatesInitializer
	{
		FDelegatesInitializer()
		{
			UNavigationSystemBase::GetSupportsDynamicChangesDelegate().BindStatic(&UNavigationSystemV1::SupportsDynamicChanges);
			UNavigationSystemBase::GetAddNavigationElementDelegate().BindStatic(&UNavigationSystemV1::AddNavigationElement);
			UNavigationSystemBase::GetRemoveNavigationElementDelegate().BindStatic(&UNavigationSystemV1::RemoveNavigationElement);
			UNavigationSystemBase::GetUpdateNavigationElementDelegate().BindStatic(&UNavigationSystemV1::OnNavigationElementUpdated);

			UNavigationSystemBase::GetUpdateNavigationElementBoundsDelegate().BindLambda(
				[](UWorld* World, FNavigationElementHandle Handle, const FBox& NewBounds, const TConstArrayView<FBox> DirtyAreas)
				{
					if (UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World))
					{
						NavSys->UpdateNavOctreeElementBounds(Handle, NewBounds, DirtyAreas);
					}
				});

			UNavigationSystemBase::RegisterNavRelevantObjectDelegate().BindLambda([](UObject& Object) { UNavigationSystemV1::OnNavRelevantObjectRegistered(Object); });
			UNavigationSystemBase::UpdateNavRelevantObjectDelegate().BindStatic(&UNavigationSystemV1::UpdateNavRelevantObjectInNavOctree);
			UNavigationSystemBase::UnregisterNavRelevantObjectDelegate().BindLambda([](UObject& Object) { UNavigationSystemV1::OnNavRelevantObjectUnregistered(Object); });
			UNavigationSystemBase::OnObjectBoundsChangedDelegate().BindLambda([](UObject& Object, const FBox& NewBounds, const TConstArrayView<FBox> DirtyAreas)
				{
					if (UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(Object.GetWorld()))
					{
						NavSys->UpdateNavOctreeElementBounds(FNavigationElementHandle(&Object), NewBounds, DirtyAreas);
					}
				});

			UNavigationSystemBase::UpdateActorDataDelegate().BindStatic(&UNavigationSystemV1::UpdateActorInNavOctree);
			UNavigationSystemBase::UpdateComponentDataDelegate().BindStatic(&UNavigationSystemV1::UpdateComponentInNavOctree);
			UNavigationSystemBase::UpdateComponentDataAfterMoveDelegate().BindLambda([](USceneComponent& Comp) { UNavigationSystemV1::UpdateNavOctreeAfterMove(&Comp); });
			UNavigationSystemBase::OnActorBoundsChangedDelegate().BindLambda([](AActor& Actor) { UNavigationSystemV1::UpdateNavOctreeBounds(&Actor); });
			UNavigationSystemBase::OnPostEditActorMoveDelegate().BindLambda([](AActor& Actor) {
				// update actor and all its components in navigation system after finishing move
				// USceneComponent::UpdateNavigationData works only in game world
				UNavigationSystemV1::UpdateNavOctreeBounds(&Actor);

				TArray<AActor*> ParentedActors;
				Actor.GetAttachedActors(ParentedActors);
				for (int32 Idx = 0; Idx < ParentedActors.Num(); Idx++)
				{
					UNavigationSystemV1::UpdateNavOctreeBounds(ParentedActors[Idx]);
				}

				// We need to check this actor has registered all their components post spawn / load
				// before attempting to update the components in the nav octree.
				// Without this check we were getting an issue with UNavRelevantComponent::GetNavigationParent().
				if (Actor.HasActorRegisteredAllComponents())
				{
					// not doing manual update of all attached actors since UpdateActorAndComponentsInNavOctree should take care of it
					UNavigationSystemV1::UpdateActorAndComponentsInNavOctree(Actor);
				}
			});
			UNavigationSystemBase::OnComponentTransformChangedDelegate().BindLambda([](USceneComponent& Comp)
			{
				if (ShouldUpdateNavOctreeOnComponentChange())
				{
					UpdateNavOctreeAfterMove(&Comp);
				}
			});
			UNavigationSystemBase::OnActorRegisteredDelegate().BindLambda([](AActor& Actor) { UNavigationSystemV1::OnActorRegistered(&Actor); });
			UNavigationSystemBase::OnActorUnregisteredDelegate().BindLambda([](AActor& Actor) { UNavigationSystemV1::OnActorUnregistered(&Actor); });
			UNavigationSystemBase::OnComponentRegisteredDelegate().BindLambda([](UActorComponent& Comp) { UNavigationSystemV1::OnComponentRegistered(&Comp); });
			UNavigationSystemBase::OnComponentUnregisteredDelegate().BindLambda([](UActorComponent& Comp) { UNavigationSystemV1::OnComponentUnregistered(&Comp); });
			UNavigationSystemBase::RegisterComponentDelegate().BindLambda([](UActorComponent& Comp) { UNavigationSystemV1::RegisterComponent(&Comp); });
			UNavigationSystemBase::UnregisterComponentDelegate().BindLambda([](UActorComponent& Comp) { UNavigationSystemV1::UnregisterComponent(&Comp); });
			UNavigationSystemBase::RemoveActorDataDelegate().BindLambda([](AActor& Actor) { UNavigationSystemV1::ClearNavOctreeAll(&Actor); });
			UNavigationSystemBase::HasComponentDataDelegate().BindLambda([](UActorComponent& Comp)
			{
				UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(Comp.GetWorld());
				const FNavigationElementHandle Element(&Comp);
				return (NavSys && (NavSys->GetNavOctreeIdForElement(Element) || NavSys->HasPendingUpdateForElement(Element)));
			});
			UNavigationSystemBase::GetDefaultSupportedAgentDelegate().BindStatic(&UNavigationSystemV1::GetDefaultSupportedAgent);
			UNavigationSystemBase::GetBiggestSupportedAgentDelegate().BindStatic(&UNavigationSystemV1::GetBiggestSupportedAgent);
			UNavigationSystemBase::UpdateActorAndComponentDataDelegate().BindStatic(&UNavigationSystemV1::UpdateActorAndComponentsInNavOctree);
			UNavigationSystemBase::OnComponentBoundsChangedDelegate().BindLambda([](UActorComponent& Comp, const FBox& NewBounds, const FBox& DirtyArea)
			{
				UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(Comp.GetWorld());
				if (NavSys)
				{
					NavSys->UpdateNavOctreeElementBounds(FNavigationElementHandle(&Comp), NewBounds, {DirtyArea});
				}
			});
			//UNavigationSystemBase::GetNavDataForPropsDelegate();
			UNavigationSystemBase::GetNavDataForActorDelegate().BindStatic(&UNavigationSystemV1::GetNavDataForActor);

#if WITH_RECAST
			UNavigationSystemBase::GetDefaultNavDataClassDelegate().BindLambda([]() { return ARecastNavMesh::StaticClass(); });
#endif // WITH_RECAST
			UNavigationSystemBase::VerifyNavigationRenderingComponentsDelegate().BindLambda([](UWorld& World, const bool bShow) {
				UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(&World);
				if (NavSys)
				{
					NavSys->VerifyNavigationRenderingComponents(bShow);
				}
			});
			UNavigationSystemBase::BuildDelegate().BindLambda([](UWorld& World) {
				UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(&World);
				if (NavSys)
				{
					NavSys->Build();
				}
			});
#if WITH_EDITOR
			UNavigationSystemBase::OnPIEStartDelegate().BindLambda([](UWorld& World) {
				UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(&World);
				if (NavSys)
				{
					NavSys->OnPIEStart();
				}
			});
			UNavigationSystemBase::OnPIEEndDelegate().BindLambda([](UWorld& World) {
				UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(&World);
				if (NavSys)
				{
					NavSys->OnPIEEnd();
				}
			});
			UNavigationSystemBase::UpdateLevelCollisionDelegate().BindLambda([](ULevel& Level) {
				UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(&Level);
				if (NavSys)
				{
					NavSys->UpdateLevelCollision(&Level);
				}
			});
			UNavigationSystemBase::SetNavigationAutoUpdateEnableDelegate().BindStatic(&UNavigationSystemV1::SetNavigationAutoUpdateEnabled);
				/*.BindLambda([](const bool bNewEnable, UNavigationSystemBase* InNavigationSystem) {

			})*/
			UNavigationSystemBase::AddNavigationUpdateLockDelegate().BindLambda([](UWorld& World, uint8 Flags) {
				UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(&World);
				if (NavSys)
				{
					NavSys->AddNavigationUpdateLock(Flags);
				}
			});
			UNavigationSystemBase::RemoveNavigationUpdateLockDelegate().BindLambda([](UWorld& World, uint8 Flags) {
				UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(&World);
				if (NavSys)
				{
					NavSys->RemoveNavigationUpdateLock(Flags);
				}
			});
			UNavigationSystemBase::GetWorldPartitionNavigationDataBuilderOverlapDelegate().BindStatic(&UNavigationSystemV1::GetWorldPartitionNavigationDataBuilderOverlap);
#endif // WITH_EDITOR

#if ENABLE_VISUAL_LOG
			FVisualLogger::NavigationDataDumpDelegate.AddStatic(&NavigationDataDump);
#endif // ENABLE_VISUAL_LOG
		}
	};
	static FDelegatesInitializer DelegatesInitializer;

	// NOP line to silence code analysis warning: "Local variable 'DelegatesInitializer' is never used"
	(void)DelegatesInitializer;
	
	// Set to the ai module's crowd manager, this module may not exist at spawn time but then it will just fail to load
	CrowdManagerClass = FSoftObjectPath(TEXT("/Script/AIModule.CrowdManager"));

	// active tiles
	NextInvokersUpdateTime = 0.;
	ActiveTilesUpdateInterval = 1.f;
	bGenerateNavigationOnlyAroundNavigationInvokers = false;
	DataGatheringMode = ENavDataGatheringModeConfig::Instant;
	bShouldDiscardSubLevelNavData = true;

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		// reserve some arbitrary size
		AsyncPathFindingQueries.Reserve( INITIAL_ASYNC_QUERIES_SIZE );
		NavDataRegistrationQueue.Reserve( REGISTRATION_QUEUE_SIZE );
	
		FWorldDelegates::OnWorldPostActorTick.AddUObject(this, &UNavigationSystemV1::OnWorldPostActorTick);
		FWorldDelegates::LevelAddedToWorld.AddUObject(this, &UNavigationSystemV1::OnLevelAddedToWorld);
		FWorldDelegates::LevelRemovedFromWorld.AddUObject(this, &UNavigationSystemV1::OnLevelRemovedFromWorld);
		FWorldDelegates::OnWorldBeginTearDown.AddUObject(this, &UNavigationSystemV1::OnBeginTearingDown);
#if !UE_BUILD_SHIPPING
		FCoreDelegates::OnGetOnScreenMessages.AddUObject(this, &UNavigationSystemV1::GetOnScreenMessages);
#endif // !UE_BUILD_SHIPPING

		if (const UWorld* World = UNavigationSystemV1::GetWorld())
		{
			Repository = World->GetSubsystem<UNavigationObjectRepository>();
		}

		if (Repository == nullptr)
		{
			UE_LOGF(LogNavigation, Warning, "UNavigationObjectRepository is required for navigation system operations.");
		}
	}
	else if (GetClass() == UNavigationSystemV1::StaticClass())
	{
		SetDefaultWalkableArea(UNavArea_Default::StaticClass());
		SetDefaultObstacleArea(UNavArea_Obstacle::StaticClass());
		
#if WITH_RECAST
		const FTransform RecastToUnrealTransform(Recast2UnrealMatrix());
		SetCoordTransform(ENavigationCoordSystem::Navigation, ENavigationCoordSystem::Unreal, RecastToUnrealTransform);
#endif // WITH_RECAST
	}
}

void UNavigationSystemV1::FinishDestroy()
{
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		CleanUp(FNavigationSystem::ECleanupMode::CleanupUnsafe);
	}
	Super::FinishDestroy();
}

void UNavigationSystemV1::GatherDebugLabels(TArray<FString>& InOutDebugLabels) const
{
	if (Repository)
	{
		InOutDebugLabels.Add(FString::Printf(TEXT("Repository Navigation Elements count: %i"), Repository->GetNumRegisteredElements()));
		InOutDebugLabels.Add(FString::Printf(TEXT("Repository UObjects count: %i"), Repository->GetNumRegisteredUObjects()));
		InOutDebugLabels.Add(FString::Printf(TEXT("Repository Custom NavLinks count: %i"), Repository->GetNumRegisteredCustomLinks()));
	}

	InOutDebugLabels.Add(FString::Printf(TEXT("NavData count: %i"), NavDataSet.Num()));
	InOutDebugLabels.Add(FString::Printf(TEXT("MainNavData: %s"), MainNavData ? *MainNavData->GetName() : TEXT("none")));
	InOutDebugLabels.Add(FString::Printf(TEXT("Custom NavLinks count: %i"), GetNumCustomLinks()));

	if (GetNavOctree())
	{
		int32 NumNodes = 0;
		int32 NumElements = 0;

		GetNavOctree()->FindNodesWithPredicate(
			[](FNavigationOctree::FNodeIndex /*ParentNodeIndex*/, FNavigationOctree::FNodeIndex /*NodeIndex*/, const FBoxCenterAndExtent&) { return true; },
			[&, Octree = GetNavOctree()](FNavigationOctree::FNodeIndex /*ParentNodeIndex*/, const FNavigationOctree::FNodeIndex NodeIndex, const FBoxCenterAndExtent&)
			{
				NumNodes++;
				NumElements += Octree->GetElementsForNode(NodeIndex).Num();
			});

		InOutDebugLabels.Add(FString::Printf(TEXT("Octree node count: %i"), NumNodes));
		InOutDebugLabels.Add(FString::Printf(TEXT("Octree element count: %i"), NumElements));
	}

#if WITH_NAVMESH_CLUSTER_LINKS
	InOutDebugLabels.Add(FString::Printf(TEXT("Using cluster links")));
#endif // WITH_NAVMESH_CLUSTER_LINKS

	if (IsActiveTilesGenerationEnabled()) // Checks bGenerateNavigationOnlyAroundNavigationInvokers
	{
		InOutDebugLabels.Add(FString::Printf(TEXT("Invoker Locations: %i"), GetInvokerLocations().Num()));
	}

	const int32 Running = GetNumRunningBuildTasks();
	const int32 Remaining = GetNumRemainingBuildTasks();
	if (Running || Remaining)
	{
		InOutDebugLabels.Add(FString::Printf(TEXT("Tile jobs running/remaining: %6d / %6d"), Running, Remaining));
	}

	InOutDebugLabels.Add(TEXT("")); // empty line
}

void UNavigationSystemV1::ConfigureAsStatic(bool bEnableStatic)
{
	bStaticRuntimeNavigation = bEnableStatic;
	SetWantsComponentChangeNotifies(!bEnableStatic);
}

void UNavigationSystemV1::SetUpdateNavOctreeOnComponentChange(bool bNewUpdateOnComponentChange)
{
	bUpdateNavOctreeOnComponentChange = bNewUpdateOnComponentChange;
}

void UNavigationSystemV1::DoInitialSetup()
{
	if (bInitialSetupHasBeenPerformed)
	{
		return;
	}

	UE_LOGF(LogNavigation, Verbose, "%s", __FUNCTION__);
	
	UpdateAbstractNavData();
	CreateCrowdManager();

	RegisterToRepositoryDelegates();

#if WITH_EDITOR		
	if (bWaitForAsyncLoadingBeforeBuildingNavigationAutomatically && GetIsAutoUpdateEnabled() && OperationMode == FNavigationSystemRunMode::EditorMode)
	{
		UE_LOGF(LogNavigation, Verbose, "%s    Adding ENavigationBuildLock::AsyncLoadLock.", __FUNCTION__);
		AddNavigationBuildLock(ENavigationBuildLock::AsyncLoadLock);

		OnAssetPostCompileEventHandle = FAssetCompilingManager::Get().OnAssetPostCompileEvent().AddUObject(this, &UNavigationSystemV1::UpdateDelayedUnlockeRegistration);

		// Run it at least once to make sure we don't stay locked.
		UpdateDelayedUnlockeRegistration({});
	}
#endif // WITH_EDITOR	

	bInitialSetupHasBeenPerformed = true;
}

void UNavigationSystemV1::UpdateAbstractNavData()
{
	if (IsValid(AbstractNavData))
	{
		return;
	}

	// spawn abstract nav data separately
	// it's responsible for direct paths and shouldn't be picked for any agent type as default one
	UWorld* NavWorld = GetWorld();
	for (TActorIterator<AAbstractNavData> It(NavWorld); It; ++It)
	{
		AAbstractNavData* Nav = (*It);
		if (IsValid(Nav))
		{
			AbstractNavData = Nav;
			break;
		}
	}

	if (AbstractNavData == NULL)
	{
		FNavDataConfig DummyConfig;
		DummyConfig.SetNavDataClass(AAbstractNavData::StaticClass());
		AbstractNavData = CreateNavigationDataInstanceInLevel(DummyConfig, nullptr);
		if (AbstractNavData)
		{
			AbstractNavData->SetFlags(RF_Transient);
		}
	}
}

void UNavigationSystemV1::SetSupportedAgentsNavigationClass(int32 AgentIndex, TSubclassOf<ANavigationData> NavigationDataClass)
{
	const bool bCDOInEditor =
#if WITH_EDITOR
		// the CDO will have 0 supported agents if none are defined which is fine in the editor
		(GIsEditor && HasAnyFlags(RF_ClassDefaultObject))
#else
		false
#endif // WITH_EDITOR
		;

	check(SupportedAgents.IsValidIndex(AgentIndex) || bCDOInEditor);

	if (SupportedAgents.IsValidIndex(AgentIndex))
	{
	SupportedAgents[AgentIndex].SetNavDataClass(NavigationDataClass);

	// keep preferred navigation data class in sync with actual class
	// this will be passed to navigation data actor and will be required
	// for comparisons done in DoesSupportAgent calls
	//
	// "Any" navigation data preference is valid only for instanced agents
	SupportedAgents[AgentIndex].SetPreferredNavData(NavigationDataClass);
	}

#if WITH_EDITOR
	if (GIsEditor)
	{
		if (HasAnyFlags(RF_ClassDefaultObject) == false)
		{
			// set it at CDO to properly show up in project settings
			// @hack the reason for doing it this way is that engine doesn't handle default TSubclassOf properties
			//	set to game-specific classes;
			UNavigationSystemV1* NavigationSystemCDO = GetMutableDefault<UNavigationSystemV1>(GetClass());
			NavigationSystemCDO->SetSupportedAgentsNavigationClass(AgentIndex, NavigationDataClass);
		}
	}
#endif // WITH_EDITOR
}

void UNavigationSystemV1::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		// Populate our NavAreaClasses list with all known nav area classes.
		// If more are loaded after this they will be registered as they come
		TArray<UClass*> CurrentNavAreaClasses;
		GetDerivedClasses(UNavArea::StaticClass(), CurrentNavAreaClasses);
		for (UClass* NavAreaClass : CurrentNavAreaClasses)
		{
			RegisterNavAreaClass(NavAreaClass);
		}

		ApplySupportedAgentsFilter();
		for (int32 AgentIndex = 0; AgentIndex < SupportedAgents.Num(); ++AgentIndex)
		{
			FNavDataConfig& SupportedAgentConfig = SupportedAgents[AgentIndex];
			SetSupportedAgentsNavigationClass(AgentIndex, SupportedAgentConfig.GetNavDataClass<ANavigationData>());
		}

		DefaultDirtyAreasController.SetDirtyAreaWarningSizeThreshold(DirtyAreaWarningSizeThreshold);
	
		if (bInitialBuildingLocked)
		{
			InitialNavBuildingLockFlags |= ENavigationBuildLock::InitialLock;
		}

		uint8 UseLockFlags = InitialNavBuildingLockFlags;

		AddNavigationBuildLock(UseLockFlags);
		
#if WITH_EDITOR
		if (GIsEditor && GEngine)
		{
			// register for any actor move change
			GEngine->OnActorMoved().AddUObject(this, &UNavigationSystemV1::OnActorMoved);

			FEditorDelegates::OnMapOpened.AddUObject(this, &UNavigationSystemV1::OnMapOpened);
		}
#endif // WITH_EDITOR

		FCoreUObjectDelegates::PostLoadMapWithWorld.AddUObject(this, &UNavigationSystemV1::OnPostLoadMap);
		UNavigationSystemV1::NavigationDirtyEvent.AddUObject(this, &UNavigationSystemV1::OnNavigationDirtied);

		ReloadCompleteDelegateHandle = FCoreUObjectDelegates::ReloadCompleteDelegate.AddUObject(this, &UNavigationSystemV1::OnReloadComplete);
	}
}

void UNavigationSystemV1::ConstructNavOctree()
{
	// Default values to keep previous behavior.
	FVector NavOctreeCenter = FVector::ZeroVector;
	double NavOctreeRadius = 64000;

	const FBox Bounds = GetNavigableWorldBounds();
	if(Bounds.IsValid)
	{
		NavOctreeCenter = Bounds.GetCenter();
		NavOctreeRadius = Bounds.GetExtent().GetAbsMax();
	}

	FNavigationDataHandler NavHandler(DefaultOctreeController, DefaultDirtyAreasController);
	NavHandler.ConstructNavOctree(NavOctreeCenter, NavOctreeRadius, DataGatheringMode, GatheringNavModifiersWarningLimitTime);
}

bool UNavigationSystemV1::ConditionalPopulateNavOctree()
{
	// Discard all navigation updates caused by octree construction
	UE_LOGF(LogNavigationDirtyArea, VeryVerbose, "%s: Reseting Dirty Areas added during octree construction. DirtyAreas.Num = [%d].", __FUNCTION__, DefaultDirtyAreasController.DirtyAreas.Num());
	TGuardValue<TArray<FNavigationDirtyArea>> DirtyGuard(DefaultDirtyAreasController.DirtyAreas, TArray<FNavigationDirtyArea>());

	// See if any of registered navigation data need navoctree
	bSupportRebuilding = RequiresNavOctree();

	if (bSupportRebuilding)
	{
		ConstructNavOctree();
		if (DefaultOctreeController.IsValid())
		{
			const bool bStoreNavGeometry = DoesAnyNavDataRequireGeometryStorage();
			DefaultOctreeController.SetNavigableGeometryStoringMode(bStoreNavGeometry ? FNavigationOctree::StoreNavGeometry : FNavigationOctree::SkipNavGeometry);
			if (bStoreNavGeometry)
			{
#if WITH_RECAST
				DefaultOctreeController.NavOctree->GeometryExportDelegate =
					FNavigationOctree::FGeometryExportDelegate::CreateStatic(&FRecastGeometryExport::ExportElementGeometry);
#endif // WITH_RECAST
			}

			if (!DefaultOctreeController.IsNavigationOctreeLocked())
			{
				UWorld* World = GetWorld();
				check(World);

				// Register level collisions
				for (int32 LevelIndex = 0; LevelIndex < World->GetNumLevels(); ++LevelIndex)
				{
					ULevel* Level = World->GetLevel(LevelIndex);
					if (ensure(Level) && Level->bIsVisible)
					{
						AddLevelToOctree(*Level);
					}
				}

				if (Repository != nullptr)
				{
					// Register all elements registered in the repository world subsystem.
					Repository->ForEachNavigationElement([this](const TSharedRef<const FNavigationElement>& Element)
						{
							RegisterNavigationElementWithNavOctree(Element, FNavigationOctreeController::OctreeUpdate_Default);
						});
				}
			}
		}
	}
	else
	{
		// Discard current octree along with pending updates
		DestroyNavOctree();
	}

	// Add all found elements to octree, this will not add new dirty areas to navigation
	ProcessPendingOctreeUpdates();

	return bSupportRebuilding;
}

#if WITH_EDITOR
void UNavigationSystemV1::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	static const FName NAME_NavDataClass = FNavDataConfig::GetNavigationDataClassPropertyName();
	static const FName NAME_SupportedAgents = GET_MEMBER_NAME_CHECKED(UNavigationSystemV1, SupportedAgents);
	static const FName NAME_AllowClientSideNavigation = GET_MEMBER_NAME_CHECKED(UNavigationSystemV1, bAllowClientSideNavigation);

	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		FName PropName = PropertyChangedEvent.Property->GetFName();
		if (PropName == NAME_NavDataClass)
		{
			int32 SupportedAgentIndex = PropertyChangedEvent.GetArrayIndex(NAME_SupportedAgents.ToString());
			if (SupportedAgents.IsValidIndex(SupportedAgentIndex))
			{
				// reflect the change to SupportedAgent's 
				TSubclassOf<ANavigationData> NavClass = SupportedAgents[SupportedAgentIndex].GetNavDataClass<ANavigationData>();
				SetSupportedAgentsNavigationClass(SupportedAgentIndex, NavClass);
				SaveConfig();
			}
		}
		else if (PropName == NAME_AllowClientSideNavigation && HasAnyFlags(RF_ClassDefaultObject))
		{
			for (FThreadSafeObjectIterator It(UNavigationSystemModuleConfig::StaticClass()); It; ++It)
			{
				((UNavigationSystemModuleConfig*)*It)->UpdateWithNavSysCDO(*this);
			}
		}
	}
}

void UNavigationSystemV1::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName NAME_GenerateNavigationOnlyAroundNavigationInvokers = GET_MEMBER_NAME_CHECKED(UNavigationSystemV1, bGenerateNavigationOnlyAroundNavigationInvokers);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		FName PropName = PropertyChangedEvent.Property->GetFName();
		if (PropName == NAME_GenerateNavigationOnlyAroundNavigationInvokers)
		{
			OnGenerateNavigationOnlyAroundNavigationInvokersChanged();
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(FNavDataConfig, AgentRadius))
		{
			const bool bIsCDO = HasAnyFlags(RF_ClassDefaultObject);
			if (!bIsCDO)
			{
				const UWorld* World = GetWorld();
				if (World && World->IsPartitionedWorld())
				{
					FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("NeedToRunPartitionResaveActorsBuilder",
						"In a world partitioned map, changing this property changes the partitioning of actors.\n"
						"For the change to take effect on partitioning, actors needs to be resaved.\n"
						"Run the WorldPartitionResaveActorsBuilder to update the whole map."));	
				}
			}
		}
	}
}
#endif // WITH_EDITOR

void UNavigationSystemV1::OnInitializeActors()
{
	
}

void UNavigationSystemV1::OnBeginTearingDown(UWorld* World)
{
	// If the world being torn down is my world context
	if (World == GetWorld())
	{
		CleanUp(FNavigationSystem::ECleanupMode::CleanupWithWorld);
	}
}

void UNavigationSystemV1::OnWorldInitDone(FNavigationSystemRunMode Mode)
{
	UE_LOGF(LogNavigation, Verbose, "%s, Mode: %ls", __FUNCTION__, *UEnum::GetValueAsString(Mode));
	
	UNavigationSystemBase::OnNavigationInitStartStaticDelegate().Broadcast(*this);

	OperationMode = Mode;
	DoInitialSetup();
	
	UWorld* World = GetWorld();
	check(World);

	// process all registered link from the repository subsystem
	// (since it's possible navigation system was not ready by the time
	// those links were serialized-in or spawned)
	if (!bWorldInitDone)
	{
		ProcessCustomLinkPendingRegistration();
	}

	if (IsThereAnywhereToBuildNavigation() == false)
	{
		// remove all navigation data instances
		for (TActorIterator<ANavigationData> It(World); It; ++It)
		{
			ANavigationData* Nav = (*It);
			if (IsValid(Nav) && Nav != GetAbstractNavData())
			{
				UnregisterNavData(Nav);
				Nav->CleanUpAndMarkPendingKill();
				bNavDataRemovedDueToMissingNavBounds = true;
			}
		}

		if (FNavigationSystem::IsEditorRunMode(OperationMode))
		{
			RemoveNavigationBuildLock(InitialNavBuildingLockFlags, ELockRemovalRebuildAction::NoRebuild);
		}
	}
	else
	{
		// Discard all bounds updates that was submitted during world initialization, 
		// to avoid navigation rebuild right after map is loaded
		PendingNavBoundsUpdates.Empty();
		
		// gather navigable bounds
		GatherNavigationBounds();

		// gather all navigation data instances and register all not-yet-registered
		// (since it's quite possible navigation system was not ready by the time
		// those instances were serialized-in or spawned)
		RegisterNavigationDataInstances();

		if (bAutoCreateNavigationData == true)
		{
			SpawnMissingNavigationData();
			// in case anything spawned has registered
			ProcessRegistrationCandidates();
		}
		else
		{
			const bool bIsBuildLocked = IsNavigationBuildingLocked();
			const bool bCanRebuild = !bIsBuildLocked && GetIsAutoUpdateEnabled();

			if (GetDefaultNavDataInstance(FNavigationSystem::DontCreate) != nullptr)
			{
				// trigger navmesh update
				for (TActorIterator<ANavigationData> It(World); It; ++It)
				{
					ANavigationData* NavData = (*It);
					if (NavData != nullptr)
					{
						const ERegistrationResult Result = RegisterNavData(NavData);
						LogNavDataRegistrationResult(Result, NavData);

						if (Result == RegistrationSuccessful)
						{
							// allowing full rebuild of the entire navmesh only for the fully dynamic generation modes
							// other modes partly rely on the serialized data and full rebuild would wipe it out
							if (bCanRebuild && IsAllowedToRebuild())
							{
								NavData->RebuildAll();
							}
						}
						else if (Result != RegistrationFailed_DataPendingKill
							&& Result != RegistrationFailed_AgentNotValid
							)
						{
							NavData->CleanUpAndMarkPendingKill();
						}
					}
				}
			}
		}

		if (FNavigationSystem::IsEditorRunMode(OperationMode))
		{
			// don't lock navigation building in editor
			RemoveNavigationBuildLock(InitialNavBuildingLockFlags, ELockRemovalRebuildAction::NoRebuild);
		}

		// See if any of registered navigation data needs NavOctree
		ConditionalPopulateNavOctree();

		// All navigation actors are registered
		// Add NavMesh parts from all sub-levels that were streamed in prior NavMesh registration
		const auto& Levels = World->GetLevels();
		for (ULevel* Level : Levels)
		{
			if (!Level->IsPersistentLevel() && Level->bIsVisible)
			{
				for (ANavigationData* NavData : NavDataSet)
				{
					if (NavData)
					{
						NavData->OnStreamingLevelAdded(Level, World);
					}
				}
			}
		}
	}

	// If any nav data was spawned at runtime (not loaded from disk), trigger a rebuild.
	// Must run after ConditionalPopulateNavOctree so the geometry is available.
	if (GetWorld()->IsGameWorld())
	{
		const bool bRequiresInitialRebuild = NavDataSet.ContainsByPredicate(
			[](const ANavigationData* NavData) { return NavData && NavData->RequiresInitialRebuild(); });
		if (bRequiresInitialRebuild)
		{
			const bool bIsLoadTime = true;
			RebuildAll(bIsLoadTime);
		}
	}


#if	WITH_EDITOR
	if (FNavigationSystem::IsEditorRunMode(Mode))
	{
		// make sure this static get applied to this instance
		bNavigationAutoUpdateEnabled = !bNavigationAutoUpdateEnabled; 
		SetNavigationAutoUpdateEnabled(!bNavigationAutoUpdateEnabled, this);
		
		// update navigation invokers
		if (bGenerateNavigationOnlyAroundNavigationInvokers)
		{
			for (TObjectIterator<UNavigationInvokerComponent> It; It; ++It)
			{
				if (World == It->GetWorld())
				{
					It->RegisterWithNavigationSystem(*this);
				}
			}
		}

		// update navdata after loading world
		if (GetIsAutoUpdateEnabled())
		{
			const bool bIsLoadTime = true;
			RebuildAll(bIsLoadTime);
		}
	}
#endif

	if (!DefaultDirtyAreasController.bCanAccumulateDirtyAreas)
	{
		DefaultDirtyAreasController.DirtyAreas.Empty();
	}

	// Dirty area controller reports oversized dirty areas only in game mode and if we are not using active tile generation.
	// When using active tile generation, this is reported only if tiles are actually marked dirty (ex: see MarkDirtyTiles).
	DefaultDirtyAreasController.SetCanReportOversizedDirtyArea(Mode == FNavigationSystemRunMode::GameMode && !IsActiveTilesGenerationEnabled());

	for (const ANavigationData* NavData : NavDataSet)
	{
		if (NavData)
		{
#if WITH_RECAST
			const ARecastNavMesh* RecastNavMesh = Cast<ARecastNavMesh>(NavData);
			if (RecastNavMesh && RecastNavMesh->bIsWorldPartitioned && NavData->GetRuntimeGenerationMode() > ERuntimeGenerationType::Static)
			{
				DefaultDirtyAreasController.SetUseWorldPartitionedDynamicMode(true);
				break;
			}
#endif // WITH_RECAST
		}
	}

	bWorldInitDone = true;
	OnNavigationInitDone.Broadcast();
	UNavigationSystemBase::OnNavigationInitDoneStaticDelegate().Broadcast(*this);
}

void UNavigationSystemV1::RegisterNavigationDataInstances()
{
	UWorld* World = GetWorld();
	
	UE_LOGF(LogNavigation, Verbose, "%s World: %ls", __FUNCTION__, *GetNameSafe(World));

	bool bProcessRegistration = false;
	for (TActorIterator<ANavigationData> It(World); It; ++It)
	{
		ANavigationData* Nav = (*It);
		if (IsValid(Nav) && Nav->IsRegistered() == false)
		{
			RequestRegistrationDeferred(*Nav);
			bProcessRegistration = true;
		}
	}
	if (bProcessRegistration == true)
	{
		ProcessRegistrationCandidates();
	}
}

void UNavigationSystemV1::CreateCrowdManager()
{
	UClass* CrowdManagerClassInstance = CrowdManagerClass.Get();
	if (CrowdManagerClassInstance)
	{
		UCrowdManagerBase* ManagerInstance = NewObject<UCrowdManagerBase>(this, CrowdManagerClassInstance);
		// creating an instance when we have a valid class should never fail
		check(ManagerInstance);
		SetCrowdManager(ManagerInstance);
	}
}

void UNavigationSystemV1::SetCrowdManager(UCrowdManagerBase* NewCrowdManager)
{
	if (NewCrowdManager == CrowdManager.Get())
	{
		return;
	}

	if (CrowdManager.IsValid())
	{
		CrowdManager->RemoveFromRoot();
	}
	CrowdManager = NewCrowdManager;
	if (NewCrowdManager != nullptr)
	{
		CrowdManager->AddToRoot();
	}
}

void UNavigationSystemV1::CalcTimeSlicedUpdateData(TArray<double>& OutCurrentTimeSlicedBuildTaskDurations, TArray<bool>& OutIsTimeSlicingArray, bool& bOutAnyNonTimeSlicedGenerators, TArray<int32, TInlineAllocator<8>>& OutNumTimeSlicedRemainingBuildTasksArray)
{
	OutNumTimeSlicedRemainingBuildTasksArray.SetNumZeroed(NavDataSet.Num());
	OutIsTimeSlicingArray.SetNumZeroed(NavDataSet.Num());
	bOutAnyNonTimeSlicedGenerators = false;
	OutCurrentTimeSlicedBuildTaskDurations.Reset(NavDataSet.Num());

	for (int32 NavDataIdx = 0; NavDataIdx < NavDataSet.Num(); ++NavDataIdx)
	{
		const ANavigationData* NavData = NavDataSet[NavDataIdx];
		const FNavDataGenerator* Generator = NavData ? NavData->GetGenerator() : nullptr;
		if (Generator)
		{
			double TimeSlicedBuildTaskDuration = 0.;
			int32 NumRemainingBuildTasksTemp = 0;

			if (Generator->GetTimeSliceData(NumRemainingBuildTasksTemp, TimeSlicedBuildTaskDuration))
			{
				OutIsTimeSlicingArray[NavDataIdx] = true;
				OutNumTimeSlicedRemainingBuildTasksArray[NavDataIdx] += NumRemainingBuildTasksTemp;
				if (TimeSlicedBuildTaskDuration > 0.)
				{
					OutCurrentTimeSlicedBuildTaskDurations.Push(TimeSlicedBuildTaskDuration);
				}
			}
			else
			{
				bOutAnyNonTimeSlicedGenerators = true;
			}
		}
	}
}

void UNavigationSystemV1::Tick(float DeltaSeconds)
{
	SET_DWORD_STAT(STAT_Navigation_ObservedPathsCount, 0);

	UWorld* World = GetWorld();

	if (World == nullptr 
		|| (bTickWhilePaused == false && World->IsPaused())
#if WITH_EDITOR
		|| (bIsPIEActive && !World->IsGameWorld())
#endif // WITH_EDITOR
		)
	{
		return;
	}

	if (PendingNavBoundsUpdates.Num() > 0)
	{
		PerformNavigationBoundsUpdate(PendingNavBoundsUpdates);
		PendingNavBoundsUpdates.Reset();
	}

	if (NavDataRegistrationQueue.Num() > 0)
	{
		CSV_SCOPED_TIMING_STAT(NavigationBuildDetailed, Navigation_ProcessRegistrationCandidates);
		ProcessRegistrationCandidates();
	}

	if (DefaultOctreeController.PendingUpdates.Num() > 0)
	{
		SCOPE_CYCLE_COUNTER(STAT_Navigation_AddingActorsToNavOctree);
		CSV_SCOPED_TIMING_STAT(NavigationBuildDetailed, Navigation_ProcessPendingOctreeUpdates);

		SCOPE_CYCLE_COUNTER(STAT_Navigation_BuildTime)
		STAT(double ThisTime = 0);
		{
			SCOPE_SECONDS_COUNTER(ThisTime);
			FNavigationDataHandler NavHandler(DefaultOctreeController, DefaultDirtyAreasController);
			NavHandler.ProcessPendingOctreeUpdates();
		}
		INC_FLOAT_STAT_BY(STAT_Navigation_CumulativeBuildTime,(float)ThisTime*1000);
	}
		
	if (IsNavigationBuildingLocked() == false)
	{
		if (bGenerateNavigationOnlyAroundNavigationInvokers)
		{
			CSV_SCOPED_TIMING_STAT(NavigationBuildDetailed, Navigation_UpdateInvokers);
			UpdateInvokers();
		}

		{
			CSV_SCOPED_TIMING_STAT(NavigationBuildDetailed, Navigation_RebuildDirtyAreas);
			RebuildDirtyAreas(DeltaSeconds);
		}

		// Tick navigation mesh async builders
		if (bAsyncBuildPaused == false)
		{
			CSV_SCOPED_TIMING_STAT(NavigationBuildDetailed, Navigation_TickAsyncBuild);
			SCOPE_CYCLE_COUNTER(STAT_Navigation_TickAsyncBuild);

			bool bDoStandardTickAsync = true;

			if (NavRegenTimeSliceManager.DoTimeSlicedUpdate())
			{
				TArray<int32, TInlineAllocator<8>> NumTimeSlicedRemainingBuildTasksArray;
				NumTimeSlicedRemainingBuildTasksArray.SetNumZeroed(NavDataSet.Num());
				
				TArray<double> CurrentTimeSlicedBuildTaskDurations;
				TArray<bool> IsTimeSlicingArray;
				bool bAnyNonTimeSlicedGenerators = false;

				NavRegenTimeSliceManager.CalcAverageDeltaTime(GFrameCounter);

				CalcTimeSlicedUpdateData(CurrentTimeSlicedBuildTaskDurations, IsTimeSlicingArray, bAnyNonTimeSlicedGenerators, NumTimeSlicedRemainingBuildTasksArray);

				int32 NumTimeSlicedRemainingBuildTasks = 0;
				for (const int32 NumTasks : NumTimeSlicedRemainingBuildTasksArray)
				{
					NumTimeSlicedRemainingBuildTasks += NumTasks;
				}

#if !UE_BUILD_SHIPPING
				NavRegenTimeSliceManager.LogTileStatistics(NavDataSet);
#endif // !UE_BUILD_SHIPPING
				
				if (NumTimeSlicedRemainingBuildTasks > 0)
				{
					NavRegenTimeSliceManager.CalcTimeSliceDuration(NavDataSet, NumTimeSlicedRemainingBuildTasks, CurrentTimeSlicedBuildTaskDurations);

					//The general idea here is to tick any non time sliced generators once per frame. Time sliced generators we aim to tick one per frame and move to the next, next frame. In the
					//case where one time sliced generator doesn't use the whole time slice we move to the next time sliced generator. That generator will only be considered to have a full frames
					//processing if either it runs out of work or uses a large % of the time slice. Depending we either tick it again next frame or go to the next time sliced generator (next frame).
					bool bNavDataIdxSet = false;
					int32 NavDataIdxTemp = NavRegenTimeSliceManager.GetNavDataIdx();
					constexpr double RemainingFractionConsideredWholeTick = 0.8;
					const int32 FirstNavDataIdx = NavDataIdxTemp = NavDataIdxTemp % NavDataSet.Num();

					for (int32 NavDataIter = 0; NavDataIter < NavDataSet.Num(); ++NavDataIter)
					{
						if (ANavigationData* NavData = NavDataSet[NavDataIdxTemp])
						{
							if (IsTimeSlicingArray[NavDataIdxTemp])
							{
								if (NavRegenTimeSliceManager.GetTimeSlicer().IsTimeSliceFinishedCached())
								{
									//if we haven't set the NavDataIdx then this is the TimeSliced Generator to process next frame
									if (!bNavDataIdxSet)
									{
										NavRegenTimeSliceManager.SetNavDataIdx(NavDataIdxTemp);
										bNavDataIdxSet = true;
									}

									//if the time slice is finished and we have no non time sliced generators then stop TickAsyncBuild, otherwise continue
									if (!bAnyNonTimeSlicedGenerators)
									{
										break;
									}
									continue;
								}
								else if (NavRegenTimeSliceManager.GetTimeSlicer().GetRemainingDurationFraction() < RemainingFractionConsideredWholeTick)
								{
									//don't check bNavDataIdxSet here, either this time sliced generator won't get enough time this frame to be considered
									//a whole tick or it will complete and there is some time sliced left - in the later case next frame we'll process the 
									//next time sliced generator we process this frame or the first Idx we processed this frame
									NavRegenTimeSliceManager.SetNavDataIdx(NavDataIdxTemp);
									bNavDataIdxSet = true;
								}
							}
							NavData->TickAsyncBuild(DeltaSeconds);
						}
						//Increment and mod NavDataIdxTemp
						++NavDataIdxTemp;
						NavDataIdxTemp %= NavDataSet.Num();
					}

					//if we processed all the time sliced generators and there is still some time slice left
					//OR if we haven't SetNavDataIdx() we should start next frame where we started this frame
					if (!NavRegenTimeSliceManager.GetTimeSlicer().IsTimeSliceFinishedCached() || !bNavDataIdxSet)
					{
						NavRegenTimeSliceManager.SetNavDataIdx(FirstNavDataIdx);
						bNavDataIdxSet = true;
					}
					//don't do the standard TickASyncBuild as we have already processed the regen appropriately 
					bDoStandardTickAsync = false;
				}
			}

			//if we aren't time sliced rebuilding and / or if there aren't any time sliced nav data's with work to do just tick all nav data
			if (bDoStandardTickAsync)
			{
				for (ANavigationData* NavData : NavDataSet)
				{
					if (NavData)
					{
						NavData->TickAsyncBuild(DeltaSeconds);
					}
				}
			}
		}
	}

#if !UE_BUILD_SHIPPING && CSV_PROFILER_STATS
	for (const TObjectPtr<ANavigationData>& NavigationData : NavDataSet)
	{
		if (NavigationData)
		{
			if (const FNavDataGenerator* Generator = NavigationData->GetGenerator())
			{
				const int32 BuildTaskNum = Generator->GetNumRemaningBuildTasks();
				const FString StatName = FString::Printf(TEXT("NumRemainingTasks_%s"), *GetNameSafe(NavigationData)); 
				FCsvProfiler::RecordCustomStat(*StatName, CSV_CATEGORY_INDEX(NavTasks), BuildTaskNum, ECsvCustomStatOp::Set);
			}
		}
	}
	
	CSV_CUSTOM_STAT(NavigationSystem, NumRunningTasks, GetNumRunningBuildTasks(), ECsvCustomStatOp::Set);
#endif // !UE_BUILD_SHIPPING && CSV_PROFILER_STATS

	// In multithreaded configuration we can process async pathfinding queries
	// in dedicated task while dispatching completed queries results on the main thread.
	// The created task can start and append new result right away so we transfer
	// completed queries before to keep the list safe.
	TArray<FAsyncPathFindingQuery> AsyncPathFindingCompletedQueriesToDispatch;
	Swap(AsyncPathFindingCompletedQueriesToDispatch, AsyncPathFindingCompletedQueries);

	// Trigger the async pathfinding queries (new ones and those that may have been postponed from last frame)
	if (AsyncPathFindingQueries.Num() > 0)
	{
		SCOPE_CYCLE_COUNTER(STAT_Navigation_TickAsyncPathfinding);
		TriggerAsyncQueries(AsyncPathFindingQueries);
		AsyncPathFindingQueries.Reset();
	}

	// Dispatch async pathfinding queries results from last frame
	DispatchAsyncQueriesResults(AsyncPathFindingCompletedQueriesToDispatch);

	if (CrowdManager.IsValid())
	{
		CSV_SCOPED_TIMING_STAT(NavigationBuildDetailed, Navigation_CrowdManager);
		CrowdManager->Tick(DeltaSeconds);
	}
}

void UNavigationSystemV1::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UNavigationSystemV1* This = CastChecked<UNavigationSystemV1>(InThis);
	Collector.AddReferencedObject(This->CrowdManager, InThis);

	// don't reference NavAreaClasses in editor (unless PIE is active)
	if (!FNavigationSystem::IsEditorRunMode(This->OperationMode))
	{
		Collector.AddReferencedObjects(This->NavAreaClasses, InThis);
	}
}

#if WITH_EDITOR
void UNavigationSystemV1::SetNavigationAutoUpdateEnabled(bool bNewEnable, UNavigationSystemBase* InNavigationSystemBase)
{
	UE_LOGF(LogNavigation, Verbose, "%s bNewEnable: %i", __FUNCTION__, bNewEnable);
	
	if (bNewEnable != bNavigationAutoUpdateEnabled)
	{
		bNavigationAutoUpdateEnabled = bNewEnable;

		UNavigationSystemV1* NavSystem = Cast<UNavigationSystemV1>(InNavigationSystemBase);
		if (NavSystem)
		{
			const bool bCurrentIsEnabled = NavSystem->GetIsAutoUpdateEnabled();
			NavSystem->DefaultDirtyAreasController.bCanAccumulateDirtyAreas = bCurrentIsEnabled
				|| (!FNavigationSystem::IsEditorRunMode(NavSystem->OperationMode) && NavSystem->OperationMode != FNavigationSystemRunMode::InvalidMode);

			if (bCurrentIsEnabled)
			{
				NavSystem->RemoveNavigationBuildLock(ENavigationBuildLock::NoUpdateInEditor);
			}
			else
			{
#if !UE_BUILD_SHIPPING
				NavSystem->DefaultDirtyAreasController.bDirtyAreasReportedWhileAccumulationLocked = false;
#endif // !UE_BUILD_SHIPPING
				NavSystem->AddNavigationBuildLock(ENavigationBuildLock::NoUpdateInEditor);
			}
		}
	}
}
#endif // WITH_EDITOR

//----------------------------------------------------------------------//
// Public querying interface
//----------------------------------------------------------------------//
FPathFindingResult UNavigationSystemV1::FindPathSync(const FNavAgentProperties& AgentProperties, FPathFindingQuery Query, EPathFindingMode::Type Mode)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_PathfindingSync);
	CSV_SCOPED_TIMING_STAT(NavigationSystem, PathfindingSync);

	if (Query.NavData.IsValid() == false)
	{
		Query.NavData = GetNavDataForProps(AgentProperties, Query.StartLocation);
	}

	FPathFindingResult Result(ENavigationQueryResult::Error);
	if (Query.NavData.IsValid())
	{
		if (Mode == EPathFindingMode::Hierarchical)
		{
			Result = Query.NavData->FindHierarchicalPath(AgentProperties, Query);
		}
		else
		{
			Result = Query.NavData->FindPath(AgentProperties, Query);
		}
	}

	return Result;
}

FPathFindingResult UNavigationSystemV1::FindPathSync(FPathFindingQuery Query, EPathFindingMode::Type Mode)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_PathfindingSync);
	CSV_SCOPED_TIMING_STAT(NavigationSystem, PathfindingSync);

	if (Query.NavData.IsValid() == false)
	{
		Query.NavData = GetDefaultNavDataInstance(FNavigationSystem::DontCreate);
	}
	
	FPathFindingResult Result(ENavigationQueryResult::Error);
	if (Query.NavData.IsValid())
	{
		if (Mode == EPathFindingMode::Regular)
		{
			Result = Query.NavData->FindPath(Query.NavAgentProperties, Query);
		}
		else // EPathFindingMode::Hierarchical
		{
			Result = Query.NavData->FindHierarchicalPath(Query.NavAgentProperties, Query);
		}
	}

	return Result;
}

bool UNavigationSystemV1::TestPathSync(FPathFindingQuery Query, EPathFindingMode::Type Mode, int32* NumVisitedNodes) const
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_PathfindingSync);
	CSV_SCOPED_TIMING_STAT(NavigationSystem, PathfindingSync);

	if (Query.NavData.IsValid() == false)
	{
		Query.NavData = GetDefaultNavDataInstance();
	}
	
	bool bExists = false;
	if (Query.NavData.IsValid())
	{
		if (Mode == EPathFindingMode::Hierarchical)
		{
			bExists = Query.NavData->TestHierarchicalPath(Query.NavAgentProperties, Query, NumVisitedNodes);
		}
		else
		{
			bExists = Query.NavData->TestPath(Query.NavAgentProperties, Query, NumVisitedNodes);
		}
	}

	return bExists;
}

void UNavigationSystemV1::AddAsyncQuery(const FAsyncPathFindingQuery& Query)
{
	check(IsInGameThread());
	AsyncPathFindingQueries.Add(Query);
}

uint32 UNavigationSystemV1::FindPathAsync(const FNavAgentProperties& AgentProperties, FPathFindingQuery Query, const FNavPathQueryDelegate& ResultDelegate, EPathFindingMode::Type Mode)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_RequestingAsyncPathfinding);

	if (Query.NavData.IsValid() == false)
	{
		Query.NavData = GetNavDataForProps(AgentProperties, Query.StartLocation);
	}

	if (Query.NavData.IsValid())
	{
		FAsyncPathFindingQuery AsyncQuery(Query, ResultDelegate, Mode);

		if (AsyncQuery.QueryID != INVALID_NAVQUERYID)
		{
			AddAsyncQuery(AsyncQuery);
		}

		return AsyncQuery.QueryID;
	}

	return INVALID_NAVQUERYID;
}

void UNavigationSystemV1::AbortAsyncFindPathRequest(uint32 AsynPathQueryID)
{
	check(IsInGameThread());
	FAsyncPathFindingQuery* Query = AsyncPathFindingQueries.GetData();
	for (int32 Index = 0; Index < AsyncPathFindingQueries.Num(); ++Index, ++Query)
	{
		if (Query->QueryID == AsynPathQueryID)
		{
			AsyncPathFindingQueries.RemoveAtSwap(Index);
			break;
		}
	}
}

FAutoConsoleTaskPriority CPrio_TriggerAsyncQueries(
	TEXT("TaskGraph.TaskPriorities.NavTriggerAsyncQueries"),
	TEXT("Task and thread priority for UNavigationSystemV1::PerformAsyncQueries."),
	ENamedThreads::BackgroundThreadPriority, // if we have background priority task threads, then use them...
	ENamedThreads::NormalTaskPriority, // .. at normal task priority
	ENamedThreads::NormalTaskPriority // if we don't have background threads, then use normal priority threads at normal task priority instead
	);


void UNavigationSystemV1::TriggerAsyncQueries(TArray<FAsyncPathFindingQuery>& PathFindingQueries)
{
	DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.NavigationSystem batched async queries"),
		STAT_FSimpleDelegateGraphTask_NavigationSystemBatchedAsyncQueries,
		STATGROUP_TaskGraphTasks);

	AsyncPathFindingTask = FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
		FSimpleDelegateGraphTask::FDelegate::CreateUObject(this, &UNavigationSystemV1::PerformAsyncQueries, PathFindingQueries),
		GET_STATID(STAT_FSimpleDelegateGraphTask_NavigationSystemBatchedAsyncQueries), nullptr, CPrio_TriggerAsyncQueries.Get());
}

void UNavigationSystemV1::PostponeAsyncQueries()
{
	if (AsyncPathFindingTask.GetReference() && !AsyncPathFindingTask->IsComplete())
	{
		bAbortAsyncQueriesRequested = true;
		FTaskGraphInterface::Get().WaitUntilTaskCompletes(AsyncPathFindingTask, ENamedThreads::GameThread);
		bAbortAsyncQueriesRequested = false;
	}
}

void UNavigationSystemV1::DispatchAsyncQueriesResults(const TArray<FAsyncPathFindingQuery>& PathFindingQueries) const
{
	if (PathFindingQueries.Num() > 0)
	{
		SCOPE_CYCLE_COUNTER(STAT_Navigation_DispatchAsyncPathfindingResults);
		CSV_SCOPED_TIMING_STAT(NavigationSystem, AsyncNavQueryFinished);

		for (const FAsyncPathFindingQuery& Query : PathFindingQueries)
		{
			Query.OnDoneDelegate.ExecuteIfBound(Query.QueryID, Query.Result.Result, Query.Result.Path);
		}
	}
}

void UNavigationSystemV1::PerformAsyncQueries(TArray<FAsyncPathFindingQuery> PathFindingQueries)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_PathfindingAsync);
	CSV_SCOPED_TIMING_STAT(NavigationSystem, PathfindingAsync);

	if (PathFindingQueries.Num() == 0)
	{
		return;
	}

	int32 NumProcessed = 0;
	for (FAsyncPathFindingQuery& Query : PathFindingQueries)
	{
		// perform query
		if (const TStrongObjectPtr<const ANavigationData> NavData = Query.NavData.Pin())
		{
			if (Query.Mode == EPathFindingMode::Hierarchical)
			{
				Query.Result = NavData->FindHierarchicalPath(Query.NavAgentProperties, Query);
			}
			else
			{
				Query.Result = NavData->FindPath(Query.NavAgentProperties, Query);
			}
		}
		else
		{
			Query.Result = ENavigationQueryResult::Error;
		}
		++NumProcessed;

		// Check for abort request from the main tread
		if (bAbortAsyncQueriesRequested)
		{
			break;
		}
	}

	const int32 NumQueries = PathFindingQueries.Num();
	const int32 NumPostponed = NumQueries - NumProcessed;

	// Queue remaining queries for next frame
	if (bAbortAsyncQueriesRequested)
	{
		AsyncPathFindingQueries.Append(PathFindingQueries.GetData() + NumProcessed, NumPostponed);
	}
	
	// Append to list of completed queries to dispatch results in main thread
	AsyncPathFindingCompletedQueries.Append(PathFindingQueries.GetData(), NumProcessed);

	UE_LOGF(LogNavigation, Log, "Async pathfinding queries: %d completed, %d postponed to next frame", NumProcessed, NumPostponed);
}

bool UNavigationSystemV1::GetRandomPoint(FNavLocation& ResultLocation, ANavigationData* NavData, FSharedConstNavQueryFilter QueryFilter)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_QueriesTimeSync);

	if (NavData == nullptr)
	{
		NavData = MainNavData;
	}

	if (NavData != nullptr)
	{
		ResultLocation = NavData->GetRandomPoint(QueryFilter);
		return true;
	}

	return false;
}

bool UNavigationSystemV1::GetRandomReachablePointInRadius(const FVector& Origin, float Radius, FNavLocation& ResultLocation, ANavigationData* NavData, FSharedConstNavQueryFilter QueryFilter) const
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_QueriesTimeSync);

	if (NavData == nullptr)
	{
		NavData = MainNavData;
	}

	return NavData != nullptr && NavData->GetRandomReachablePointInRadius(Origin, Radius, ResultLocation, QueryFilter);
}

bool UNavigationSystemV1::GetRandomPointInNavigableRadius(const FVector& Origin, float Radius, FNavLocation& ResultLocation, ANavigationData* NavData, FSharedConstNavQueryFilter QueryFilter) const
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_QueriesTimeSync);

	if (NavData == nullptr)
	{
		NavData = MainNavData;
	}

	return NavData != nullptr && NavData->GetRandomPointInNavigableRadius(Origin, Radius, ResultLocation, QueryFilter);
}

ENavigationQueryResult::Type UNavigationSystemV1::GetPathCost(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& OutPathCost, const ANavigationData* NavData, FSharedConstNavQueryFilter QueryFilter) const
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_QueriesTimeSync);

	if (NavData == nullptr)
	{
		NavData = GetDefaultNavDataInstance();
	}

	return NavData != nullptr ? NavData->CalcPathCost(PathStart, PathEnd, OutPathCost, QueryFilter) : ENavigationQueryResult::Error;
}

ENavigationQueryResult::Type UNavigationSystemV1::GetPathLength(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& OutPathLength, const ANavigationData* NavData, FSharedConstNavQueryFilter QueryFilter) const
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_QueriesTimeSync);

	if (NavData == nullptr)
	{
		NavData = GetDefaultNavDataInstance();
	}

	return NavData != nullptr ? NavData->CalcPathLength(PathStart, PathEnd, OutPathLength, QueryFilter) : ENavigationQueryResult::Error;
}

ENavigationQueryResult::Type UNavigationSystemV1::GetPathLengthAndCost(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& OutPathLength, FVector::FReal& OutPathCost, const ANavigationData* NavData, FSharedConstNavQueryFilter QueryFilter) const
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_QueriesTimeSync);

	if (NavData == nullptr)
	{
		NavData = GetDefaultNavDataInstance();
	}

	return NavData != nullptr ? NavData->CalcPathLengthAndCost(PathStart, PathEnd, OutPathLength, OutPathCost, QueryFilter) : ENavigationQueryResult::Error;
}

bool UNavigationSystemV1::ProjectPointToNavigation(const FVector& Point, FNavLocation& OutLocation, const FVector& Extent, const ANavigationData* NavData, FSharedConstNavQueryFilter QueryFilter) const
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_QueriesTimeSync);

	if (NavData == nullptr)
	{
		NavData = GetDefaultNavDataInstance();
	}

	return NavData != nullptr && NavData->ProjectPoint(Point, OutLocation
											, UE::Navigation::Private::IsValidExtent(Extent) ? Extent : NavData->GetConfig().DefaultQueryExtent
											, QueryFilter);
}

UNavigationPath* UNavigationSystemV1::FindPathToActorSynchronously(UObject* WorldContextObject, const FVector& PathStart, AActor* GoalActor, float TetherDistance, AActor* PathfindingContext, TSubclassOf<UNavigationQueryFilter> FilterClass)
{
	if (GoalActor == nullptr)
	{
		return nullptr; 
	}

	INavAgentInterface* NavAgent = Cast<INavAgentInterface>(GoalActor);
	UNavigationPath* GeneratedPath = FindPathToLocationSynchronously(WorldContextObject, PathStart, NavAgent ? NavAgent->GetNavAgentLocation() : GoalActor->GetActorLocation(), PathfindingContext, FilterClass);
	if (GeneratedPath != nullptr && GeneratedPath->GetPath().IsValid() == true)
	{
		GeneratedPath->GetPath()->SetGoalActorObservation(*GoalActor, TetherDistance);
	}

	return GeneratedPath;
}

UNavigationPath* UNavigationSystemV1::FindPathToLocationSynchronously(UObject* WorldContextObject, const FVector& PathStart, const FVector& PathEnd, AActor* PathfindingContext, TSubclassOf<UNavigationQueryFilter> FilterClass)
{
	UWorld* World = nullptr;

	if (WorldContextObject != nullptr)
	{
		World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	}
	if (World == nullptr && PathfindingContext != nullptr)
	{
		World = GEngine->GetWorldFromContextObject(PathfindingContext, EGetWorldErrorMode::LogAndReturnNull);
	}

	UNavigationPath* ResultPath = nullptr;

	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);

	if (NavSys != nullptr && NavSys->GetDefaultNavDataInstance() != nullptr)
	{
		ResultPath = NewObject<UNavigationPath>(NavSys);
		bool bValidPathContext = false;
		const ANavigationData* NavigationData = nullptr;

		if (PathfindingContext != nullptr)
		{
			INavAgentInterface* NavAgent = Cast<INavAgentInterface>(PathfindingContext);
			
			if (NavAgent != nullptr)
			{
				const FNavAgentProperties& AgentProps = NavAgent->GetNavAgentPropertiesRef();
				NavigationData = NavSys->GetNavDataForProps(AgentProps, PathStart);
				bValidPathContext = true;
			}
			else if (Cast<ANavigationData>(PathfindingContext))
			{
				NavigationData = (ANavigationData*)PathfindingContext;
				bValidPathContext = true;
			}
		}
		if (bValidPathContext == false)
		{
			// just use default
			NavigationData = NavSys->GetDefaultNavDataInstance();
		}

		check(NavigationData);

		const FPathFindingQuery Query(PathfindingContext, *NavigationData, PathStart, PathEnd, UNavigationQueryFilter::GetQueryFilter(*NavigationData, PathfindingContext, FilterClass));
		const FPathFindingResult Result = NavSys->FindPathSync(Query, EPathFindingMode::Regular);
		if (Result.IsSuccessful())
		{
			ResultPath->SetPath(Result.Path);
		}
	}

	return ResultPath;
}

bool UNavigationSystemV1::NavigationRaycast(UObject* WorldContextObject, const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, TSubclassOf<UNavigationQueryFilter> FilterClass, AController* Querier)
{
	return NavigationRaycastWithAdditionalResults(WorldContextObject, RayStart, RayEnd, HitLocation, nullptr, FilterClass, Querier);
}

bool UNavigationSystemV1::NavigationRaycastWithAdditionalResults(UObject* WorldContextObject, const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, FNavigationRaycastAdditionalResults* AdditionalResults, TSubclassOf<UNavigationQueryFilter> FilterClass, AController* Querier)
{
	UWorld* World = nullptr;

	if (WorldContextObject != nullptr)
	{
		World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	}
	if (World == nullptr && Querier != nullptr)
	{
		World = GEngine->GetWorldFromContextObject(Querier, EGetWorldErrorMode::LogAndReturnNull);
	}

	// blocked, i.e. not traversable, by default
	bool bRaycastBlocked = true;
	HitLocation = RayStart;
	if (AdditionalResults)
	{
		AdditionalResults->bIsRayEndInCorridor = false;
	}

	const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);

	if (NavSys)
	{
		// figure out which navigation data to use
		const ANavigationData* NavData = nullptr;
		INavAgentInterface* MyNavAgent = Cast<INavAgentInterface>(Querier);
		if (MyNavAgent)
		{
			const FNavAgentProperties& AgentProps = MyNavAgent->GetNavAgentPropertiesRef();
			NavData = NavSys->GetNavDataForProps(AgentProps, RayStart);
		}
		if (NavData == nullptr)
		{
			NavData = NavSys->GetDefaultNavDataInstance();
		}

		if (NavData != nullptr)
		{
			bRaycastBlocked = NavData->Raycast(RayStart, RayEnd, HitLocation, AdditionalResults, UNavigationQueryFilter::GetQueryFilter(*NavData, Querier, FilterClass));
		}
	}

	return bRaycastBlocked;
}

void UNavigationSystemV1::GetNavAgentPropertiesArray(TArray<FNavAgentProperties>& OutNavAgentProperties) const
{
	AgentToNavDataMap.GetKeys(OutNavAgentProperties);
}

ANavigationData* UNavigationSystemV1::GetNavDataForProps(const FNavAgentProperties& AgentProperties, const FVector& AgentLocation, const FVector& Extent) const
{
	return const_cast<ANavigationData*>(GetNavDataForProps(AgentProperties));
}

ANavigationData* UNavigationSystemV1::GetNavDataForProps(const FNavAgentProperties& AgentProperties)
{
	return const_cast<ANavigationData*>(AsConst(*this).GetNavDataForProps(AgentProperties));
}

// @todo could optimize this by having "SupportedAgentIndex" in FNavAgentProperties
const ANavigationData* UNavigationSystemV1::GetNavDataForProps(const FNavAgentProperties& AgentProperties) const
{
	if (SupportedAgents.Num() <= 1)
	{
		return MainNavData;
	}

	// Because an invalid AgentProperties uses -1 values the code below is able to match the PreferredNavData.
	UE_CLOGF(!(AgentProperties.IsValid() || AgentProperties.PreferredNavData.IsValid()), LogNavigation, Warning, "Looking for NavData using invalid FNavAgentProperties.");
	
	const TWeakObjectPtr<ANavigationData>* NavDataForAgent = AgentToNavDataMap.Find(AgentProperties);
	const ANavigationData* NavDataInstance = NavDataForAgent ? NavDataForAgent->Get() : nullptr;

	if (NavDataInstance == nullptr)
	{
		TArray<FNavAgentProperties> AgentPropertiesList;
		AgentToNavDataMap.GenerateKeyArray(AgentPropertiesList);
		
		FNavAgentProperties BestFitNavAgent;
		float BestExcessHeight = -FLT_MAX;
		float BestExcessRadius = -FLT_MAX;
		float ExcessRadius = -FLT_MAX;
		float ExcessHeight = -FLT_MAX;
		const float AgentHeight = bSkipAgentHeightCheckWhenPickingNavData ? 0.f : AgentProperties.AgentHeight;

		for (TArray<FNavAgentProperties>::TConstIterator It(AgentPropertiesList); It; ++It)
		{
			const FNavAgentProperties& NavIt = *It;
			const bool bNavClassMatch = NavIt.IsNavDataMatching(AgentProperties);
			if (!bNavClassMatch)
			{
				continue;
			}

			// Make sure to only allow NavData and Agent matches that have matching bCanFly.
			// This prevents ground-based Pawns from inadvertently getting associated with flying-based NavData when their PreferredNavData isn't set.

			if (NavIt.bCanFly != AgentProperties.bCanFly)
			{
				continue;
			}

			ExcessRadius = NavIt.AgentRadius - AgentProperties.AgentRadius;
			ExcessHeight = bSkipAgentHeightCheckWhenPickingNavData ? 0.f : (NavIt.AgentHeight - AgentHeight);

			const bool bExcessRadiusIsBetter = ((ExcessRadius == 0) && (BestExcessRadius != 0)) 
				|| ((ExcessRadius > 0) && (BestExcessRadius < 0))
				|| ((ExcessRadius > 0) && (BestExcessRadius > 0) && (ExcessRadius < BestExcessRadius))
				|| ((ExcessRadius < 0) && (BestExcessRadius < 0) && (ExcessRadius > BestExcessRadius));
			const bool bExcessHeightIsBetter = ((ExcessHeight == 0) && (BestExcessHeight != 0))
				|| ((ExcessHeight > 0) && (BestExcessHeight < 0))
				|| ((ExcessHeight > 0) && (BestExcessHeight > 0) && (ExcessHeight < BestExcessHeight))
				|| ((ExcessHeight < 0) && (BestExcessHeight < 0) && (ExcessHeight > BestExcessHeight));
			const bool bBestIsValid = (BestExcessRadius >= 0) && (BestExcessHeight >= 0);
			const bool bRadiusEquals = (ExcessRadius == BestExcessRadius);
			const bool bHeightEquals = (ExcessHeight == BestExcessHeight);

			bool bValuesAreBest = ((bExcessRadiusIsBetter || bRadiusEquals) && (bExcessHeightIsBetter || bHeightEquals));
			if (!bValuesAreBest && !bBestIsValid)
			{
				bValuesAreBest = bExcessRadiusIsBetter || (bRadiusEquals && bExcessHeightIsBetter);
			}

			if (bValuesAreBest)
			{
				BestFitNavAgent = NavIt;
				BestExcessHeight = ExcessHeight;
				BestExcessRadius = ExcessRadius;
			}
		}

		if (BestFitNavAgent.IsValid())
		{
			NavDataForAgent = AgentToNavDataMap.Find(BestFitNavAgent);
			NavDataInstance = NavDataForAgent ? NavDataForAgent->Get() : nullptr;
		}
	}

	return NavDataInstance ? NavDataInstance : MainNavData;
}

ANavigationData* UNavigationSystemV1::GetNavDataForAgentName(const FName AgentName) const
{
	ANavigationData* Result = nullptr;

	for (ANavigationData* NavData : NavDataSet)
	{
		if (IsValid(NavData) && NavData->GetConfig().Name == AgentName)
		{
			Result = NavData;
			break;
		}
	}

	return Result;
}

FBox UNavigationSystemV1::GetNavigableWorldBounds() const
{
	return GetWorldBounds();
}

void UNavigationSystemV1::SetBuildBounds(const FBox& Bounds)
{
	BuildBounds = Bounds;
}

bool UNavigationSystemV1::ContainsNavData(const FBox& Bounds) const
{
	for (const ANavigationData* NavData : NavDataSet)
	{
		if (NavData && Bounds.Intersect(NavData->GetBounds()))
		{
			return true;
		}
	}
	return false;
}

FBox UNavigationSystemV1::ComputeNavDataBounds() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UNavigationSystemV1::ComputeNavDataBounds);
	
	FBox Bounds(ForceInit);
	for (const ANavigationData* NavData : NavDataSet)
	{
		if (NavData)
		{
			Bounds += NavData->GetBounds();
		}
	}
	return Bounds;
}

void UNavigationSystemV1::AddNavigationDataChunk(ANavigationDataChunkActor& DataChunkActor)
{
	for (ANavigationData* NavData : NavDataSet)
	{
		if (NavData)
		{
			NavData->OnStreamingNavDataAdded(DataChunkActor);
		}
	}
}

void UNavigationSystemV1::RemoveNavigationDataChunk(ANavigationDataChunkActor& DataChunkActor)
{
	for (ANavigationData* NavData : NavDataSet)
	{
		if (NavData)
		{
			NavData->OnStreamingNavDataRemoved(DataChunkActor);
		}
	}
}

void UNavigationSystemV1::FillNavigationDataChunkActor(const FBox& QueryBounds, ANavigationDataChunkActor& DataChunkActor, FBox& OutTilesBounds)
{
	for (const ANavigationData* NavData : NavDataSet)
	{
		if (NavData)
		{
			NavData->FillNavigationDataChunkActor(QueryBounds, DataChunkActor, OutTilesBounds);
		}
	}
}

ANavigationData* UNavigationSystemV1::GetDefaultNavDataInstance(FNavigationSystem::ECreateIfMissing CreateNewIfNoneFound)
{
	checkSlow(IsInGameThread() == true);

	if (!IsValid(MainNavData))
	{
		MainNavData = nullptr;

		// @TODO this should be done a differently. There should be specified a "default agent"
		for (int32 NavDataIndex = 0; NavDataIndex < NavDataSet.Num(); ++NavDataIndex)
		{
			ANavigationData* NavData = NavDataSet[NavDataIndex];
			if (IsValid(NavData) && NavData->CanBeMainNavData()
				&& (DefaultAgentName == NAME_None || NavData->GetConfig().Name == DefaultAgentName))
			{
				MainNavData = NavData;
				break;
			}
		}

#if WITH_RECAST
		if (/*GIsEditor && */(MainNavData == nullptr) && CreateNewIfNoneFound == FNavigationSystem::Create)
		{
			// Spawn a new one if we're in the editor.  In-game, either we loaded one or we don't get one.
			MainNavData = GetWorld()->SpawnActor<ANavigationData>(ARecastNavMesh::StaticClass());
		}
#endif // WITH_RECAST
		
		// either way make sure it's registered. Registration stores unique
		// navmeshes, so we have nothing to lose
		if (MainNavData != nullptr && !MainNavData->IsRegistered())
		{
			const ERegistrationResult Result = RegisterNavData(MainNavData);
			LogNavDataRegistrationResult(Result, MainNavData);
		}
	}

	return MainNavData;
}

FSharedNavQueryFilter UNavigationSystemV1::CreateDefaultQueryFilterCopy() const 
{ 
	return MainNavData ? MainNavData->GetDefaultQueryFilter()->GetCopy() : nullptr; 
}

bool UNavigationSystemV1::IsNavigationBuilt(const AWorldSettings* Settings) const
{
	if (Settings == nullptr || Settings->IsNavigationSystemEnabled() == false || IsThereAnywhereToBuildNavigation() == false)
	{
		return true;
	}

	bool bIsBuilt = true;

	for (int32 NavDataIndex = 0; NavDataIndex < NavDataSet.Num(); ++NavDataIndex)
	{
		ANavigationData* NavData = NavDataSet[NavDataIndex];
		if (NavData != nullptr && NavData->GetWorldSettings() == Settings)
		{
			FNavDataGenerator* Generator = NavData->GetGenerator();
			if ((NavData->GetRuntimeGenerationMode() != ERuntimeGenerationType::Static
#if WITH_EDITOR
				|| GEditor != nullptr
#endif // WITH_EDITOR
				) && (Generator == nullptr || Generator->IsBuildInProgressCheckDirty() == true))
			{
				bIsBuilt = false;
				break;
			}
		}
	}

	return bIsBuilt;
}

bool UNavigationSystemV1::IsThereAnywhereToBuildNavigation() const
{
	// not check if there are any volumes or other structures requiring/supporting navigation building
	if (bWholeWorldNavigable == true)
	{
		return true;
	}

	for (const FNavigationBounds& Bounds : RegisteredNavBounds)
	{
		if (Bounds.AreaBox.IsValid)
		{
			return true;
		}
	}

	// @TODO this should be made more flexible to be able to trigger this from game-specific 
	// code (like Navigation System's subclass maybe)
	bool bCreateNavigation = false;

	for (TActorIterator<ANavMeshBoundsVolume> It(GetWorld()); It; ++It)
	{
		ANavMeshBoundsVolume const* const V = (*It);
		if (IsValid(V))
		{
			bCreateNavigation = true;
			break;
		}
	}

	return bCreateNavigation;
}

bool UNavigationSystemV1::IsNavigationRelevant(const AActor* TestActor) const
{
	const INavRelevantInterface* NavInterface = Cast<const INavRelevantInterface>(TestActor);
	if (NavInterface && NavInterface->IsNavigationRelevant())
	{
		return true;
	}

	if (TestActor)
	{
		TInlineComponentArray<UActorComponent*> Components;
		TestActor->GetComponents(Components);
		for (int32 Idx = 0; Idx < Components.Num(); Idx++)
		{
			NavInterface = Cast<const INavRelevantInterface>(Components[Idx]);
			if (NavInterface && NavInterface->IsNavigationRelevant())
			{
				return true;
			}
		}
	}

	return false;
}

FBox UNavigationSystemV1::GetWorldBounds() const
{
	checkSlow(IsInGameThread() == true);

	NavigableWorldBounds = FBox(ForceInit);

	if (GetWorld() != nullptr)
	{
		if (bWholeWorldNavigable == false)
		{
			for (const FNavigationBounds& Bounds : RegisteredNavBounds)
			{
				NavigableWorldBounds += Bounds.AreaBox;
			}
		}
		else
		{
			// @TODO - super slow! Need to ask where I can get this from
			for (FActorIterator It(GetWorld()); It; ++It)
			{
				if (IsNavigationRelevant(*It))
				{
					UE_LOGF(LogNavigation, VeryVerbose, "%s %ls is NavRelevant with bounds %ls (%ls).",
						__FUNCTION__, *It->GetName(), *(*It)->GetComponentsBoundingBox().ToCompactString(), *It->GetFullName());
					NavigableWorldBounds += (*It)->GetComponentsBoundingBox();
				}
			}
		}
	}

	UE_LOGF(LogNavigation, Log, "%s NavigableWorldBounds: %ls.", __FUNCTION__, *NavigableWorldBounds.ToCompactString());
	return NavigableWorldBounds;
}

FBox UNavigationSystemV1::GetLevelBounds(ULevel* InLevel) const
{
	FBox NavigableLevelBounds(ForceInit);

	if (InLevel)
	{
		auto Actor = InLevel->Actors.CreateConstIterator();
		const int32 ActorCount = InLevel->Actors.Num();
		for (int32 ActorIndex = 0; ActorIndex < ActorCount; ++ActorIndex, ++Actor)
		{
			if (IsNavigationRelevant(*Actor))
			{
				NavigableLevelBounds += (*Actor)->GetComponentsBoundingBox();
			}
		}
	}

	return NavigableLevelBounds;
}

const TSet<FNavigationBounds>& UNavigationSystemV1::GetNavigationBounds() const
{
	return RegisteredNavBounds;
}

void UNavigationSystemV1::ApplyWorldOffset(const FVector& InOffset, bool bWorldShift)
{
	// Move the navmesh bounds by the offset
	for (FNavigationBounds& Bounds : RegisteredNavBounds)
	{
		Bounds.AreaBox = Bounds.AreaBox.ShiftBy(InOffset);
	}

	// Attempt at generation of new nav mesh after the shift
	// Only fully dynamic navmesh needs a complete geometry rebuild after a world shift.
	if (GetRuntimeGenerationType() == ERuntimeGenerationType::Dynamic)
	{
		//stop generators from building navmesh
		CancelBuild();

		ConditionalPopulateNavOctree();
		Build();

		for (ANavigationData* NavData : NavDataSet)
		{
			if (NavData)
			{
				NavData->ConditionalConstructGenerator();
#if WITH_RECAST
				ARecastNavMesh* RecastNavMesh = Cast<ARecastNavMesh>(NavData);
				if (RecastNavMesh)
				{
					RecastNavMesh->RequestDrawingUpdate();
				}
#endif // WITH_RECAST
			}
		}
	}
	else // static navmesh
	{
		//not sure what happens when we shift farther than the extents of the NavOctree are
		for (ANavigationData* NavData : NavDataSet)
		{
			if (NavData)
			{
				NavData->ApplyWorldOffset(InOffset, bWorldShift);
			}
		}
	}
}

//----------------------------------------------------------------------//
// Bookkeeping 
//----------------------------------------------------------------------//
void UNavigationSystemV1::RequestRegistrationDeferred(ANavigationData& NavData)
{
	FScopeLock RegistrationLock(&NavDataRegistrationSection);

	if (NavDataRegistrationQueue.Num() < REGISTRATION_QUEUE_SIZE)
	{
		UE_LOGF(LogNavigation, Verbose, "%s Adding to registration queue: %ls.", __FUNCTION__, *GetFullNameSafe(&NavData));
		NavDataRegistrationQueue.AddUnique(&NavData);
	}
	else
	{
		UE_LOGF(LogNavigation, Warning, "%s Navigation System: registration queue full! System: %ls NavData: %ls.", __FUNCTION__, *GetPathNameSafe(this), *GetPathNameSafe(&NavData));
	}
}

void UNavigationSystemV1::ProcessRegistrationCandidates()
{
	FScopeLock RegistrationLock(&NavDataRegistrationSection);

	if (NavDataRegistrationQueue.Num() == 0)
	{
		return;
	}
	
	const int CandidatesCount = NavDataRegistrationQueue.Num();
	int32 NumNavDataProcessed = 0;
	for (int32 CandidateIndex = CandidatesCount - 1; CandidateIndex >= 0; --CandidateIndex)
	{
		ANavigationData* NavDataPtr = NavDataRegistrationQueue[CandidateIndex];
		ULevel* OwningLevel = NavDataPtr != nullptr ? NavDataPtr->GetLevel() : nullptr;
		if (OwningLevel && OwningLevel->bIsVisible)
		{
			const ERegistrationResult Result = RegisterNavData(NavDataPtr);
			LogNavDataRegistrationResult(Result, NavDataPtr);

			if (Result != RegistrationSuccessful && Result != RegistrationFailed_DataPendingKill)
			{
				UE_LOGF(LogNavigation, Warning, "%s %ls will be removed. This warning will disapear once the map is saved.", 
					__FUNCTION__, *GetNameSafe(NavDataPtr));
				
#if WITH_EDITOR				
				if (GEditor)
				{
					NavigationDataToRemove.Add(NavDataPtr);	
				}
				else
#endif // WITH_EDITOR					
				{
					NavDataPtr->Destroy();
				}

				if (NavDataPtr == MainNavData)
				{
					MainNavData = nullptr;
				}
			}

			NumNavDataProcessed++;
			NavDataRegistrationQueue.RemoveAtSwap(CandidateIndex);
		}
		else
		{
			UE_LOGF(LogNavigation, Verbose, "%s   Skipped %ls because owning level not visible.", __FUNCTION__, *GetFullNameSafe(NavDataPtr));
		}
	}	
	
	if (NumNavDataProcessed)
	{
		MainNavData = GetDefaultNavDataInstance(FNavigationSystem::DontCreate);

		// See if any of registered navigation data now needs NavOctree
		if (DefaultOctreeController.IsValid() == false && RequiresNavOctree() == true)
		{
			ConditionalPopulateNavOctree();
		}
	}
}

void UNavigationSystemV1::ProcessCustomLinkPendingRegistration()
{
	if (Repository == nullptr)
	{
		return;
	}

	for (TWeakInterfacePtr<INavLinkCustomInterface> It : Repository->GetCustomLinks())
	{
		if (INavLinkCustomInterface* Interface = It.Get())
		{
			RegisterCustomLink(*Interface);
		}
	}
}

UNavigationSystemV1::ERegistrationResult UNavigationSystemV1::RegisterNavData(ANavigationData* NavData)
{
	UE_LOGF(LogNavigation, Verbose, "%s %ls", __FUNCTION__, *GetFullNameSafe(NavData));
	
	if (NavData == nullptr)
	{
		return RegistrationError;
	}

	if (!IsValid(NavData))
	{
		return RegistrationFailed_DataPendingKill;
	}

	if (NavData->IsRegistered() == true)
	{
		return RegistrationSuccessful;
	}

	FScopeLock Lock(&NavDataRegistration);

	UNavigationSystemV1::ERegistrationResult Result = RegistrationError;

	// find out which, if any, navigation agents are supported by this nav data
	// if none then fail the registration
	FNavDataConfig NavConfig = NavData->GetConfig();

	// not discarding navmesh when there's only one Supported Agent
	if (NavConfig.IsValid() == false && SupportedAgents.Num() == 1)
	{
		// fill in AgentProps with whatever is the instance's setup
		NavConfig = SupportedAgents[0];
		NavData->SetConfig(SupportedAgents[0]);
		NavData->SetSupportsDefaultAgent(true);	
		NavData->ProcessNavAreas(ObjectPtrDecay(NavAreaClasses), 0);
	}

	if (NavConfig.IsValid() == true)
	{
		if (NavData->IsA(AAbstractNavData::StaticClass()))
		{
			if (AbstractNavData == nullptr || AbstractNavData == NavData)
			{
				// fake registration since it's a special navigation data type 
				// and it would get discarded for not implementing any particular
				// navigation agent
				// Node that we don't add abstract navigation data to NavDataSet
				NavData->OnRegistered();

				Result = RegistrationSuccessful;
			}
			else
			{
				// otherwise specified agent type already has its navmesh implemented, fail redundant instance
				Result = RegistrationFailed_AgentAlreadySupported;
			}
		}
		else
		{
			// check if this kind of agent has already its navigation implemented
			TWeakObjectPtr<ANavigationData>* NavDataForAgent = AgentToNavDataMap.Find(NavConfig);
			ANavigationData* NavDataInstanceForAgent = NavDataForAgent ? NavDataForAgent->Get() : nullptr;

			if (NavDataInstanceForAgent == nullptr)
			{
				// ok, so this navigation agent doesn't have its navmesh registered yet, but do we want to support it?
				bool bAgentSupported = false;

				for (int32 AgentIndex = 0; AgentIndex < SupportedAgents.Num(); ++AgentIndex)
				{
					if (NavData->GetClass() == SupportedAgents[AgentIndex].GetNavDataClass<ANavigationData>()
						&& SupportedAgents[AgentIndex].IsEquivalent(NavConfig) == true)
					{
						// it's supported, then just in case it's not a precise match (IsEquivalent succeeds with some precision) 
						// update NavData with supported Agent
						bAgentSupported = true;

						NavData->SetConfig(SupportedAgents[AgentIndex]);
						AgentToNavDataMap.Add(SupportedAgents[AgentIndex], NavData);
						NavData->SetSupportsDefaultAgent(SupportedAgents[AgentIndex].Name == DefaultAgentName);
						NavData->ProcessNavAreas(ObjectPtrDecay(NavAreaClasses), AgentIndex);

						OnNavDataRegisteredEvent.Broadcast(NavData);

						NavDataSet.AddUnique(NavData);
						NavData->OnRegistered();

						break;
					}
				}
				Result = bAgentSupported == true ? RegistrationSuccessful : RegistrationFailed_AgentNotValid;
			}
			else if (NavDataInstanceForAgent == NavData)
			{
				ensure(NavDataSet.Find(NavData) != INDEX_NONE);
				// let's treat double registration of the same nav data with the same agent as a success
				Result = RegistrationSuccessful;
			}
			else
			{
				// otherwise specified agent type already has its navmesh implemented, fail redundant instance
				Result = RegistrationFailed_AgentAlreadySupported;
			}
		}
	}
	else
	{
		Result = RegistrationFailed_AgentNotValid;
	}

	NavRegenTimeSliceManager.ResetTileWaitTimeArrays(NavDataSet);

#if !UE_BUILD_SHIPPING
	NavRegenTimeSliceManager.ResetTileHistoryData(NavDataSet);
#endif // UE_BUILD_SHIPPING

	// @todo else might consider modifying this NavData to implement navigation for one of the supported agents
	// care needs to be taken to not make it implement navigation for agent who's real implementation has 
	// not been loaded yet.

	if (Result == RegistrationSuccessful && CrowdManager != nullptr)
	{
		CrowdManager->OnNavDataRegistered(*NavData);
	}

	return Result;
}

void UNavigationSystemV1::UnregisterNavData(ANavigationData* NavData)
{
	UE_LOGF(LogNavigation, Verbose, "%s %ls", __FUNCTION__, *GetFullNameSafe(NavData));
	
	NavDataSet.RemoveSingle(NavData);

	if (NavData == nullptr)
	{
		return;
	}

	AgentToNavDataMap.Remove(NavData->GetNavAgentProperties());

	FScopeLock Lock(&NavDataRegistration);
	NavDataRegistrationQueue.Remove(NavData);
	NavData->OnUnregistered();

	NavRegenTimeSliceManager.ResetTileWaitTimeArrays(NavDataSet);

#if !UE_BUILD_SHIPPING
	NavRegenTimeSliceManager.ResetTileHistoryData(NavDataSet);
#endif // UE_BUILD_SHIPPING
	
	if (CrowdManager != nullptr)
	{
		CrowdManager->OnNavDataUnregistered(*NavData);
	}
}

void UNavigationSystemV1::RegisterCustomLink(INavLinkCustomInterface& CustomLink)
{
	ensureMsgf(CustomLink.GetLinkOwner() == nullptr || GetWorld() == CustomLink.GetLinkOwner()->GetWorld(),
		TEXT("Registering a link from a world different than the navigation system world should not happen."));

	const FNavLinkId OldId = CustomLink.GetId();
	FNavLinkId NewId = OldId;
	bool bGenerateNewId = false;

	// Test for Id clash
	if (CustomNavLinksMap.Contains(OldId))
	{
		if (OldId.IsLegacyId() == false)
		{
			UWorld* World = GetWorld();
			check(World);

			// During PIE or game we just generate a new Id, this is most likely to be from a runtime (non editor placed) prefab like a level instance but could be from 
			// a legitimate but extremely unlikely Id clash after loading.
			// If this occurs in EWorldType::Editor world it's a legitimate ID clash, currently we do not handle this edge case here as it should be incredibly unlikely to occur
			// and we do not save changes when cooking or building paths running a commandlet etc.
			bGenerateNewId = World->WorldType == EWorldType::PIE || World->WorldType == EWorldType::Game;
			if (ensureMsgf(bGenerateNewId, TEXT("Id clash in non Game and non PIE world. This should be incredibly rare!")))
			{
				// Pass in NewGuid() here as EWorldType::Game does not have access to the ActorInstanceGuid in any case and any random Unique Guid is acceptable here 
				// if we are not in EWorldType::Editor. Editor is different as we need the cook to be deterministic but for level instances individual actors are not 
				// serialized (but they are when cooked).
				NewId = FNavLinkId::GenerateUniqueId(CustomLink.GetAuxiliaryId(), FGuid::NewGuid());
			}
			
			// This should be very unlikely to occur, if its causing issues we should add code to handle this being careful to account for the editor world being run as a commandlet to cook and build paths on separate runs.
			UE_CLOGF(!bGenerateNewId, LogNavLink, Warning, "%s navlink ID %llu is clashing with existing ID (Owner: %ls). "
				"This will not be regenerated automatically in editor although for dynamic navmesh this will be handled at run time in game. "
				"For static mesh in the editor world the INavLinkCustomInterface implementor should regenerate the ID, "
				"deleting the owning actor and or component and placing again should fix this.", __FUNCTION__, CustomLink.GetId().GetId(), *GetFullNameSafe(CustomLink.GetLinkOwner()));
		}
		else
		{
			bGenerateNewId = true;
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			NewId = FNavLinkId(INavLinkCustomInterface::GetUniqueId());
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		// If the Id has changed mark the area dirty, this will fix the clash in the editor world and also in game for dynamic Navmesh, but not in game for static Navmesh.
		if (NewId != OldId)
		{
			CustomLink.UpdateLinkId(NewId);
			UE_LOGF(LogNavLink, VeryVerbose, "%s new navlink ID %llu.", __FUNCTION__, CustomLink.GetId().GetId());

			const FBox LinkBounds = ComputeCustomLinkBounds(CustomLink);
			if (LinkBounds.IsValid)
			{
				AddDirtyArea(LinkBounds, ENavigationDirtyFlag::DynamicModifier);
			}
		}
	}

	ensureMsgf(CustomLink.GetId().IsValid(), TEXT("%hs, registering a CustomLink with an invalid id."), __FUNCTION__);
	
	UE_CLOGF(bGenerateNewId && CustomNavLinksMap.Contains(CustomLink.GetId()), LogNavLink, Warning, "%s New navlink ID %llu is clashing with existing ID (Owner: %ls).",
		__FUNCTION__, CustomLink.GetId().GetId(), *GetFullNameSafe(CustomLink.GetLinkOwner()));
	CustomNavLinksMap.Add(CustomLink.GetId(), FNavigationSystem::FCustomLinkOwnerInfo(&CustomLink));
}

void UNavigationSystemV1::UnregisterCustomLink(INavLinkCustomInterface& CustomLink)
{
	CustomNavLinksMap.Remove(CustomLink.GetId());
}

INavLinkCustomInterface* UNavigationSystemV1::GetCustomLink(FNavLinkId UniqueLinkId) const
{
	const FNavigationSystem::FCustomLinkOwnerInfo* LinkInfo = CustomNavLinksMap.Find(UniqueLinkId);
	return (LinkInfo && LinkInfo->IsValid()) ? LinkInfo->LinkInterface : nullptr;
}

void UNavigationSystemV1::UpdateCustomLink(const INavLinkCustomInterface* CustomLink)
{
	for (TMap<FNavAgentProperties, TWeakObjectPtr<ANavigationData> >::TIterator It(AgentToNavDataMap); It; ++It)
	{
		ANavigationData* NavData = It.Value().Get();
		if (NavData)
		{
			NavData->UpdateCustomLink(CustomLink);
		}
	}
}

void UNavigationSystemV1::RequestCustomLinkRegistering(INavLinkCustomInterface& CustomLink, UObject* Owner)
{
	SCOPE_CYCLE_COUNTER(STAT_NavOctreeBookkeeping);

	if (Owner != nullptr)
	{
		if (UNavigationObjectRepository* Repository = UWorld::GetSubsystem<UNavigationObjectRepository>(Owner->GetWorld()))
		{
			UE_LOGF(LogNavLink, Log, "%s 0x%p", __FUNCTION__, &CustomLink);
			Repository->RegisterCustomNavLinkObject(CustomLink);
		}
	}
}

void UNavigationSystemV1::RequestCustomLinkUnregistering(INavLinkCustomInterface& CustomLink, UObject* Owner)
{
	SCOPE_CYCLE_COUNTER(STAT_NavOctreeBookkeeping);

	if (Owner != nullptr)
	{
		if (UNavigationObjectRepository* Repository = UWorld::GetSubsystem<UNavigationObjectRepository>(Owner->GetWorld()))
		{
			UE_LOGF(LogNavLink, Log, "%s 0x%p", __FUNCTION__, &CustomLink);
			Repository->UnregisterCustomNavLinkObject(CustomLink);
		}
	}
}

FBox UNavigationSystemV1::ComputeCustomLinkBounds(const INavLinkCustomInterface& CustomLink)
{
	const UObject* CustomLinkOb = CustomLink.GetLinkOwner();
	const UActorComponent* OwnerComp = Cast<UActorComponent>(CustomLinkOb);
	const AActor* OwnerActor = OwnerComp ? OwnerComp->GetOwner() : Cast<AActor>(CustomLinkOb);

	FBox LinkBounds(ForceInitToZero);

	// If the owner is a navdata, the NavLink Custom Interface is shared for multiple link instances.
	// It does not have individual link data so act has if it had no owner actor.
	if (OwnerActor && !OwnerActor->IsA(ANavigationData::StaticClass()))
	{
		ENavLinkDirection::Type DummyDir = ENavLinkDirection::BothWays;
		FVector RelativePtA, RelativePtB;
		CustomLink.GetLinkData(RelativePtA, RelativePtB, DummyDir);

		const FTransform OwnerActorTM = OwnerActor->GetTransform();
		const FVector WorldPtA = OwnerActorTM.TransformPosition(RelativePtA);
		const FVector WorldPtB = OwnerActorTM.TransformPosition(RelativePtB);

		LinkBounds += WorldPtA;
		LinkBounds += WorldPtB;
	}
	return LinkBounds;
}

void UNavigationSystemV1::RequestAreaUnregistering(UClass* NavAreaClass)
{
	for (TObjectIterator<UNavigationSystemV1> NavSysIt; NavSysIt; ++NavSysIt)
	{
		NavSysIt->UnregisterNavAreaClass(NavAreaClass);
	}
}

void UNavigationSystemV1::UnregisterNavAreaClass(UClass* NavAreaClass)
{
	// remove from known areas
	if (NavAreaClasses.Remove(NavAreaClass) > 0)
	{
		// notify navigation data
		// notify existing nav data
		OnNavigationAreaEvent(NavAreaClass, ENavAreaEvent::Unregistered);

		const UWorld* const World = GetWorld();
		if (ensure(World))
		{
			UNavigationSystemBase::OnNavAreaUnregisteredDelegate().Broadcast(*World, NavAreaClass);
		}
	}
}

void UNavigationSystemV1::RequestAreaRegistering(UClass* NavAreaClass)
{
	for (TObjectIterator<UNavigationSystemV1> NavSysIt; NavSysIt; ++NavSysIt)
	{
		NavSysIt->RegisterNavAreaClass(NavAreaClass);
	}
}

void UNavigationSystemV1::RegisterNavAreaClass(UClass* AreaClass)
{
	// can't be null
	if (AreaClass == nullptr)
	{
		return;
	}

	// can't be abstract
	if (AreaClass->HasAnyClassFlags(CLASS_Abstract))
	{
		return;
	}

	// special handling of blueprint based areas
	if (AreaClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
	{
		// can't be skeleton of blueprint class
		if (AreaClass->GetName().Contains(TEXT("SKEL_")))
		{
			return;
		}

		// can't be class from Developers folder (won't be saved properly anyway)
		const UPackage* Package = AreaClass->GetOutermost();
		if (Package && Package->GetName().Contains(TEXT("/Developers/")))
		{
			return;
		}
	}

	if (NavAreaClasses.Contains(AreaClass))
	{
		// Already added
		return;
	}

	UNavArea* AreaClassCDO = GetMutableDefault<UNavArea>(AreaClass);
	check(AreaClassCDO);

	// initialize flags
	AreaClassCDO->InitializeArea();

	// add to know areas
	NavAreaClasses.Add(AreaClass);

	// notify existing nav data
	OnNavigationAreaEvent(AreaClass, ENavAreaEvent::Registered);

#if WITH_EDITOR
	UNavAreaMeta_SwitchByAgent* SwitchByAgentCDO = Cast<UNavAreaMeta_SwitchByAgent>(AreaClassCDO);
	// update area properties
	if (SwitchByAgentCDO)
	{
		SwitchByAgentCDO->UpdateAgentConfig();
	}
#endif

	const UWorld* const World = GetWorld();
	if (ensure(World))
	{
		UNavigationSystemBase::OnNavAreaRegisteredDelegate().Broadcast(*World, AreaClass);
	}
}

void UNavigationSystemV1::OnNavigationAreaEvent(UClass* AreaClass, ENavAreaEvent::Type Event)
{
	// notify existing nav data
	for (auto NavigationData : NavDataSet)
	{
		if (NavigationData != NULL && NavigationData->IsPendingKillPending() == false)
		{
			NavigationData->OnNavAreaEvent(AreaClass, Event);
		}
	}
}

int32 UNavigationSystemV1::GetSupportedAgentIndex(const ANavigationData* NavData) const
{
	if (SupportedAgents.Num() == 1)
	{
		return 0;
	}

	const FNavDataConfig& TestConfig = NavData->GetConfig();
	for (int32 AgentIndex = 0; AgentIndex < SupportedAgents.Num(); AgentIndex++)
	{
		if (SupportedAgents[AgentIndex].IsValid() && SupportedAgents[AgentIndex].IsEquivalent(TestConfig))
		{
			return AgentIndex;
		}
	}
	
	return INDEX_NONE;
}

int32 UNavigationSystemV1::GetSupportedAgentIndex(const FNavAgentProperties& NavAgent) const
{
	if (SupportedAgents.Num() == 1)
	{
		return 0;
	}

	for (int32 AgentIndex = 0; AgentIndex < SupportedAgents.Num(); AgentIndex++)
	{
		if (SupportedAgents[AgentIndex].IsValid() && SupportedAgents[AgentIndex].IsEquivalent(NavAgent))
		{
			return AgentIndex;
		}
	}

	return INDEX_NONE;
}

void UNavigationSystemV1::DescribeFilterFlags(UEnum* FlagsEnum) const
{
#if WITH_EDITOR
	TArray<FString> FlagDesc;
	FString EmptyStr;
	FlagDesc.Init(EmptyStr, 16);

	const int32 NumEnums = FMath::Min(16, FlagsEnum->NumEnums() - 1);	// skip _MAX
	for (int32 FlagIndex = 0; FlagIndex < NumEnums; FlagIndex++)
	{
		FlagDesc[FlagIndex] = FlagsEnum->GetDisplayNameTextByIndex(FlagIndex).ToString();
	}

	DescribeFilterFlags(FlagDesc);
#endif
}

void UNavigationSystemV1::DescribeFilterFlags(const TArray<FString>& FlagsDesc) const
{
#if WITH_EDITOR
	const int32 MaxFlags = 16;
	TArray<FString> UseDesc = FlagsDesc;

	FString EmptyStr;
	while (UseDesc.Num() < MaxFlags)
	{
		UseDesc.Add(EmptyStr);
	}

	// get special value from recast's navmesh
#if WITH_RECAST
	uint16 NavLinkFlag = ARecastNavMesh::GetNavLinkFlag();
	for (int32 FlagIndex = 0; FlagIndex < MaxFlags; FlagIndex++)
	{
		if ((NavLinkFlag >> FlagIndex) & 1)
		{
			UseDesc[FlagIndex] = TEXT("Navigation link");
			break;
		}
	}
#endif

	// setup properties
	FStructProperty* StructProp1 = FindFProperty<FStructProperty>(UNavigationQueryFilter::StaticClass(), TEXT("IncludeFlags"));
	FStructProperty* StructProp2 = FindFProperty<FStructProperty>(UNavigationQueryFilter::StaticClass(), TEXT("ExcludeFlags"));
	check(StructProp1);
	check(StructProp2);

	UStruct* Structs[] = { StructProp1->Struct, StructProp2->Struct };
	const FString CustomNameMeta = "DisplayName";

	for (int32 StructIndex = 0; StructIndex < UE_ARRAY_COUNT(Structs); StructIndex++)
	{
		for (int32 FlagIndex = 0; FlagIndex < MaxFlags; FlagIndex++)
		{
			FString PropName = FString::Printf(TEXT("bNavFlag%d"), FlagIndex);
			FProperty* Prop = FindFProperty<FProperty>(Structs[StructIndex], *PropName);
			check(Prop);

			if (UseDesc[FlagIndex].Len())
			{
				Prop->SetPropertyFlags(CPF_Edit);
				Prop->SetMetaData(*CustomNameMeta, *UseDesc[FlagIndex]);
			}
			else
			{
				Prop->ClearPropertyFlags(CPF_Edit);
			}
		}
	}

#endif
}

void UNavigationSystemV1::ResetCachedFilter(TSubclassOf<UNavigationQueryFilter> FilterClass)
{
	for (int32 NavDataIndex = 0; NavDataIndex < NavDataSet.Num(); NavDataIndex++)
	{
		if (NavDataSet[NavDataIndex])
		{
			NavDataSet[NavDataIndex]->RemoveQueryFilter(FilterClass);
		}
	}
}

bool UNavigationSystemV1::ShouldCreateNavigationSystemInstance(const UWorld* World) const
{
	ensureMsgf(IsTemplate(), TEXT("This method is expected to only be called on Template objects"
								" to determine if an instance of this type should be created."));

	return (World != nullptr)
		&& (ShouldAllowClientSideNavigation() || (World->GetNetMode() != NM_Client));
}

void UNavigationSystemV1::InitializeForWorld(UWorld& World, FNavigationSystemRunMode Mode)
{
	OnWorldInitDone(Mode);
}

UNavigationSystemV1* UNavigationSystemV1::GetCurrent(UWorld* World)
{
	return FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);
}

UNavigationSystemV1* UNavigationSystemV1::GetCurrent(UObject* WorldContextObject)
{
	return FNavigationSystem::GetCurrent<UNavigationSystemV1>(WorldContextObject);
}

ANavigationData* UNavigationSystemV1::GetNavDataWithID(const uint16 NavDataID) const
{
	for (int32 NavDataIndex = 0; NavDataIndex < NavDataSet.Num(); ++NavDataIndex)
	{
		const ANavigationData* NavData = NavDataSet[NavDataIndex];
		if (NavData != nullptr && NavData->GetNavDataUniqueID() == NavDataID)
		{
			return const_cast<ANavigationData*>(NavData);
		}
	}

	return nullptr;
}

FNavigationElementHandle UNavigationSystemV1::GetNavigationElementHandleForUObject(const UObject* Object) const
{
	checkf(Repository, TEXT("%hs is expected to be called after the repository gets cached."), __FUNCTION__);
	return Repository->GetNavigationElementHandleForUObject(Object);
}

TSharedPtr<const FNavigationElement> UNavigationSystemV1::GetNavigationElementForUObject(const UObject* Object) const
{
	checkf(Repository, TEXT("%hs is expected to be called after the repository gets cached."), __FUNCTION__);
	return Repository->GetNavigationElementForUObject(Object);
}

void UNavigationSystemV1::OnNavRelevantObjectRegistered(UObject& Object)
{
	if (IsNavigationSystemStatic())
	{
		return;
	}

	if (const INavRelevantInterface* NavInterface = Cast<INavRelevantInterface>(&Object))
	{
		RegisterNavRelevantObjectStatic(*NavInterface, Object);
	}
}

void UNavigationSystemV1::RegisterComponentToNavOctree(UActorComponent* Comp)
{
	SCOPE_CYCLE_COUNTER(STAT_NavOctreeBookkeeping);

	if ((Comp == nullptr) || IsNavigationSystemStatic())
	{
		return;
	}

	if (UE::Navigation::Private::ShouldComponentWaitForActorToRegister(Comp))
	{
		UE_LOGF(LogNavigation, VeryVerbose
			, "%s: %ls registration skipped (waiting for actor to register)", __FUNCTION__, *GetNameSafe(Comp));
		return;
	}

	if (INavRelevantInterface* NavInterface = Cast<INavRelevantInterface>(Comp))
	{
		const AActor* OwnerActor = Comp->GetOwner();
		if (OwnerActor && OwnerActor->IsComponentRelevantForNavigation(Comp))
		{
			RegisterNavRelevantObjectStatic(*NavInterface, *Comp);
		}
	}
}

bool UNavigationSystemV1::SupportsDynamicChanges(UWorld* World)
{
	if (IsNavigationSystemStatic())
	{
		return false;
	}

	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);
	return NavSys && NavSys->RequiresNavOctree();
}

FNavigationElementHandle UNavigationSystemV1::AddNavigationElement(UWorld* World, FNavigationElement&& Element)
{
	SCOPE_CYCLE_COUNTER(STAT_NavOctreeBookkeeping);

	if (IsNavigationSystemStatic())
	{
		return FNavigationElementHandle::Invalid;
	}

	if (UNavigationObjectRepository* Repository = UWorld::GetSubsystem<UNavigationObjectRepository>(World))
	{
		if (const TSharedPtr<const FNavigationElement> SharedElement = Repository->AddNavigationElement(MoveTemp(Element)))
		{
			return SharedElement->GetHandle();
		}
	}

	return FNavigationElementHandle::Invalid;
}

void UNavigationSystemV1::RemoveNavigationElement(UWorld* World, const FNavigationElementHandle ElementHandle)
{
	SCOPE_CYCLE_COUNTER(STAT_NavOctreeBookkeeping);

	if (IsNavigationSystemStatic())
	{
		return;
	}

	if (UNavigationObjectRepository* Repository = UWorld::GetSubsystem<UNavigationObjectRepository>(World))
	{
		Repository->RemoveNavigationElement(ElementHandle);
	}
}

void UNavigationSystemV1::OnNavRelevantObjectUnregistered(UObject& Object)
{
	UnregisterNavRelevantObjectStatic(Object);
}

void UNavigationSystemV1::UnregisterComponentToNavOctree(UActorComponent* Comp)
{
	// skip IsComponentRelevantForNavigation check, it's only for adding new stuff
	if (Comp)
	{
		if (UE::Navigation::Private::ShouldComponentWaitForActorToRegister(Comp))
		{
			return;
		}

		UnregisterNavRelevantObjectStatic(*Comp);
	}
}

void UNavigationSystemV1::AddDirtyArea(const FBox& NewArea, const ENavigationDirtyFlag Flags, const FName& DebugReason /*= NAME_None*/)
{
	DefaultDirtyAreasController.AddArea(NewArea, Flags, nullptr, nullptr, DebugReason);
}

void UNavigationSystemV1::AddDirtyArea(const FBox& NewArea, const ENavigationDirtyFlag Flags, const TFunction<const TSharedPtr<const FNavigationElement>()>& ElementProviderFunc, const FName& DebugReason /*= NAME_None*/)
{
	DefaultDirtyAreasController.AddArea(NewArea, Flags, ElementProviderFunc, /*DirtyElement*/ nullptr, DebugReason);
}

void UNavigationSystemV1::AddDirtyAreas(const TArray<FBox>& NewAreas, const ENavigationDirtyFlag Flags, const FName& DebugReason /*= NAME_None*/)
{
	if (Flags == ENavigationDirtyFlag::None)
	{
		return;
	}

	for (int32 NewAreaIndex = 0; NewAreaIndex < NewAreas.Num(); NewAreaIndex++)
	{
		AddDirtyArea(NewAreas[NewAreaIndex], Flags, DebugReason);
	}
}

int32 UNavigationSystemV1::GetNumDirtyAreas() const
{
	return DefaultDirtyAreasController.GetNumDirtyAreas();
}

void UNavigationSystemV1::ResetDefaultDirtyAreasController()
{
	DefaultDirtyAreasController.Reset();
}

bool UNavigationSystemV1::HasDirtyAreasQueued() const
{
	return DefaultDirtyAreasController.IsDirty();
}

FSetElementId UNavigationSystemV1::RegisterNavigationElementWithNavOctree(const TSharedRef<const FNavigationElement>& Element, const int32 UpdateFlags)
{
	return FNavigationDataHandler(DefaultOctreeController, DefaultDirtyAreasController).RegisterElementWithNavOctree(Element, UpdateFlags);
}

void UNavigationSystemV1::AddElementToNavOctree(const FNavigationDirtyElement& DirtyElement)
{
	FNavigationDataHandler(DefaultOctreeController, DefaultDirtyAreasController).AddElementToNavOctree(DirtyElement);
}

bool UNavigationSystemV1::GetNavOctreeElementData(const FNavigationElementHandle Element, ENavigationDirtyFlag& OutDirtyFlags, FBox& OutDirtyBounds)
{
	return DefaultOctreeController.GetNavOctreeElementData(Element, OutDirtyFlags, OutDirtyBounds);
}

void UNavigationSystemV1::UnregisterNavigationElementWithOctree(const TSharedRef<const FNavigationElement>& Element, const int32 UpdateFlags)
{
	FNavigationDataHandler(DefaultOctreeController, DefaultDirtyAreasController).UnregisterElementWithNavOctree(Element, UpdateFlags);
}

void UNavigationSystemV1::RemoveFromNavOctree(const FOctreeElementId2& ElementId, const int32 UpdateFlags)
{
	FNavigationDataHandler(DefaultOctreeController, DefaultDirtyAreasController).RemoveFromNavOctree(ElementId, UpdateFlags);
}

void UNavigationSystemV1::ProcessPendingOctreeUpdates()
{
	FNavigationDataHandler NavHandler(DefaultOctreeController, DefaultDirtyAreasController);
	NavHandler.ProcessPendingOctreeUpdates();
}

void UNavigationSystemV1::FlushPendingOperations()
{
	// Make sure all changes are reflected in the octree
	ProcessPendingOctreeUpdates();
	
	PendingNavBoundsUpdates.Reset();

	ResetDefaultDirtyAreasController();
}

void UNavigationSystemV1::DemandLazyDataGathering(FNavigationRelevantData& ElementData)
{
	FNavigationDataHandler(DefaultOctreeController, DefaultDirtyAreasController).DemandLazyDataGathering(ElementData);
}

const FNavigationRelevantData* UNavigationSystemV1::GetDataForElement(const FNavigationElementHandle Element) const
{
	return DefaultOctreeController.GetDataForElement(Element);
}

FNavigationRelevantData* UNavigationSystemV1::GetMutableDataForElement(const FNavigationElementHandle Element)
{
	return DefaultOctreeController.GetMutableDataForElement(Element);
}

void UNavigationSystemV1::RegisterNavRelevantObjectStatic(const INavRelevantInterface& NavRelevantObject, const UObject& Object)
{
	SCOPE_CYCLE_COUNTER(STAT_NavOctreeBookkeeping);

	if (IsNavigationSystemStatic())
	{
		return;
	}

	if (UNavigationObjectRepository* Repository = UWorld::GetSubsystem<UNavigationObjectRepository>(Object.GetWorld()))
	{
		Repository->RegisterNavRelevantObject(NavRelevantObject);
	}
}

void UNavigationSystemV1::RegisterNavRelevantObjectInternal(const INavRelevantInterface& NavRelevantObject, const UObject& Object)
{
	SCOPE_CYCLE_COUNTER(STAT_NavOctreeBookkeeping);

	if (IsNavigationSystemStatic())
	{
		return;
	}

	if (Repository == nullptr)
	{
		return;
	}

	Repository->RegisterNavRelevantObject(NavRelevantObject);
}

void UNavigationSystemV1::UnregisterNavRelevantObjectStatic(const UObject& Object)
{
	SCOPE_CYCLE_COUNTER(STAT_NavOctreeBookkeeping);

	if (IsNavigationSystemStatic())
	{
		return;
	}

	if (UNavigationObjectRepository* Repository = UWorld::GetSubsystem<UNavigationObjectRepository>(Object.GetWorld()))
	{
		Repository->UnregisterNavRelevantObject(&Object);
	}
}

void UNavigationSystemV1::UnregisterNavRelevantObjectInternal(const UObject& Object)
{
	SCOPE_CYCLE_COUNTER(STAT_NavOctreeBookkeeping);

	if (IsNavigationSystemStatic())
	{
		return;
	}

	if (Repository == nullptr)
	{
		return;
	}

	Repository->UnregisterNavRelevantObject(&Object);
}

void UNavigationSystemV1::UpdateNavRelevantObjectInNavOctreeStatic(
	const INavRelevantInterface& InNavRelevantObject,
	const UObject& InObject,
	UNavigationSystemV1* InNavigationSystem,
	TFunctionRef<void(UNavigationSystemV1&, const TSharedRef<const FNavigationElement>&)> InCallback)
{
	SCOPE_CYCLE_COUNTER(STAT_NavOctreeBookkeeping);

	if (IsNavigationSystemStatic())
	{
		return;
	}

	if (!ensureMsgf(InNavRelevantObject.IsNavigationRelevant(), TEXT("%hs: %s is not navigation relevant"), __FUNCTION__, *InObject.GetName()))
	{
		return;
	}

	if (UNavigationSystemV1* NavSys = InNavigationSystem ? InNavigationSystem : FNavigationSystem::GetCurrent<UNavigationSystemV1>(InObject.GetWorld()))
	{
		if (NavSys->Repository)
		{
			if (const TSharedPtr<const FNavigationElement> SharedElement = NavSys->Repository->UpdateNavigationElementForUObject(InNavRelevantObject, InObject))
			{
				InCallback(*NavSys, SharedElement.ToSharedRef());
			}
		}
	}
	else
	{
		// Navigation system not available so use the static registration to be stored in the repository
		// so the navigation system will gather it on initialization.
		UE_LOGF(LogNavigation, VeryVerbose,
			"%s: %ls Registering to the repository (NavigationSystem not available)", __FUNCTION__, *InObject.GetName());

		RegisterNavRelevantObjectStatic(InNavRelevantObject, InObject);
	}
}

void UNavigationSystemV1::UpdateNavRelevantObjectInNavOctree(UObject& Object)
{
	SCOPE_CYCLE_COUNTER(STAT_NavOctreeBookkeeping);

	if (IsNavigationSystemStatic())
	{
		return;
	}

	const INavRelevantInterface* NavRelevantInterface = Cast<INavRelevantInterface>(&Object);
	if (NavRelevantInterface && NavRelevantInterface->IsNavigationRelevant())
	{
		UpdateNavRelevantObjectInNavOctreeStatic(*NavRelevantInterface, Object, /*NavigationSystem*/nullptr,
			[](UNavigationSystemV1& NavSys, const TSharedRef<const FNavigationElement>& SharedElement)
			{
				NavSys.UpdateNavOctreeElement(SharedElement->GetHandle(), SharedElement, FNavigationOctreeController::OctreeUpdate_Default);
			});
	}
}

void UNavigationSystemV1::OnNavigationElementUpdated(UWorld* World, const FNavigationElementHandle ElementHandle, FNavigationElement&& Element)
{
	SCOPE_CYCLE_COUNTER(STAT_NavOctreeBookkeeping);

	if (IsNavigationSystemStatic())
	{
		return;
	}

	if (UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World))
	{
		NavSys->UpdateNavOctreeElement(ElementHandle, MakeShared<FNavigationElement>(MoveTemp(Element)) , FNavigationOctreeController::OctreeUpdate_Default);
	}
}

void UNavigationSystemV1::UpdateActorInNavOctree(AActor& Actor)
{
	UpdateNavRelevantObjectInNavOctree(Actor);
}

void UNavigationSystemV1::UpdateComponentInNavOctree(UActorComponent& Comp)
{
	SCOPE_CYCLE_COUNTER(STAT_NavOctreeBookkeeping);

	if (ShouldUpdateNavOctreeOnComponentChange() == false)
	{
		return;
	}

	// Due to an issue with PostEditChangeProperty and AActor::RerunConstructionScripts()
	// we need to make sure that we are not processing an invalid component.
	// Could be converted to an ensure once UE-252220 is fixed
	if (!IsValid(&Comp))
	{
		return;
	}

	if (UE::Navigation::Private::ShouldComponentWaitForActorToRegister(&Comp))
	{
		return;
	}

	// special case for early out: use cached nav relevancy
	if (Comp.bNavigationRelevant)
	{
		if (const AActor* OwnerActor = Comp.GetOwner())
		{
			const INavRelevantInterface* NavRelevantInterface = Cast<INavRelevantInterface>(&Comp);
			if (ensureMsgf(NavRelevantInterface != nullptr, TEXT("Components reaching this point are expected to implement INavRelevantInterface.")))
			{
				if (OwnerActor->IsComponentRelevantForNavigation(&Comp) && Comp.IsNavigationRelevant())
				{

					UpdateNavRelevantObjectInNavOctreeStatic(*NavRelevantInterface, Comp, /*NavigationSystem*/nullptr,
						[](UNavigationSystemV1& NavSys, const TSharedRef<const FNavigationElement>& SharedElement)
						{
							NavSys.UpdateNavOctreeElement(SharedElement->GetHandle(), SharedElement, FNavigationOctreeController::OctreeUpdate_Default);
						});
				}
				else
				{
					if (UNavigationObjectRepository* Repository = UWorld::GetSubsystem<UNavigationObjectRepository>(Comp.GetWorld()))
					{
						Repository->UnregisterNavRelevantObject(&Comp);
					}
				}
			}
		}
	}
	else if (Comp.CanEverAffectNavigation()
#if WITH_EDITOR
		|| GIsReconstructingBlueprintInstances
		// This condition handles a crappy edge case with component registration in Editor
		// Problem occurs when a component in an instance has 'bCanEverAffectNavigation = false' and AActor::RerunConstructionScripts() is called
		// 1. Current component values are serialized to FActorComponentInstanceData
		// 2. Component gets unregistered then destroyed (nothing to do here since it is not affecting navigation)
		// 3. New component gets created and registered using default values from the template (default is affecting navigation so we register to the octree)
		// 4. FActorComponentInstanceData is applied to the component (changing `bCanEverAffectNavigation` from `true` to `false` directly in memory)
		// 5. Component will re-register itself since it was registered at Step 3
		//    Problem is that we normally don't need to do anything for components never affecting navigation
		//    so we never unregister that component from the octree!
#endif // WITH_EDITOR
		)
	{
		// could have been relevant before and now it isn't. Need to check if there's an octree element ID for it
		if (UNavigationObjectRepository* Repository = UWorld::GetSubsystem<UNavigationObjectRepository>(Comp.GetWorld()))
		{
			Repository->UnregisterNavRelevantObject(&Comp);
		}
	}
}

void UNavigationSystemV1::UpdateActorAndComponentsInNavOctree(AActor& Actor, const bool bUpdateAttachedActors)
{
	SCOPE_CYCLE_COUNTER(STAT_NavOctreeBookkeeping);

	if (IsNavigationSystemStatic())
	{
		return;
	}

	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(Actor.GetWorld());

	// Callback to update an actor with its components
	auto UpdateActorAndComponentFunc = [NavSys](AActor& ActorToUpdate)
		{
			const INavRelevantInterface* ActorNavRelevantInterface = Cast<INavRelevantInterface>(&ActorToUpdate);
			if (ActorNavRelevantInterface && ActorNavRelevantInterface->IsNavigationRelevant())
			{
				UpdateNavRelevantObjectInNavOctreeStatic(*ActorNavRelevantInterface, ActorToUpdate, NavSys,
					[](UNavigationSystemV1& InNavSys, const TSharedRef<const FNavigationElement>& SharedElement)
					{
						InNavSys.UpdateNavOctreeElement(SharedElement->GetHandle(), SharedElement, FNavigationOctreeController::OctreeUpdate_Default);
					});
			}

			for (UActorComponent* Component : ActorToUpdate.GetComponents())
			{
				if (Component != nullptr
					&& Component->CanEverAffectNavigation()
					&& Component->IsRegistered()
					&& ActorToUpdate.IsComponentRelevantForNavigation(Component))
				{
					const INavRelevantInterface* ComponentNavRelevantInterface = Cast<INavRelevantInterface>(Component);
					if (ComponentNavRelevantInterface && ComponentNavRelevantInterface->IsNavigationRelevant())
					{
						UpdateNavRelevantObjectInNavOctreeStatic(*ComponentNavRelevantInterface, *Component, NavSys,
							[](UNavigationSystemV1& InNavSys, const TSharedRef<const FNavigationElement>& InElement)
							{
								InNavSys.UpdateNavOctreeElement(InElement->GetHandle(), InElement, FNavigationOctreeController::OctreeUpdate_Default);
							});
						continue;
					}
				}

				if (NavSys)
				{
					NavSys->UnregisterNavRelevantObjectInternal(*Component);
				}
			}
		};

	if (ShouldUpdateNavOctreeOnComponentChange())
	{
		UpdateActorAndComponentFunc(Actor);
	}
	else
	{
		const INavRelevantInterface* ActorNavRelevantInterface = Cast<INavRelevantInterface>(&Actor);
		if (ActorNavRelevantInterface && ActorNavRelevantInterface->IsNavigationRelevant())
		{
			UpdateNavRelevantObjectInNavOctreeStatic(*ActorNavRelevantInterface, Actor, NavSys,
				[](UNavigationSystemV1& InNavSys, const TSharedRef<const FNavigationElement>& SharedElement)
				{
					InNavSys.UpdateNavOctreeElement(SharedElement->GetHandle(), SharedElement, FNavigationOctreeController::OctreeUpdate_Default);
				});
		}
	}

	if (bUpdateAttachedActors)
	{
		TArray<AActor*> UniqueAttachedActors;
		if (GetAllAttachedActors(Actor, UniqueAttachedActors) > 0)
		{
			for (AActor* AttachedActor : UniqueAttachedActors)
			{
				checkf(AttachedActor, TEXT("GetAllAttachedActors should only return unique, non-null ptrs."));
				UpdateActorAndComponentFunc(*AttachedActor);
			}
		}
	}
}

void UNavigationSystemV1::UpdateNavOctreeAfterMove(USceneComponent* Comp)
{
	AActor* OwnerActor = Comp->GetOwner();
	if (OwnerActor && OwnerActor->GetRootComponent() == Comp)
	{
		UpdateActorAndComponentsInNavOctree(*OwnerActor, true);
	}
}

int32 UNavigationSystemV1::GetAllAttachedActors(const AActor& RootActor, TArray<AActor*>& OutAttachedActors)
{
	OutAttachedActors.Reset();
	RootActor.GetAttachedActors(OutAttachedActors);

	TArray<AActor*> TempAttachedActors;
	for (int32 ActorIndex = 0; ActorIndex < OutAttachedActors.Num(); ++ActorIndex)
	{
		check(OutAttachedActors[ActorIndex]);
		// find all attached actors
		OutAttachedActors[ActorIndex]->GetAttachedActors(TempAttachedActors);

		for (int32 AttachmentIndex = 0; AttachmentIndex < TempAttachedActors.Num(); ++AttachmentIndex)
		{
			// and store the ones we don't know about yet
			OutAttachedActors.AddUnique(TempAttachedActors[AttachmentIndex]);
		}
	}

	return OutAttachedActors.Num();
}

void UNavigationSystemV1::UpdateAttachedActorsInNavOctree(AActor& RootActor)
{
	TArray<AActor*> UniqueAttachedActors;
	if (GetAllAttachedActors(RootActor, UniqueAttachedActors) > 0)
	{
		for (AActor* AttachedActor : UniqueAttachedActors)
		{
			UpdateActorAndComponentsInNavOctree(*AttachedActor, /*bUpdateAttachedActors=*/false);
		}
	}
}

void UNavigationSystemV1::UpdateNavOctreeBounds(AActor* Actor)
{
	for (UActorComponent* Component : Actor->GetComponents())
	{
		INavRelevantInterface* NavElement = Cast<INavRelevantInterface>(Component);
		if (NavElement)
		{
			NavElement->UpdateNavigationBounds();
		}
	}
}

void UNavigationSystemV1::ClearNavOctreeAll(AActor* Actor)
{
	if (Actor)
	{
		OnActorUnregistered(Actor);

		TInlineComponentArray<UActorComponent*> Components;
		Actor->GetComponents(Components);

		for (int32 Idx = 0; Idx < Components.Num(); Idx++)
		{
			OnComponentUnregistered(Components[Idx]);
		}
	}
}

void UNavigationSystemV1::UpdateNavOctreeElement(const FNavigationElementHandle Handle, const TSharedRef<const FNavigationElement>& Element, const int32 UpdateFlags)
{
	if (IsNavigationSystemStatic())
	{
		return;
	}

	FNavigationDataHandler(DefaultOctreeController, DefaultDirtyAreasController).UpdateNavOctreeElement(Handle, Element, UpdateFlags);
}

bool UNavigationSystemV1::UpdateNavOctreeElementBounds(const FNavigationElementHandle Handle, const FBox& NewBounds, const TConstArrayView<FBox> DirtyAreas)
{
	if (IsNavigationSystemStatic())
	{
		return false;
	}

	if (const TSharedPtr<FNavigationElement> NavigationElement = Repository ? Repository->GetMutableNavigationElementForHandle(Handle) : nullptr)
	{
		NavigationElement->SetBounds(NewBounds);
	}

	return FNavigationDataHandler(DefaultOctreeController, DefaultDirtyAreasController).UpdateNavOctreeElementBounds(Handle, NewBounds, DirtyAreas);
}

bool UNavigationSystemV1::ReplaceAreaInOctreeData(const FNavigationElementHandle Handle, const TSubclassOf<UNavArea> OldArea, const TSubclassOf<UNavArea> NewArea, const bool bReplaceChildClasses)
{
	if (IsNavigationSystemStatic())
	{
		return false;
	}

	return FNavigationDataHandler(DefaultOctreeController, DefaultDirtyAreasController).ReplaceAreaInOctreeData(Handle, OldArea, NewArea, bReplaceChildClasses);
}

void UNavigationSystemV1::OnComponentRegistered(UActorComponent* Comp)
{
	RegisterComponentToNavOctree(Comp);
}

void UNavigationSystemV1::OnComponentUnregistered(UActorComponent* Comp)
{
	UnregisterComponentToNavOctree(Comp);
}

void UNavigationSystemV1::RegisterComponent(UActorComponent* Comp)
{
	RegisterComponentToNavOctree(Comp);
}

void UNavigationSystemV1::UnregisterComponent(UActorComponent* Comp)
{
	UnregisterComponentToNavOctree(Comp);
}

void UNavigationSystemV1::OnActorRegistered(AActor* Actor)
{
	if (IsNavigationSystemStatic())
	{
		return;
	}

	if (const INavRelevantInterface* NavInterface = Cast<INavRelevantInterface>(Actor))
	{
		RegisterNavRelevantObjectStatic(*NavInterface, *Actor);
	}

	if (UE::Navigation::Private::bComponentShouldWaitForActorToRegister)
	{
		checkf(Actor && Actor->HasActorRegisteredAllComponents()
			, TEXT("Actor is expected to be valid and all its components registered."));

		// Tell all components they need to update their navigation bounds before getting registered to the navigation octree.
		UpdateNavOctreeBounds(Actor);

		// We can now process all the components registered to the scene.
		// Note that we do so using the delegate since it is possible for derived systems to override them.
		for (UActorComponent* Component : Actor->GetComponents())
		{
			if (Component && Component->IsRegistered())
			{
				FNavigationSystem::OnComponentRegistered(*Component);
			}
		}
	}
}

void UNavigationSystemV1::OnActorUnregistered(AActor* Actor)
{
	if (IsNavigationSystemStatic())
	{
		return;
	}

	if (Actor)
	{
		UnregisterNavRelevantObjectStatic(*Actor);
	}
}

void UNavigationSystemV1::FindElementsInNavOctree(const FBox& QueryBox, const FNavigationOctreeFilter& Filter, TArray<FNavigationOctreeElement>& Elements)
{	
	FNavigationDataHandler(DefaultOctreeController, DefaultDirtyAreasController).FindElementsInNavOctree(QueryBox, Filter, Elements);
}

void UNavigationSystemV1::ReleaseInitialBuildingLock()
{
	RemoveNavigationBuildLock(ENavigationBuildLock::InitialLock);
}

void UNavigationSystemV1::InitializeLevelCollisions()
{
	if (IsNavigationSystemStatic())
	{
		bInitialLevelsAdded = true;
		return;
	}

	UWorld* World = GetWorld();
	if (!bInitialLevelsAdded && FNavigationSystem::GetCurrent<UNavigationSystemV1>(World) == this)
	{
		// Process all visible levels
		const auto& Levels = World->GetLevels();
		for (ULevel* Level : Levels)
		{
			if (Level->bIsVisible)
			{
				AddLevelCollisionToOctree(Level);
			}
		}

		bInitialLevelsAdded = true;
	}
}

#if WITH_EDITOR
void UNavigationSystemV1::UpdateLevelCollision(ULevel* InLevel)
{
	if (InLevel != nullptr)
	{
		UWorld* World = GetWorld();
		OnLevelRemovedFromWorld(InLevel, World);
		OnLevelAddedToWorld(InLevel, World);
	}
}
#endif

void UNavigationSystemV1::OnNavigationBoundsUpdated(ANavMeshBoundsVolume* NavVolume)
{
	if (NavVolume == nullptr || IsNavigationSystemStatic())
	{
		return;
	}

	FNavigationBoundsUpdateRequest UpdateRequest;
	UpdateRequest.NavBounds.UniqueID = NavVolume->GetUniqueID();
	UpdateRequest.NavBounds.AreaBox = NavVolume->GetComponentsBoundingBox(true);
	UpdateRequest.NavBounds.Level = NavVolume->GetLevel();
	UpdateRequest.NavBounds.SupportedAgents = NavVolume->SupportedAgents;
	
	if (UpdateRequest.NavBounds.AreaBox.IsValid)
	{
		UpdateRequest.UpdateRequest = FNavigationBoundsUpdateRequest::Updated;
	}
	else
	{
		// Make a removal request if the bounds are invalid.
		UpdateRequest.UpdateRequest = FNavigationBoundsUpdateRequest::Removed;
	}

	CheckToLimitNavigationBoundsToLoadedRegions(UpdateRequest.NavBounds);
	AddNavigationBoundsUpdateRequest(UpdateRequest);
}

void UNavigationSystemV1::OnNavigationBoundsAdded(ANavMeshBoundsVolume* NavVolume)
{
	if (NavVolume == nullptr || IsNavigationSystemStatic())
	{
		return;
	}

	FNavigationBoundsUpdateRequest UpdateRequest;
	UpdateRequest.NavBounds.UniqueID = NavVolume->GetUniqueID();
	UpdateRequest.NavBounds.AreaBox = NavVolume->GetComponentsBoundingBox(true);
	UpdateRequest.NavBounds.Level = NavVolume->GetLevel();
	UpdateRequest.NavBounds.SupportedAgents = NavVolume->SupportedAgents;

	UpdateRequest.UpdateRequest = FNavigationBoundsUpdateRequest::Added;

	CheckToLimitNavigationBoundsToLoadedRegions(UpdateRequest.NavBounds);
	AddNavigationBoundsUpdateRequest(UpdateRequest);
}

void UNavigationSystemV1::OnNavigationBoundsRemoved(ANavMeshBoundsVolume* NavVolume)
{
	if (NavVolume == nullptr || IsNavigationSystemStatic())
	{
		return;
	}
	
	FNavigationBoundsUpdateRequest UpdateRequest;
	UpdateRequest.NavBounds.UniqueID = NavVolume->GetUniqueID();
	UpdateRequest.NavBounds.AreaBox = NavVolume->GetComponentsBoundingBox(true);
	UpdateRequest.NavBounds.Level = NavVolume->GetLevel();
	UpdateRequest.NavBounds.SupportedAgents = NavVolume->SupportedAgents;

	UpdateRequest.UpdateRequest = FNavigationBoundsUpdateRequest::Removed;

	CheckToLimitNavigationBoundsToLoadedRegions(UpdateRequest.NavBounds);
	AddNavigationBoundsUpdateRequest(UpdateRequest);
}

void UNavigationSystemV1::CheckToLimitNavigationBoundsToLoadedRegions(FNavigationBounds& OutBounds) const
{
#if WITH_EDITOR && WITH_RECAST
	// Find out if at least one of the nav meshes is world partitioned
	bool bAnyWorldPartitionedNavMeshes = false;
	for (const TObjectPtr<ANavigationData>& NavData : NavDataSet)
	{
		const ARecastNavMesh* RecastNavMesh = Cast<ARecastNavMesh>(NavData);
		if (RecastNavMesh && RecastNavMesh->bIsWorldPartitioned)
		{
			bAnyWorldPartitionedNavMeshes = true;
			break;
		}
	}

	// Don't limit nav bounds if none of the nav meshes are world partitioned
	if (!bAnyWorldPartitionedNavMeshes)
	{
		return;
	}

	// Don't limit nav bounds at runtime
	const UWorld* World = MainNavData->GetWorld();
	if (!World || World->WorldType != EWorldType::Editor)
	{
		return;
	}
	
	// Don't limit nav bounds if not in a world partitioned world
	const UWorldPartition* WorldPartition = World->GetWorldPartition();
	if (!WorldPartition)
	{
		return;
	}
	
	// Get all loaded regions from the world partition
	const TArray<FBox> LoadedWorldPartitionRegions = WorldPartition->GetUserLoadedEditorRegions();

	// Store all overlaps between loaded world partition regions and UpdateRequest's nav bounds
	TArray<FBox> OverlapRegions;
	for (const FBox& LoadedWorldPartitionRegion : LoadedWorldPartitionRegions)
	{
		if (OutBounds.AreaBox.Intersect(LoadedWorldPartitionRegion))
		{
			OverlapRegions.Add(OutBounds.AreaBox.Overlap(LoadedWorldPartitionRegion));
		}
	}

	// Merge all regions which overlap UpdateRequest's nav bounds
	if (!OverlapRegions.IsEmpty())
	{
		OutBounds.AreaBox = FBox(ForceInitToZero);
		for (const FBox& OverlapRegion : OverlapRegions)
		{
			OutBounds.AreaBox += OverlapRegion;
		}
	}
#endif // WITH_EDITOR && WITH_RECAST
}

void UNavigationSystemV1::AddNavigationBoundsUpdateRequest(const FNavigationBoundsUpdateRequest& UpdateRequest)
{
	UE_LOGF(LogNavigation, Verbose, "%s Update request: %i", __FUNCTION__, UpdateRequest.UpdateRequest);
	
	const int32 ExistingIdx = PendingNavBoundsUpdates.IndexOfByPredicate([&](const FNavigationBoundsUpdateRequest& Element) {
		return UpdateRequest.NavBounds.UniqueID == Element.NavBounds.UniqueID;
	});

	if (ExistingIdx != INDEX_NONE)
	{
		// catch the case where the bounds was removed and immediately re-added with the same bounds as before
		// in that case, we can cancel any update at all
		bool bCanCancelUpdate = false;
		if (PendingNavBoundsUpdates[ExistingIdx].UpdateRequest == FNavigationBoundsUpdateRequest::Removed && UpdateRequest.UpdateRequest == FNavigationBoundsUpdateRequest::Added)
		{
			for (TSet<FNavigationBounds>::TConstIterator It(RegisteredNavBounds); It; ++It)
			{
				if (*It == UpdateRequest.NavBounds)
				{
					bCanCancelUpdate = true;
					break;
				}
			}
		}
		if (bCanCancelUpdate)
		{
			PendingNavBoundsUpdates.RemoveAt(ExistingIdx);
		}
		else
		{
			// Overwrite any previous updates
			PendingNavBoundsUpdates[ExistingIdx] = UpdateRequest;
		}
	}
	else
	{
		PendingNavBoundsUpdates.Add(UpdateRequest);
	}
}

void UNavigationSystemV1::PerformNavigationBoundsUpdate(const TArray<FNavigationBoundsUpdateRequest>& UpdateRequests)
{
	// NOTE: we used to create missing nav data first, before updating nav bounds, 
	// but some nav data classes (like RecastNavMesh) may depend on the nav bounds
	// being already known at the moment of creation or serialization, so it makes more 
	// sense to update bounds first 

	// Create list of areas that needs to be updated
	TArray<FBox> UpdatedAreas;
	for (const FNavigationBoundsUpdateRequest& Request : UpdateRequests)
	{
		FSetElementId ExistingElementId = RegisteredNavBounds.FindId(Request.NavBounds);

		switch (Request.UpdateRequest)
		{
		case FNavigationBoundsUpdateRequest::Removed:
			{
				if (ExistingElementId.IsValidId())
				{
					UE_LOGF(LogNavigation, Log, "%s Remove bounds %ls", __FUNCTION__, *RegisteredNavBounds[ExistingElementId].AreaBox.ToCompactString());
					UpdatedAreas.Add(RegisteredNavBounds[ExistingElementId].AreaBox);
					RegisteredNavBounds.Remove(ExistingElementId);
				}
			}
			break;

		case FNavigationBoundsUpdateRequest::Added:
		case FNavigationBoundsUpdateRequest::Updated:
			{
				if (ExistingElementId.IsValidId())
				{
					UE_LOGF(LogNavigation, Log, "%s Update bounds %ls", __FUNCTION__, *RegisteredNavBounds[ExistingElementId].AreaBox.ToCompactString());
					const FBox ExistingBox = RegisteredNavBounds[ExistingElementId].AreaBox;
					const bool bSameArea = (Request.NavBounds.AreaBox == ExistingBox);
					if (!bSameArea)
					{
						UpdatedAreas.Add(ExistingBox);
					}

					// always assign new bounds data, it may have different properties (like supported agents)
					RegisteredNavBounds[ExistingElementId] = Request.NavBounds;
				}
				else
				{
					UE_LOGF(LogNavigation, Log, "%s Add bounds %ls", __FUNCTION__, *Request.NavBounds.AreaBox.ToCompactString());
					AddNavigationBounds(Request.NavBounds);
				}

				UpdatedAreas.Add(Request.NavBounds.AreaBox);
			}

			break;
		}
	}

	if (UpdatedAreas.Num())
	{
		for (ANavigationData* NavData : NavDataSet)
		{
			if (NavData)
			{
				NavData->OnNavigationBoundsChanged();	
			}
		}
	}

	if (!IsNavigationBuildingLocked())
	{
		// Propagate to generators areas that needs to be updated
		AddDirtyAreas(UpdatedAreas, ENavigationDirtyFlag::All | ENavigationDirtyFlag::NavigationBounds, "Navigation bounds update");
	}

	// I'm not sure why we even do the following as part of this function
	// @TODO investigate if we can extract it into a separate function and
	// call it directly
	if (NavDataSet.Num() == 0)
	{
		//TODO: will hitch when user places first navigation volume in the world 

		if (NavDataRegistrationQueue.Num() > 0)
		{
			ProcessRegistrationCandidates();
		}

		if (NavDataSet.Num() == 0 && bAutoCreateNavigationData == true)
		{
			SpawnMissingNavigationData();
			ProcessRegistrationCandidates();
		}

		ConditionalPopulateNavOctree();
	}
}

void UNavigationSystemV1::AddNavigationBounds(const FNavigationBounds& NewBounds)
{
	RegisteredNavBounds.Add(NewBounds);
}

void UNavigationSystemV1::GatherNavigationBounds()
{
	// Gather all available navigation bounds
	RegisteredNavBounds.Empty();
	for (TActorIterator<ANavMeshBoundsVolume> It(GetWorld()); It; ++It)
	{
		// Iterator can access actors with unregistered components which can result in invalid bounding boxes.
		// In this case we skip these actors and wait calls to OnNavigationBoundsAdded.
		const ANavMeshBoundsVolume* V = (*It);
		if (IsValid(V) && V->HasActorRegisteredAllComponents())
		{
			FNavigationBounds NavBounds;
			NavBounds.UniqueID = V->GetUniqueID();
			NavBounds.AreaBox = V->GetComponentsBoundingBox(true);
			NavBounds.Level = V->GetLevel();
			NavBounds.SupportedAgents = V->SupportedAgents;

			AddNavigationBounds(NavBounds);
		}
	}
}

void UNavigationSystemV1::GetInvokerSeedLocations(const UWorld& InWorld, TArray<FVector, TInlineAllocator<32>>& OutSeedLocations)
{
	for (FConstPlayerControllerIterator PlayerIt = InWorld.GetPlayerControllerIterator(); PlayerIt; ++PlayerIt)
	{
		const APlayerController* PlayerController = PlayerIt->Get();
		if (PlayerController)
		{
			if (PlayerController->GetPawn())
			{
				OutSeedLocations.Add(PlayerController->GetPawn()->GetActorLocation());
			}
			else if (PlayerController->PlayerCameraManager)
			{
				OutSeedLocations.Add(PlayerController->PlayerCameraManager->GetCameraLocation());
			}
		}
	}
}

void UNavigationSystemV1::Build()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UNavigationSystemV1::Build);
	
	UE_LOGF(LogNavigationDataBuild, Display, "UNavigationSystemV1::Build started...");

	UWorld* World = GetWorld();
	if (!World)
	{
		UE_LOGF(LogNavigation, Error, "Unable to build navigation due to missing World pointer");
		return;
	}

	FNavigationSystem::DiscardNavigationDataChunks(*World);

	const bool bHasWork = IsThereAnywhereToBuildNavigation();
	const bool bLockedIgnoreEditor = (NavBuildingLockFlags & ~ENavigationBuildLock::NoUpdateInEditor) != 0;
	if (!bHasWork || bLockedIgnoreEditor)
	{
		UE_LOGF(LogNavigation, Warning, "%s Navigation NOT building because navigation build is locked (flags: 0x%x).", __FUNCTION__, NavBuildingLockFlags);
		return;
	}

	const double BuildStartTime = FPlatformTime::Seconds();

	if (bAutoCreateNavigationData == true
#if WITH_EDITOR
		|| FNavigationSystem::IsEditorRunMode(OperationMode)
#endif // WITH_EDITOR
		)
	{
		SpawnMissingNavigationData();
	}

	// make sure freshly created navigation instances are registered before we try to build them
	ProcessRegistrationCandidates();
	
	// update invokers in case we're not updating navmesh automatically, in which case
	// navigation generators wouldn't have up-to-date info.
	if (bGenerateNavigationOnlyAroundNavigationInvokers)
	{
		UpdateInvokers();
	}

	if (BuildBounds.IsValid)
	{
		// Prepare to build tiles overlapping the bounds
		DirtyTilesInBuildBounds();
	}

	// and now iterate through all registered and just start building them
	RebuildAll();

	// Block until build is finished
	for (ANavigationData* NavData : NavDataSet)
	{
		if (NavData)
		{
			NavData->EnsureBuildCompletion();
		}
	}

#if !UE_BUILD_SHIPPING
	// no longer report that navmesh needs to be rebuild
	DefaultDirtyAreasController.bDirtyAreasReportedWhileAccumulationLocked = false;
#endif // !UE_BUILD_SHIPPING

	UE_LOGF(LogNavigationDataBuild, Display, "UNavigationSystemV1::Build total execution time: %.2fs", float(FPlatformTime::Seconds() - BuildStartTime));
	UE_LOGF(LogNavigation, Display, "UNavigationSystemV1::Build total execution time: %.5fs", float(FPlatformTime::Seconds() - BuildStartTime));
}

void UNavigationSystemV1::CancelBuild()
{
	for (ANavigationData* NavData : NavDataSet)
	{
		if (NavData)
		{
			if (NavData->GetGenerator())
			{
				NavData->GetGenerator()->CancelBuild();
			}
		}
	}
}

void UNavigationSystemV1::SpawnMissingNavigationData()
{
	const int32 AllSupportedAgentsCount = SupportedAgents.Num();
	check(AllSupportedAgentsCount >= 0);
	int32 ValidSupportedAgentsCount = 0;
	for (int32 AgentIndex = 0; AgentIndex < AllSupportedAgentsCount; ++AgentIndex)
	{
		if (SupportedAgentsMask.Contains(AgentIndex))
		{
			++ValidSupportedAgentsCount;
		}
	}
	
	// Bit array might be a bit of an overkill here, but this function will be called very rarely
	TBitArray<> AlreadyInstantiated;
	uint8 NumberFound = 0;

	// 1. check whether any of required navigation data has already been instantiated
	NumberFound = FillInstantiatedDataMask(AlreadyInstantiated);

	// 2. for any not already instantiated navigation data call creator functions
	if (NumberFound < ValidSupportedAgentsCount)
	{
		SpawnMissingNavigationDataInLevel(AlreadyInstantiated);
	}

	if (MainNavData == nullptr || MainNavData->IsPendingKillPending())
	{
		MainNavData = GetDefaultNavDataInstance(FNavigationSystem::DontCreate);
	}
}

uint8 UNavigationSystemV1::FillInstantiatedDataMask(TBitArray<>& OutInstantiatedMask, ULevel* InLevel /*= nullptr*/)
{
	int32 AllSupportedAgentsCount = SupportedAgents.Num();
	OutInstantiatedMask.Init(false, AllSupportedAgentsCount);
	uint8 NumberFound = 0;

	auto SetMatchingAgentIndexFunc = [&](ANavigationData* Nav) {
		for (int32 AgentIndex = 0; AgentIndex < AllSupportedAgentsCount; ++AgentIndex)
		{
			if (OutInstantiatedMask[AgentIndex] == false
				&& Nav->GetClass() == SupportedAgents[AgentIndex].GetNavDataClass<ANavigationData>()
				&& Nav->DoesSupportAgent(SupportedAgents[AgentIndex]) == true)
			{
				OutInstantiatedMask[AgentIndex] = true;
				++NumberFound;
				break;
			}
		}
	};

	if (InLevel != nullptr)
	{
		for (AActor* Actor: InLevel->Actors)
		{
			if (ANavigationData* NavData = Cast<ANavigationData>(Actor))
			{
				SetMatchingAgentIndexFunc(NavData);
				if (NumberFound >= AllSupportedAgentsCount)
				{
					break;
				}
			}
		}
	} 
	else
	{
		UWorld* NavWorld = GetWorld();	
		for (TActorIterator<ANavigationData> It(NavWorld); It && NumberFound < AllSupportedAgentsCount; ++It)
		{
			ANavigationData* Nav = (*It);
			if (IsValid(Nav)
				// mz@todo the 'is level in' condition is temporary
				&& (Nav->GetTypedOuter<UWorld>() == NavWorld || NavWorld->GetLevels().Contains(Nav->GetLevel())))
			{
				// find out which one it is
				SetMatchingAgentIndexFunc(Nav);
			}
		}
	}

	return NumberFound;
}

void UNavigationSystemV1::SpawnMissingNavigationDataInLevel(const TBitArray<>& InInstantiatedMask, ULevel* InLevel/*=nullptr*/)
{
	UWorld* NavWorld = GetWorld();
	
	UE_LOGF(LogNavigation, Verbose, "%s Level: %ls, OuterWorld: %ls", __FUNCTION__, *GetNameSafe(InLevel), *GetNameSafe(NavWorld));

	ensure(SupportedAgents.Num() == InInstantiatedMask.Num());
	int32 AllSupportedAgentsCount = InInstantiatedMask.Num();

	for (int32 AgentIndex = 0; AgentIndex < AllSupportedAgentsCount; ++AgentIndex)
	{
		const FNavDataConfig& NavConfig = SupportedAgents[AgentIndex];
		
		if (InInstantiatedMask[AgentIndex] == false
			&& SupportedAgentsMask.Contains(AgentIndex)
			&& NavConfig.GetNavDataClass<ANavigationData>() != nullptr)
		{
			UE_LOGF(LogNavigation, Verbose, "%s   AgentIndex: %i is supported. NavConfig name: %ls.", __FUNCTION__, AgentIndex, *NavConfig.Name.ToString());
			
			const ANavigationData* NavDataCDO = NavConfig.GetNavDataClass<ANavigationData>()->GetDefaultObject<ANavigationData>();
			if (NavDataCDO == nullptr || !NavDataCDO->CanSpawnOnRebuild())
			{
				continue;
			}

			if (NavWorld->WorldType != EWorldType::Editor && NavDataCDO->GetRuntimeGenerationMode() == ERuntimeGenerationType::Static)
			{
				// if we're not in the editor, and specified navigation class is configured 
				// to be static, and there're no navigation bounds present, then we don't want to create an instance	
				if (!IsThereAnywhereToBuildNavigation())
				{
					UE_LOGF(LogNavigation, Log, "Not spawning navigation data for %ls since indicated NavigationData type is not configured for dynamic generation",
						*NavConfig.Name.ToString());
					continue;
				}
			}

			// Check if it's already wating for registration
			const bool bIsWaitingForRegistration = NavDataRegistrationQueue.ContainsByPredicate([&NavConfig](const TObjectPtr<ANavigationData>& NavData)
			{
				return NavData->GetConfig().Name == NavConfig.Name;
			});
			
			if (bIsWaitingForRegistration)
			{
				UE_LOGF(LogNavigation, Verbose, "%s   Skipping %ls already waiting for registration.", __FUNCTION__, *NavConfig.Name.ToString());
				continue;
			}

			// Warn that we're about to spawn and build a navmesh that wasn't saved with this level.
			// This triggers an unexpected runtime rebuild which has a performance cost.
			// The level author should rebuild navigation in the editor and resave the level to prevent this.
			if (NavWorld->IsGameWorld() && NavDataCDO->GetRuntimeGenerationMode() == ERuntimeGenerationType::Static)
			{
				UE_LOGF(LogNavigation, Warning, "SpawnMissingNavigationData: No saved navigation data found for '%ls' in level '%ls'. "
					"Spawning and rebuilding at runtime to recover - this has a performance cost. "
					"Rebuild navigation and resave the level to fix this.",
					*NavConfig.Name.ToString(), *GetNameSafe(InLevel ? InLevel : NavWorld->PersistentLevel.Get()));
			}
			ANavigationData* Instance = CreateNavigationDataInstanceInLevel(NavConfig, InLevel);
			if (Instance)
			{
				// Only flag Static navmesh for an initial rebuild - Dynamic already has its own
				// runtime pipeline, and DynamicModifiersOnly must not trigger geometry storage.
				if (NavWorld->IsGameWorld() && Instance->GetRuntimeGenerationMode() == ERuntimeGenerationType::Static)
				{
					Instance->MarkRequiresInitialRebuild();
				}
				RequestRegistrationDeferred(*Instance);
			}
			else
			{
				UE_LOGF(LogNavigation, Warning, "%s   Was not able to create navigation data for SupportedAgent[%d]: %ls", __FUNCTION__, AgentIndex, *NavConfig.Name.ToString());
			}
		}
	}
}

ANavigationData* UNavigationSystemV1::CreateNavigationDataInstanceInLevel(const FNavDataConfig& NavConfig, ULevel* SpawnLevel)
{
	UWorld* World = GetWorld();
	check(World);

	const int32 NavSupportedAgents = GetSupportedAgentIndex(NavConfig);

	// not creating new NavData instance if the agent it's representing is not supported
	// with the exception of AbstractNavData
	if (NavSupportedAgents == INDEX_NONE
		&& NavConfig.GetNavDataClass<AAbstractNavData>() == nullptr)
	{
		UE_LOGF(LogNavigation, Warning, "Unable to create NavigationData instance for config \'%ls\' as this agent is not supported by current NavigationSystem instance"
			, *NavConfig.GetDescription());
		return nullptr;
	}

	FActorSpawnParameters SpawnInfo;
	SpawnInfo.OverrideLevel = SpawnLevel;
	if (bSpawnNavDataInNavBoundsLevel && SpawnLevel == nullptr && RegisteredNavBounds.Num() > 0)
	{
		// pick the first valid level that supports these agents
		for (const FNavigationBounds& Bounds : RegisteredNavBounds)
		{
			if (Bounds.SupportedAgents.Contains(NavSupportedAgents) && Bounds.Level.IsValid())
			{
				SpawnInfo.OverrideLevel = Bounds.Level.Get();
				break;
			}
		}
	}
	if (SpawnInfo.OverrideLevel == nullptr)
	{
		SpawnInfo.OverrideLevel = World->PersistentLevel;
	}

	ANavigationData* Instance = World->SpawnActor<ANavigationData>(*NavConfig.GetNavDataClass<ANavigationData>(), SpawnInfo);

	if (Instance != nullptr)
	{
		Instance->SetConfig(NavConfig);
		if (NavConfig.Name != NAME_None)
		{
			FString StrName = FString::Printf(TEXT("%s-%s"), *(Instance->GetFName().GetPlainNameString()), *(NavConfig.Name.ToString()));
			// temporary solution to make sure we don't try to change name while there's already
			// an object with this name
			UObject* ExistingObject = StaticFindObject(/*Class=*/ nullptr, Instance->GetOuter(), *StrName, EFindObjectFlags::ExactClass);
			while (ExistingObject != nullptr)
			{
				ANavigationData* ExistingNavigationData = Cast<ANavigationData>(ExistingObject);
				if (ExistingNavigationData)
				{
					UnregisterNavData(ExistingNavigationData);
				}

				// Reset the existing object's name
				ExistingObject->Rename(nullptr, nullptr, REN_DontCreateRedirectors | REN_ForceGlobalUnique | REN_DoNotDirty | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
				// see if there's another one, it does happen when undo/redoing 
				// nav instance deletion in the editor
				ExistingObject = StaticFindObject(/*Class=*/ nullptr, Instance->GetOuter(), *StrName, EFindObjectFlags::ExactClass);
			}

			// Set descriptive name
			Instance->Rename(*StrName, nullptr, REN_DoNotDirty | REN_AllowPackageLinkerMismatch);
#if WITH_EDITOR
			if (World->WorldType == EWorldType::Editor)
			{
				FString ActorLabel = StrName;
				if (Instance->IsPackageExternal())
				{
					// When using external package, don't rely on actor's name to generate a label as it contains a unique actor identifier which obfuscates the label
					ActorLabel = FString::Printf(TEXT("%s-%s"), *(Instance->GetClass()->GetFName().GetPlainNameString()), *(NavConfig.Name.ToString()));
				}
				
				constexpr bool bMarkDirty = false;
				Instance->SetActorLabel(ActorLabel, bMarkDirty);
			}
#endif // WITH_EDITOR
		}
	}

	return Instance;
}

void UNavigationSystemV1::OnPIEStart()
{
	bIsPIEActive = true;
	// no updates for editor world while PIE is active
	const UWorld* MyWorld = GetWorld();
	if (MyWorld && !MyWorld->IsGameWorld())
	{
		bAsyncBuildPaused = true;
		AddNavigationBuildLock(ENavigationBuildLock::NoUpdateInPIE);
	}
}

void UNavigationSystemV1::OnPIEEnd()
{
	bIsPIEActive = false;
	const UWorld* MyWorld = GetWorld();
	if (MyWorld && !MyWorld->IsGameWorld())
	{
		bAsyncBuildPaused = false;

		// There's no need to request navigation rebuilding when PIE ended.
		RemoveNavigationBuildLock(ENavigationBuildLock::NoUpdateInPIE, ELockRemovalRebuildAction::NoRebuild);
	}
}

void UNavigationSystemV1::AddNavigationBuildLock(uint8 Flags)
{
	const bool bWasLocked = IsNavigationBuildingLocked();

	NavBuildingLockFlags |= Flags;

	const bool bIsLocked = IsNavigationBuildingLocked();
	UE_LOGF(LogNavigation, Verbose, "UNavigationSystemV1::AddNavigationBuildLock WasLocked=%ls IsLocked=%ls (flags: 0x%x)",
		*LexToString(bWasLocked), *LexToString(bIsLocked), NavBuildingLockFlags);
	
	if (!bWasLocked && bIsLocked)
	{
		DefaultDirtyAreasController.OnNavigationBuildLocked();
	}
}

void UNavigationSystemV1::RemoveNavigationBuildLock(uint8 Flags, const ELockRemovalRebuildAction RebuildAction /*= ELockRemovalRebuildAction::Rebuild*/)
{
	const bool bWasLocked = IsNavigationBuildingLocked();

	NavBuildingLockFlags &= ~Flags;

	const bool bIsLocked = IsNavigationBuildingLocked();
	UE_LOGF(LogNavigation, Verbose, "UNavigationSystemV1::RemoveNavigationBuildLock WasLocked=%ls IsLocked=%ls (flags: 0x%x)",
		*LexToString(bWasLocked), *LexToString(bIsLocked), NavBuildingLockFlags);
	if (bWasLocked && !bIsLocked)
	{
		DefaultDirtyAreasController.OnNavigationBuildUnlocked();

		const bool bRebuild = 
			(RebuildAction == ELockRemovalRebuildAction::RebuildIfNotInEditor && !FNavigationSystem::IsEditorRunMode(OperationMode)) || 
			(RebuildAction == ELockRemovalRebuildAction::Rebuild);
		
		if (bRebuild)
		{
			RebuildAll();
		}
	}
}

void UNavigationSystemV1::SetNavigationOctreeLock(bool bLock)
{
	UE_LOGF(LogNavigation, Verbose, "UNavigationSystemV1::SetNavigationOctreeLock IsLocked=%ls", *LexToString(bLock));

	DefaultOctreeController.SetNavigationOctreeLock(bLock);
}


void UNavigationSystemV1::RebuildAll(bool bIsLoadTime)
{
	UE_LOGF(LogNavigation, Verbose, "UNavigationSystemV1::RebuildAll");

	const bool bIsInGame = GetWorld()->IsGameWorld();
	
	GatherNavigationBounds();

	for (int32 NavDataIndex = 0; NavDataIndex < NavDataSet.Num(); ++NavDataIndex)
	{
		ANavigationData* NavData = NavDataSet[NavDataIndex];
				
		if (NavData)
		{
			const bool bShouldRebuildAll = (!bIsLoadTime || NavData->NeedsRebuildOnLoad()) && (!bIsInGame || NavData->SupportsRuntimeGeneration()) && (BuildBounds.IsValid == 0);
			if (bShouldRebuildAll)
			{
				UE_LOGF(LogNavigationDataBuild, Display, "   RebuildAll building NavData:  %ls.", *NavData->GetConfig().GetDescription());
				UE_LOGF(LogNavigationDataBuild, Verbose, "   RebuildAll bIsLoadTime=%ls, NavData->NeedsRebuildOnLoad()=%ls, bIsInGame=%ls, NavData->SupportsRuntimeGeneration()=%ls, BuildBounds.IsValid=%ls",
					*LexToString(bIsLoadTime), *LexToString(NavData->NeedsRebuildOnLoad()), *LexToString(bIsInGame), *LexToString(NavData->SupportsRuntimeGeneration()), *LexToString(BuildBounds.IsValid));

#if	WITH_EDITOR
				NavData->SetIsBuildingOnLoad(bIsLoadTime);
#endif
				NavData->RebuildAll();
			}
			else
			{
				UE_LOGF(LogNavigationDataBuild, Verbose, "   RebuildAll skipped.  bIsLoadTime=%ls, NavData->NeedsRebuildOnLoad()=%ls, bIsInGame=%ls, NavData->SupportsRuntimeGeneration()=%ls, BuildBounds.IsValid=%ls",
					*LexToString(bIsLoadTime), *LexToString(NavData->NeedsRebuildOnLoad()), *LexToString(bIsInGame), *LexToString(NavData->SupportsRuntimeGeneration()), *LexToString(BuildBounds.IsValid));
			}
		}
	}
}

void UNavigationSystemV1::RebuildDirtyAreas(float DeltaSeconds)
{
	SCOPE_CYCLE_COUNTER(STAT_Navigation_TickMarkDirty);
	UWorld* World = GetWorld();
	const bool bForceRebuilding = (World != nullptr) && (World->IsGameWorld() == false);
	DefaultDirtyAreasController.Tick(DeltaSeconds, NavDataSet, bForceRebuilding);
}

bool UNavigationSystemV1::IsNavigationBuildInProgress()
{
	bool bRet = false;

	if (NavDataSet.Num() == 0)
	{
		// @todo this is wrong! Should not need to create a navigation data instance in a "getter" like function
		// update nav data. If none found this is the place to create one
		GetDefaultNavDataInstance(FNavigationSystem::DontCreate);
	}
	
	for (int32 NavDataIndex = 0; NavDataIndex < NavDataSet.Num(); ++NavDataIndex)
	{
		ANavigationData* NavData = NavDataSet[NavDataIndex];
		if (NavData != nullptr && NavData->GetGenerator() != nullptr && NavData->GetGenerator()->IsBuildInProgressCheckDirty() == true)
		{
			bRet = true;
			break;
		}
	}

	return bRet;
}

void UNavigationSystemV1::OnNavigationGenerationFinished(ANavigationData& NavData)
{
	OnNavigationGenerationFinishedDelegate.Broadcast(&NavData);

#if WITH_EDITOR
	if (GetWorld()->IsGameWorld() == false)
	{
		if (DirtyNavDataPackageOnNavigationGenerationFinished())
		{
			NavData.MarkPackageDirty();
		}
		
		UE_LOGF(LogNavigationDataBuild, Verbose, "Navigation data generation finished for %ls (%ls).", *NavData.GetActorLabel(), *NavData.GetFullName());
	}

	// Reset bIsBuildingOnLoad
	NavData.SetIsBuildingOnLoad(false);
#endif //WITH_EDITOR
}

int32 UNavigationSystemV1::GetNumRemainingBuildTasks() const
{
	int32 NumTasks = 0;
	
	for (ANavigationData* NavData : NavDataSet)
	{
		if (NavData && NavData->GetGenerator())
		{
			NumTasks+= NavData->GetGenerator()->GetNumRemaningBuildTasks();
		}
	}
	
	return NumTasks;
}

int32 UNavigationSystemV1::GetNumRunningBuildTasks() const 
{
	int32 NumTasks = 0;
	
	for (ANavigationData* NavData : NavDataSet)
	{
		if (NavData && NavData->GetGenerator())
		{
			NumTasks+= NavData->GetGenerator()->GetNumRunningBuildTasks();
		}
	}
	
	return NumTasks;
}

void UNavigationSystemV1::OnLevelAddedToWorld(ULevel* InLevel, UWorld* InWorld)
{
	if ((InWorld != GetWorld()) || (InLevel == nullptr))
	{
		return;
	}
	
	UE_LOGF(LogNavigation, Verbose, "%s Level: %ls, World: %ls", __FUNCTION__, *GetNameSafe(InLevel), *GetNameSafe(InWorld));

	if ((IsNavigationSystemStatic() == false))
	{
		AddLevelCollisionToOctree(InLevel);
	}

	if (!InLevel->IsPersistentLevel())
	{
		for (ANavigationData* NavData : NavDataSet)
		{
			if (NavData)
			{
				NavData->OnStreamingLevelAdded(InLevel, InWorld);
			}
		}
	}

#if WITH_EDITOR
	if (FNavigationSystem::IsEditorRunMode(OperationMode))
	{
		// see if there are any unregistered yet valid nav data instances
		// In general we register navdata on its PostLoad, but in some cases
		// levels get removed from world and readded and in that case we might
		// miss registering them
		for (AActor* Actor : InLevel->Actors)
		{
			ANavigationData* NavData = Cast<ANavigationData>(Actor);
			if (NavData != nullptr && NavData->IsRegistered() == false)
			{
				RequestRegistrationDeferred(*NavData);
			}
		}
	}
	else
#endif // WITH_EDITOR
	if (OperationMode == FNavigationSystemRunMode::InvalidMode)
	{
		// While streaming multiple levels it is possible that NavigationData and NavMeshBoundsVolume from different levels gets
		// loaded in different order so we need to wait navigation system initialization to make sure everything is registered properly.
		// Otherwise the register may fail and discard the navigation data since navbounds are not registered.
		UE_LOGF(LogNavigation, Log, "%s won't process navigation data registration candidates until OperationMode is set. Waiting for OnWorldInitDone.", __FUNCTION__);
	}
	else if (NavDataRegistrationQueue.Num() > 0)
	{
		UE_LOGF(LogNavigation, Log, "%s Calling ProcessRegistrationCandidates (level: %ls, world: %ls).", __FUNCTION__, *GetFullNameSafe(InLevel), *GetFullNameSafe(InWorld));
		ProcessRegistrationCandidates();
	}
}

void UNavigationSystemV1::OnLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld)
{
	if ((InWorld == GetWorld()) && (InLevel != nullptr))
	{
		if (IsNavigationSystemStatic() == false)
		{
			RemoveLevelCollisionFromOctree(InLevel);
		}

		if (InLevel && !InLevel->IsPersistentLevel())
		{
			for (int32 DataIndex = NavDataSet.Num() - 1; DataIndex >= 0; --DataIndex)
			{		
				ANavigationData* NavData = NavDataSet[DataIndex];
				if (NavData)
				{
					if (NavData->GetLevel() != InLevel)
					{
						NavData->OnStreamingLevelRemoved(InLevel, InWorld);
					}
					else
					{
						// removing manually first so that UnregisterNavData won't mess with NavDataSet
						NavDataSet.RemoveAt(DataIndex, EAllowShrinking::No);
						UnregisterNavData(NavData);
					}
				}
			}
		}
	}
}

void UNavigationSystemV1::AddLevelToOctree(ULevel& Level)
{
	// We only need to add level collision (BSP)
	// Actors and components are handled by the navigation element repository.
	AddLevelCollisionToOctree(&Level);
}

void UNavigationSystemV1::AddLevelCollisionToOctree(ULevel* Level)
{
	if (Level)
	{
		FNavigationDataHandler(DefaultOctreeController, DefaultDirtyAreasController).AddLevelCollisionToOctree(*Level);
	}
}

void UNavigationSystemV1::RemoveLevelCollisionFromOctree(ULevel* Level)
{
	if (Level)
	{
		FNavigationDataHandler(DefaultOctreeController, DefaultDirtyAreasController).RemoveLevelCollisionFromOctree(*Level);
	}
}

void UNavigationSystemV1::OnPostLoadMap(UWorld* LoadedWorld)
{
	if (LoadedWorld != GetWorld())
	{
		return;
	}

	UE_LOGF(LogNavigation, Verbose, "%s (Package: %ls)", __FUNCTION__, *GetNameSafe(LoadedWorld->GetOuter()));

	// If map has been loaded and there are some navigation bounds volumes 
	// then create appropriate navigation structure.
	ANavigationData* NavData = GetDefaultNavDataInstance(FNavigationSystem::DontCreate);

	// Do this if there's currently no navigation
	if ( NavData == nullptr &&
		 bAutoCreateNavigationData &&
		 IsThereAnywhereToBuildNavigation() &&
		 (GetRuntimeGenerationType() != ERuntimeGenerationType::Static) )	// Prevent creating a static default nav instance out of the editor (GetRuntimeGenerationType() is always dynamic in editor).
	{
		NavData = GetDefaultNavDataInstance(FNavigationSystem::Create);
		UE_LOGF(LogNavigation, Verbose, "%s Created DefaultNavDataInstance %ls", __FUNCTION__, *GetNameSafe(NavData));
	}
}

#if WITH_EDITOR
void UNavigationSystemV1::OnMapOpened(const FString& Filename, bool bAsTemplate)
{
	// Now that the maps as loaded, remove unnecessary packages (and actors for OFPA).
	for (TWeakObjectPtr<ANavigationData> NavigationDataPtr : NavigationDataToRemove)
	{
		if (NavigationDataPtr.IsValid())
		{
			bool bIsUsingExternalActor = false;
			if (const UWorld* World = GetWorld())
			{
				const ULevel* Level = World->PersistentLevel;
				bIsUsingExternalActor = Level && Level->IsUsingExternalActors();
			}

			UPackage* Package = nullptr;
			FString PackageName;
			if (bIsUsingExternalActor)
			{
				Package = NavigationDataPtr->GetPackage();
				if (Package)
				{
				    PackageName = SourceControlHelpers::PackageFilename(Package);
				}	
			}
			
			UE_LOGF(LogNavigation, Verbose, "%s Destroying %ls because it failed to register.", __FUNCTION__, *NavigationDataPtr->GetFullName());
			NavigationDataPtr->DestroyActor();

			if (bIsUsingExternalActor && Package)
			{
				bool bDeleted = false;

				FPackageSourceControlHelper PackageHelper;
				if (PackageHelper.UseSourceControl())
				{
					// Delete with source control
					bDeleted = PackageHelper.Delete(Package);
				}
				else
				{
					// Direct file delete
					if (!PackageName.IsEmpty())
					{
						bDeleted = FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*PackageName);
					}
				}

				if (bDeleted)
				{
					UE_LOGF(LogNavigation, Verbose, "%s File deleted (%ls).", __FUNCTION__, *PackageName);
				}
			}
		}
	}
	NavigationDataToRemove.Reset();
}

void UNavigationSystemV1::OnActorMoved(AActor* Actor)
{
	if (Cast<ANavMeshBoundsVolume>(Actor))
	{
		OnNavigationBoundsUpdated((ANavMeshBoundsVolume*)Actor);
	}
	// We need to check this actor has registered all their components post spawn / load
	// before attempting to update the components in the nav octree.
	// Without this check we were getting an issue with UNavRelevantComponent::GetNavigationParent().
	else if (Actor && Actor->HasActorRegisteredAllComponents())
	{
		UpdateActorAndComponentsInNavOctree(*Actor, /*bUpdateAttachedActors=*/true);
	}
}
#endif // WITH_EDITOR

void UNavigationSystemV1::OnNavigationDirtied(const FBox& Bounds)
{
	AddDirtyArea(Bounds, ENavigationDirtyFlag::All, "OnNavigationDirtied");
}

void UNavigationSystemV1::OnReloadComplete(EReloadCompleteReason Reason)
{
	if (RequiresNavOctree() && DefaultOctreeController.NavOctree.IsValid() == false)
	{
		ConditionalPopulateNavOctree();

		if (bInitialBuildingLocked)
		{
			RemoveNavigationBuildLock(ENavigationBuildLock::InitialLock, ELockRemovalRebuildAction::RebuildIfNotInEditor);
		}
	}
}

void UNavigationSystemV1::CleanUp(FNavigationSystem::ECleanupMode Mode)
{
	if (bCleanUpDone)
	{
		return;
	}

	UE_LOGF(LogNavigation, Log, "UNavigationSystemV1::CleanUp");

#if WITH_EDITOR
	if (GIsEditor && GEngine)
	{
		FEditorDelegates::OnMapOpened.RemoveAll(this);
		
		GEngine->OnActorMoved().RemoveAll(this);
	}

	UE_LOGF(LogNavigation, Verbose, "%s Removing functions from OnAssetPostCompileEvent delegate.", __FUNCTION__);
	FAssetCompilingManager::Get().OnAssetPostCompileEvent().Remove(OnAssetPostCompileEventHandle);
	OnAssetPostCompileEventHandle.Reset();
	UnregisterDelayedUnlock();
#endif // WITH_EDITOR

	FCoreUObjectDelegates::PostLoadMapWithWorld.RemoveAll(this);
	UNavigationSystemV1::NavigationDirtyEvent.RemoveAll(this);
	FWorldDelegates::LevelAddedToWorld.RemoveAll(this);
	FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);
	FWorldDelegates::OnWorldBeginTearDown.RemoveAll(this);
#if !UE_BUILD_SHIPPING
	FCoreDelegates::OnGetOnScreenMessages.RemoveAll(this);
#endif // !UE_BUILD_SHIPPING

	FCoreUObjectDelegates::ReloadCompleteDelegate.Remove(ReloadCompleteDelegateHandle);

	// Unregister and cleanup navigation data before destroying their dependencies.
	// The order of operations here mirrors ANavigationData::UnregisterAndCleanUp(), 
	// minus it having to resolve this NavigationSystem.
	for (int32 Idx = NavDataSet.Num() - 1; Idx >= 0; --Idx)
	{
		ANavigationData* NavData = NavDataSet[Idx];
		if (NavData)
		{
			// Unregister the nav data
			if (NavData->IsRegistered())
			{
				UnregisterNavData(NavData);
			}

			// Clean up nav data before the cleaning up the rest of the system. This may block while the NavData waits on async 
			// tasks that it started, but this is safer than cleaning up navigation systems while those tasks are running, since
			// those tasks may access state we're about to destroy such as the NavOctree.
			NavData->CleanUp();
		}
	}

	DestroyNavOctree();

	SetCrowdManager(nullptr);

	if (NavDataSet.Num())
	{
		UE_LOGF(LogNavigation, Error, "UNavigationSystemV1::CleanUp still has data in NavDataSet after unregister them all");
		NavDataSet.Reset();
	}

	if (AgentToNavDataMap.Num())
	{
		UE_LOGF(LogNavigation, Error, "UNavigationSystemV1::CleanUp still has agents mapped to navigation data after clean up");
		AgentToNavDataMap.Reset();
	}
	
	MainNavData = nullptr;

	const UWorld* MyWorld = (Mode == FNavigationSystem::ECleanupMode::CleanupWithWorld) ? GetWorld() : nullptr;
	if (MyWorld)
	{
		if (bInitialSetupHasBeenPerformed)
		{
			UnregisterFromRepositoryDelegates();
		}

		// reset unique link Id for new map
		if (MyWorld->WorldType == EWorldType::Game || MyWorld->WorldType == EWorldType::Editor)
		{
			UE_LOGF(LogNavLink, VeryVerbose, "Reset navlink id on cleanup.");
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			INavLinkCustomInterface::ResetUniqueId();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	bCleanUpDone = true;
}

void UNavigationSystemV1::DestroyNavOctree()
{
	DefaultOctreeController.Reset();
}

bool UNavigationSystemV1::RequiresNavOctree() const
{
	UWorld* World = GetWorld();
	check(World);
	
	// We always require navoctree in editor worlds
	if (!World->IsGameWorld())
	{
		return true;
	}
		
	for (ANavigationData* NavData : NavDataSet)
	{
		if (NavData && NavData->SupportsRuntimeGeneration())
		{
			return true;
		}
	}
	
	return false;
}

ERuntimeGenerationType UNavigationSystemV1::GetRuntimeGenerationType() const
{
	UWorld* World = GetWorld();
	check(World);
	
	// We always use ERuntimeGenerationType::Dynamic in editor worlds
	if (!World->IsGameWorld())
	{
		return ERuntimeGenerationType::Dynamic;
	}
	
	ERuntimeGenerationType RuntimeGenerationType = ERuntimeGenerationType::Static;

	for (ANavigationData* NavData : NavDataSet)
	{
		if (NavData && NavData->GetRuntimeGenerationMode() > RuntimeGenerationType)
		{
			RuntimeGenerationType = NavData->GetRuntimeGenerationMode();
		}
	}
	
	return RuntimeGenerationType;
}

bool UNavigationSystemV1::DoesAnyNavDataSupportRuntimeGeneration() const
{
	UWorld* World = GetWorld();
	check(World);

	if (!World->IsGameWorld())
	{
		return true;
	}

	for (ANavigationData* NavData : NavDataSet)
	{
		if (NavData && NavData->SupportsRuntimeGeneration())
		{
			return true;
		}
	}

	return false;
}

bool UNavigationSystemV1::DoesAnyNavDataRequireGeometryStorage() const
{
	UWorld* World = GetWorld();
	check(World);

	if (!World->IsGameWorld())
	{
		return true;
	}

	for (ANavigationData* NavData : NavDataSet)
	{
		if (NavData && (NavData->GetRuntimeGenerationMode() == ERuntimeGenerationType::Dynamic || NavData->RequiresInitialRebuild()))
		{
			return true;
		}
	}

	return false;
}

// Deprecated
void UNavigationSystemV1::LogNavDataRegistrationResult(ERegistrationResult InResult)
{
	LogNavDataRegistrationResult(InResult, nullptr);
}

void UNavigationSystemV1::LogNavDataRegistrationResult(ERegistrationResult InResult, const ANavigationData* NavigationData) const
{
	switch (InResult)
	{
	case UNavigationSystemV1::RegistrationError:
		UE_VLOG_UELOG(this, LogNavigation, Warning,
			TEXT("NavData RegistrationError, could not be registered (%s)."), *GetNameSafe(NavigationData));
		break;
	case UNavigationSystemV1::RegistrationFailed_DataPendingKill:
		UE_VLOG_UELOG(this, LogNavigation, Warning,
			TEXT("NavData RegistrationFailed_DataPendingKill (%s)."), *GetNameSafe(NavigationData));
		break;
	case UNavigationSystemV1::RegistrationFailed_AgentAlreadySupported:
		UE_VLOG_UELOG(this, LogNavigation, Warning,
			TEXT("NavData RegistrationFailed_AgentAlreadySupported, specified agent type already has its navmesh implemented (%s)."), *GetNameSafe(NavigationData));
		break;
	case UNavigationSystemV1::RegistrationFailed_AgentNotValid:
		UE_VLOG_UELOG(this, LogNavigation, Warning,
			TEXT("NavData RegistrationFailed_AgentNotValid, NavData instance contains navmesh that doesn't support any of expected agent types (%s)."), *GetNameSafe(NavigationData));
		break;
	case UNavigationSystemV1::RegistrationFailed_NotSuitable:
		UE_VLOG_UELOG(this, LogNavigation, Warning,
			TEXT("NavData RegistrationFailed_NotSuitable (%s)."), *GetNameSafe(NavigationData));
		break;
	case UNavigationSystemV1::RegistrationSuccessful:
		UE_VLOG_UELOG(this, LogNavigation, Verbose,
			TEXT("NavData RegistrationSuccessful (%s)."), *GetNameSafe(NavigationData));
		break;
	default:
		UE_VLOG_UELOG(this, LogNavigation, Warning,
			TEXT("Registration not successful default warning (%s)."), *GetNameSafe(NavigationData));
		break;
	}
}

bool UNavigationSystemV1::IsAllowedToRebuild() const
{
	const UWorld* World = GetWorld();
	
	return World && (!World->IsGameWorld() || DoesAnyNavDataSupportRuntimeGeneration());
}

void UNavigationSystemV1::OnGenerateNavigationOnlyAroundNavigationInvokersChanged()
{
	if (DefaultOctreeController.NavOctree.IsValid())
	{
		DefaultOctreeController.NavOctree->SetDataGatheringMode(DataGatheringMode);
	}

	for (auto NavData : NavDataSet)
	{
		if (NavData)
		{
			NavData->RestrictBuildingToActiveTiles(bGenerateNavigationOnlyAroundNavigationInvokers);
		}
	}
}

//----------------------------------------------------------------------//
// Blueprint functions
//----------------------------------------------------------------------//
UNavigationSystemV1* UNavigationSystemV1::GetNavigationSystem(UObject* WorldContextObject)
{
	return GetCurrent(WorldContextObject);
}

bool UNavigationSystemV1::K2_ProjectPointToNavigation(UObject* WorldContextObject, const FVector& Point, FVector& ProjectedLocation, ANavigationData* NavData, TSubclassOf<UNavigationQueryFilter> FilterClass, const FVector QueryExtent)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);

	ProjectedLocation = Point;
	bool bResult = false;

	if (NavSys)
	{
		FNavLocation OutNavLocation;
		ANavigationData* UseNavData = NavData ? NavData : NavSys->GetDefaultNavDataInstance(FNavigationSystem::DontCreate);
		if (UseNavData)
		{
			bResult = NavSys->ProjectPointToNavigation(Point, OutNavLocation, QueryExtent, NavData
				, UNavigationQueryFilter::GetQueryFilter(*UseNavData, WorldContextObject, FilterClass));
			ProjectedLocation = OutNavLocation.Location;
		}
	}

	return bResult;
}

bool UNavigationSystemV1::K2_GetRandomReachablePointInRadius(UObject* WorldContextObject, const FVector& Origin, FVector& RandomLocation, float Radius, ANavigationData* NavData, TSubclassOf<UNavigationQueryFilter> FilterClass)
{
	FNavLocation RandomPoint(Origin);
	bool bResult = false;

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);
	if (NavSys)
	{
		ANavigationData* UseNavData = NavData ? NavData : NavSys->GetDefaultNavDataInstance(FNavigationSystem::DontCreate);
		if (UseNavData)
		{
			bResult = NavSys->GetRandomReachablePointInRadius(Origin, Radius, RandomPoint, UseNavData, UNavigationQueryFilter::GetQueryFilter(*UseNavData, WorldContextObject, FilterClass));
			RandomLocation = RandomPoint.Location;
		}
	}

	return bResult;
}

bool UNavigationSystemV1::K2_GetRandomLocationInNavigableRadius(UObject* WorldContextObject, const FVector& Origin, FVector& RandomLocation, float Radius, ANavigationData* NavData, TSubclassOf<UNavigationQueryFilter> FilterClass)
{
	FNavLocation RandomPoint(Origin);
	bool bResult = false;
	RandomLocation = Origin;

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);
	if (NavSys)
	{
		ANavigationData* UseNavData = NavData ? NavData : NavSys->GetDefaultNavDataInstance(FNavigationSystem::DontCreate);
		if (UseNavData)
		{
			if (NavSys->GetRandomPointInNavigableRadius(Origin, Radius, RandomPoint, UseNavData, UNavigationQueryFilter::GetQueryFilter(*UseNavData, WorldContextObject, FilterClass)))
			{
				bResult = true;
				RandomLocation = RandomPoint.Location;
			}
		}
	}

	return bResult;
}

ENavigationQueryResult::Type UNavigationSystemV1::GetPathCost(UObject* WorldContextObject, const FVector& PathStart, const FVector& PathEnd, double& OutPathCost, ANavigationData* NavData, TSubclassOf<UNavigationQueryFilter> FilterClass)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);
	if (NavSys)
	{
		ANavigationData* UseNavData = NavData ? NavData : NavSys->GetDefaultNavDataInstance(FNavigationSystem::DontCreate);
		if (UseNavData)
		{
			return NavSys->GetPathCost(PathStart, PathEnd, OutPathCost, UseNavData, UNavigationQueryFilter::GetQueryFilter(*UseNavData, WorldContextObject, FilterClass));
		}
	}

	return ENavigationQueryResult::Error;
}

ENavigationQueryResult::Type UNavigationSystemV1::GetPathLength(UObject* WorldContextObject, const FVector& PathStart, const FVector& PathEnd, double& OutPathLength, ANavigationData* NavData, TSubclassOf<UNavigationQueryFilter> FilterClass)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);
	if (NavSys)
	{
		ANavigationData* UseNavData = NavData ? NavData : NavSys->GetDefaultNavDataInstance(FNavigationSystem::DontCreate);
		if (UseNavData)
		{
			return NavSys->GetPathLength(PathStart, PathEnd, OutPathLength, UseNavData, UNavigationQueryFilter::GetQueryFilter(*UseNavData, WorldContextObject, FilterClass));
		}
	}

	return ENavigationQueryResult::Error;
}

bool UNavigationSystemV1::IsNavigationBeingBuilt(UObject* WorldContextObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);
	
	if (NavSys && !NavSys->IsNavigationBuildingPermanentlyLocked())
	{
		return NavSys->HasDirtyAreasQueued() || NavSys->IsNavigationBuildInProgress();
	}

	return false;
}

bool UNavigationSystemV1::IsNavigationBeingBuiltOrLocked(UObject* WorldContextObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);

	if (NavSys)
	{
		return NavSys->IsNavigationBuildingLocked() || NavSys->HasDirtyAreasQueued() || NavSys->IsNavigationBuildInProgress();
	}

	return false;
}

bool UNavigationSystemV1::K2_ReplaceAreaInOctreeData(const UObject* Object, TSubclassOf<UNavArea> OldArea, TSubclassOf<UNavArea> NewArea)
{
	SCOPE_CYCLE_COUNTER(STAT_NavOctreeBookkeeping);

	if (Repository == nullptr)
	{
		return false;
	}

	if (const FNavigationElementHandle Handle = Repository->GetNavigationElementHandleForUObject(Object))
	{
		return ReplaceAreaInOctreeData(Handle, OldArea, NewArea);
	}
	return false;
}

//----------------------------------------------------------------------//
// HACKS!!!
//----------------------------------------------------------------------//
bool UNavigationSystemV1::ShouldGeneratorRun(const FNavDataGenerator* Generator) const
{
	if (Generator != nullptr && (IsNavigationSystemStatic() == false))
	{
		for (int32 NavDataIndex = 0; NavDataIndex < NavDataSet.Num(); ++NavDataIndex)
		{
			ANavigationData* NavData = NavDataSet[NavDataIndex];
			if (NavData != nullptr && NavData->GetGenerator() == Generator)
			{
				return true;
			}
		}
	}

	return false;
}

bool UNavigationSystemV1::HandleCycleNavDrawnCommand( const TCHAR* Cmd, FOutputDevice& Ar )
{
	CycleNavigationDataDrawn();

	return true;
}

bool UNavigationSystemV1::HandleCountNavMemCommand()
{
	UE_LOGF(LogNavigation, Warning, "Logging NavigationSystem memory usage:");

	if (DefaultOctreeController.NavOctree.IsValid())
	{
		UE_LOGF(LogNavigation, Warning, "NavOctree memory: %llu", DefaultOctreeController.NavOctree->GetSizeBytes());
	}

	for (int32 NavDataIndex = 0; NavDataIndex < NavDataSet.Num(); ++NavDataIndex)
	{
		ANavigationData* NavData = NavDataSet[NavDataIndex];
		if (NavData != nullptr)
		{
			NavData->LogMemUsed();
		}
	}
	return true;
}

//----------------------------------------------------------------------//
// Commands
//----------------------------------------------------------------------//
bool FNavigationSystemExec::Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	UNavigationSystemV1*  NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(InWorld);

	if (NavSys && NavSys->NavDataSet.Num() > 0)
	{
		if (FParse::Command(&Cmd, TEXT("CYCLENAVDRAWN")))
		{
			NavSys->HandleCycleNavDrawnCommand( Cmd, Ar );
			// not returning true to enable all navigation systems to cycle their own data
			return false;
		}
		else if (FParse::Command(&Cmd, TEXT("CountNavMem")))
		{
			NavSys->HandleCountNavMemCommand();
			return false;
		}
		/** Builds the navigation mesh (or rebuilds it). **/
		else if (FParse::Command(&Cmd, TEXT("RebuildNavigation")))
		{
			NavSys->Build();
		}
		else if (FParse::Command(&Cmd, TEXT("RedrawNav")) || FParse::Command(&Cmd, TEXT("RedrawNavigation")))
		{
			for (ANavigationData* NavData : NavSys->NavDataSet)
			{
				if (NavData)
				{
					NavData->MarkComponentsRenderStateDirty();
				}
			}
		}
	}

	return false;
}

void UNavigationSystemV1::CycleNavigationDataDrawn()
{
	++CurrentlyDrawnNavDataIndex;
	if (CurrentlyDrawnNavDataIndex >= NavDataSet.Num())
	{
		CurrentlyDrawnNavDataIndex = INDEX_NONE;
	}

	for (int32 NavDataIndex = 0; NavDataIndex < NavDataSet.Num(); ++NavDataIndex)
	{
		ANavigationData* NavData = NavDataSet[NavDataIndex];
		if (NavData != nullptr)
		{
			const bool bNewEnabledDrawing = (CurrentlyDrawnNavDataIndex == INDEX_NONE) || (NavDataIndex == CurrentlyDrawnNavDataIndex);
			NavData->SetNavRenderingEnabled(bNewEnabledDrawing);
		}
	}
}

bool UNavigationSystemV1::IsNavigationDirty() const
{
	if (!IsThereAnywhereToBuildNavigation())
	{
		// Nowhere to build navigation so it can't be dirty.
		return false;
	}
	
#if !UE_BUILD_SHIPPING
	if (DefaultDirtyAreasController.HadDirtyAreasReportedWhileAccumulationLocked())
	{
		return true;
	}
#endif // !UE_BUILD_SHIPPING

	for (int32 NavDataIndex=0; NavDataIndex < NavDataSet.Num(); ++NavDataIndex)
	{
		if (NavDataSet[NavDataIndex] && NavDataSet[NavDataIndex]->NeedsRebuild())
		{
			return true;
		}
	}

	return false;
}

bool UNavigationSystemV1::CanRebuildDirtyNavigation() const
{
	const bool bIsInGame = GetWorld()->IsGameWorld();

	for (const ANavigationData* NavData : NavDataSet)
	{
		if (NavData)
		{
			const bool bIsDirty = NavData->NeedsRebuild();
			const bool bCanRebuild = !bIsInGame || NavData->SupportsRuntimeGeneration();

			if (bIsDirty && !bCanRebuild)
			{
				return false;
			}
		}
	}

	return true;
}

bool UNavigationSystemV1::DirtyNavDataPackageOnNavigationGenerationFinished() const
{
	return bDirtyNavDataPackageOnNavigationGenerationFinished;
}

bool UNavigationSystemV1::DoesPathIntersectBox(const FNavigationPath* Path, const FBox& Box, uint32 StartingIndex, FVector* AgentExtent)
{
	return Path != nullptr && Path->DoesIntersectBox(Box, StartingIndex, nullptr, AgentExtent);
}

bool UNavigationSystemV1::DoesPathIntersectBox(const FNavigationPath* Path, const FBox& Box, const FVector& AgentLocation, uint32 StartingIndex, FVector* AgentExtent)
{
	return Path != nullptr && Path->DoesIntersectBox(Box, AgentLocation, StartingIndex, nullptr, AgentExtent);
}

void UNavigationSystemV1::SetMaxSimultaneousTileGenerationJobsCount(int32 MaxNumberOfJobs)
{
#if WITH_RECAST
	for (auto NavigationData : NavDataSet)
	{
		ARecastNavMesh* RecastNavMesh = Cast<ARecastNavMesh>(NavigationData);
		if (RecastNavMesh)
		{
			RecastNavMesh->SetMaxSimultaneousTileGenerationJobsCount(MaxNumberOfJobs);
		}
	}
#endif
}

void UNavigationSystemV1::ResetMaxSimultaneousTileGenerationJobsCount()
{
#if WITH_RECAST
	for (auto NavigationData : NavDataSet)
	{
		ARecastNavMesh* RecastNavMesh = Cast<ARecastNavMesh>(NavigationData);
		if (RecastNavMesh)
		{
			const ARecastNavMesh* CDO = RecastNavMesh->GetClass()->GetDefaultObject<ARecastNavMesh>();
			RecastNavMesh->SetMaxSimultaneousTileGenerationJobsCount(CDO->MaxSimultaneousTileGenerationJobsCount);
		}
	}
#endif
}

//----------------------------------------------------------------------//
// Active tiles
//----------------------------------------------------------------------//

void UNavigationSystemV1::RegisterNavigationInvoker(AActor& Invoker, float TileGenerationRadius, float TileRemovalRadius, const FNavAgentSelector& Agents, ENavigationInvokerPriority Priority)
{
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(Invoker.GetWorld());
	if (NavSys)
	{
		NavSys->RegisterInvoker(Invoker, TileGenerationRadius, TileRemovalRadius, Agents, Priority);
	}
}

void UNavigationSystemV1::UnregisterNavigationInvoker(AActor& Invoker)
{
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(Invoker.GetWorld());
	if (NavSys)
	{
		NavSys->UnregisterInvoker(Invoker);
	}
}

void UNavigationSystemV1::SetGeometryGatheringMode(ENavDataGatheringModeConfig NewMode)
{
	DataGatheringMode = NewMode;
	if (DefaultOctreeController.NavOctree.IsValid())
	{
		DefaultOctreeController.NavOctree->SetDataGatheringMode(DataGatheringMode);
	}
}

namespace UE::Navigation::Private
{
	void LogNavInvokerRegistration(const UNavigationSystemV1& NavSystem, const FNavigationInvoker& Data)
	{
		UE_SUPPRESS(LogNavInvokers, Log,
		{
			TStringBuilder<128> InvokerNavData;
			for (int32 NavDataIndex = 0; NavDataIndex < NavSystem.NavDataSet.Num(); NavDataIndex++)
			{
				const ANavigationData* NavData = NavSystem.NavDataSet[NavDataIndex].Get();
				if (NavData)
				{
					const int32 NavDataSupportedAgentIndex = NavSystem.GetSupportedAgentIndex(NavData);
					if (Data.SupportedAgents.Contains(NavDataSupportedAgentIndex))
					{
						InvokerNavData.Append(FString::Printf(TEXT("%s "), *NavData->GetName()));
					}
				}
			}

			const FString RegisterText = FString::Printf(TEXT("Register invoker r: %.0f, r area: %.0f m2, removal r: %.0f, priority: %s, (%s %s) "),
				Data.GenerationRadius, UE_PI*FMath::Square(Data.GenerationRadius/100.f), Data.RemovalRadius, *UEnum::GetDisplayValueAsText(Data.Priority).ToString(), *Data.GetName(), *InvokerNavData);
			UE_LOGF(LogNavInvokers, Log, "%ls", *RegisterText);

			FVector InvokerLocation = FVector::ZeroVector;
			Data.GetLocation(InvokerLocation);
			UE_VLOG_CYLINDER(&NavSystem, LogNavInvokers, Log, InvokerLocation, InvokerLocation + FVector(0, 0, 20), Data.GenerationRadius, FColorList::LimeGreen, TEXT("%s"), *RegisterText);
			UE_VLOG_CYLINDER(&NavSystem, LogNavInvokers, Log, InvokerLocation, InvokerLocation + FVector(0, 0, 20), Data.RemovalRadius, FColorList::IndianRed, TEXT(""));
		});
	}
}

void UNavigationSystemV1::RegisterInvoker(AActor& Invoker, float TileGenerationRadius, float TileRemovalRadius, const FNavAgentSelector& Agents, ENavigationInvokerPriority InPriority)
{
	UE_CVLOG(bGenerateNavigationOnlyAroundNavigationInvokers == false, this, LogNavInvokers, Warning
		, TEXT("Trying to register %s as invoker, but NavigationSystem is not set up for invoker-centric generation. See GenerateNavigationOnlyAroundNavigationInvokers in NavigationSystem's properties")
		, *Invoker.GetName());

	TileGenerationRadius = FMath::Clamp(TileGenerationRadius, 0.f, BIG_NUMBER);
	TileRemovalRadius = FMath::Clamp(TileRemovalRadius, TileGenerationRadius, BIG_NUMBER);

	FNavigationInvoker& Data = Invokers.FindOrAdd(&Invoker);
	Data.Actor = &Invoker;
	Data.GenerationRadius = TileGenerationRadius;
	Data.RemovalRadius = TileRemovalRadius;
	Data.SupportedAgents = Agents;
	Data.SupportedAgents.MarkInitialized();
	Data.Priority = InPriority;

	UE::Navigation::Private::LogNavInvokerRegistration(*this, Data);
}

void UNavigationSystemV1::RegisterInvoker(const TWeakInterfacePtr<INavigationInvokerInterface>& Invoker, float TileGenerationRadius, float TileRemovalRadius, const FNavAgentSelector& Agents, ENavigationInvokerPriority InPriority)
{
	UE_CVLOG(bGenerateNavigationOnlyAroundNavigationInvokers == false, this, LogNavInvokers, Warning
		, TEXT("Trying to register %s as invoker, but NavigationSystem is not set up for invoker-centric generation. See GenerateNavigationOnlyAroundNavigationInvokers in NavigationSystem's properties")
		, *GetNameSafe(Invoker.GetObject()));

	UObject* InvokerObject = Invoker.GetObject();
	if (ensure(InvokerObject != nullptr))
	{
		FNavigationInvoker& Data = Invokers.FindOrAdd(InvokerObject);
		Data.Object = Invoker;
		Data.GenerationRadius = TileGenerationRadius;
		Data.RemovalRadius = TileRemovalRadius;
		Data.SupportedAgents = Agents;
		Data.SupportedAgents.MarkInitialized();
		Data.Priority = InPriority;

		UE::Navigation::Private::LogNavInvokerRegistration(*this, Data);
	}
}

void UNavigationSystemV1::UnregisterInvoker(AActor& Invoker)
{
	UnregisterInvoker_Internal(Invoker);
}

void UNavigationSystemV1::UnregisterInvoker(const TWeakInterfacePtr<INavigationInvokerInterface>& Invoker)
{
	if (const UObject* InvokerObject = Invoker.GetObject())
	{
		UnregisterInvoker_Internal(*InvokerObject);
	}
}

void UNavigationSystemV1::UnregisterInvoker_Internal(const UObject& Invoker)
{
	UE_VLOG(this, LogNavInvokers, Log, TEXT("Removing %s from invokers list"), *Invoker.GetName());
	Invokers.Remove(&Invoker);
}

void UNavigationSystemV1::RegisterToRepositoryDelegates()
{
	if (Repository == nullptr)
	{
		return;
	}

	Repository->OnCustomNavLinkObjectRegistered.BindWeakLambda(this, [this](INavLinkCustomInterface& CustomLink)
		{
			RegisterCustomLink(CustomLink);
		});

	Repository->OnCustomNavLinkObjectUnregistered.BindWeakLambda(this, [this](INavLinkCustomInterface& CustomLink)
		{
			UnregisterCustomLink(CustomLink);
		});

	Repository->OnNavigationElementAddedDelegate.BindWeakLambda(this, [this](const TSharedRef<const FNavigationElement>& Element)
		{
			RegisterNavigationElementWithNavOctree(Element, FNavigationOctreeController::OctreeUpdate_Default);
		});

	Repository->OnNavigationElementRemovedDelegate.BindWeakLambda(this, [this](const TSharedRef<const FNavigationElement>& Element)
		{
			UnregisterNavigationElementWithOctree(Element, FNavigationOctreeController::OctreeUpdate_Default);
		});
}

void UNavigationSystemV1::UnregisterFromRepositoryDelegates() const
{
	if (Repository == nullptr)
	{
		return;
	}

	Repository->OnCustomNavLinkObjectRegistered = nullptr;
	Repository->OnCustomNavLinkObjectUnregistered = nullptr;
	Repository->OnNavigationElementAddedDelegate = nullptr;
	Repository->OnNavigationElementRemovedDelegate = nullptr;
}

#if WITH_EDITOR
void UNavigationSystemV1::UpdateDelayedUnlockeRegistration(const TArray<FAssetCompileData>& AssetCompileData)
{
	UE_LOGF(LogNavigation, Verbose, "%s Assests remaining %i.", __FUNCTION__, FAssetCompilingManager::Get().GetNumRemainingAssets());
	
	// Register a delayed unlock navigation when all assets are finished compiling.
	if (FAssetCompilingManager::Get().GetNumRemainingAssets() == 0)
	{
		// Wait until we get some idle time to avoid unlocking too soon
		constexpr float TimeDelay = 2.f;
		RegisterDelayedUnlock(TimeDelay);
	}
	else
	{
		// We're not idle anymore, unregister unlocking until we get to 0 assets again
		UnregisterDelayedUnlock();
	}
}

void UNavigationSystemV1::RegisterDelayedUnlock(float DelayInSeconds)
{
	UE_LOGF(LogNavigation, VeryVerbose, "%hs", __FUNCTION__);
	
	// Unregister the previous one and register a new one.
	UnregisterDelayedUnlock();

	// The editor might be unresponsive during heavy asset compilation, so we 
	// not only need a delay, but also a minimum amount of frames
	// to pass until we call the actual refresh.
	DelayedUnlockHandle = FTSTicker::GetCoreTicker().AddTicker(
		TEXT("UNavigationSystemV1 async loading wait for unlock"),
		0.f,
		[this, FireInTickCount = 16, DelayInSeconds](float DeltaTime) mutable
		{
			UE_LOGF(LogNavigation, VeryVerbose, "%s    Time remaining %f.", __FUNCTION__, DelayInSeconds);
				
			DelayInSeconds -= DeltaTime;
			if (--FireInTickCount <= 0 && DelayInSeconds <= 0.f && FAssetCompilingManager::Get().GetNumRemainingAssets() == 0)
			{
				UE_LOGF(LogNavigation, Verbose, "%s    Removing functions from OnAssetPostCompileEvent delegate.", __FUNCTION__);
				FAssetCompilingManager::Get().OnAssetPostCompileEvent().RemoveAll(this);
				
				// Cleared initial async load dirty areas and remove AsyncLoadLock
				UE_LOGF(LogNavigation, Verbose, "%s    Done with assets. Removing ENavigationBuildLock::AsyncLoadLock.", __FUNCTION__);
				DefaultDirtyAreasController.Reset();
				RemoveNavigationBuildLock(ENavigationBuildLock::AsyncLoadLock, ELockRemovalRebuildAction::NoRebuild);
				
				return false; // stop ticker
			}

			return true;
		}
	);
}

void UNavigationSystemV1::UnregisterDelayedUnlock()
{
	if (DelayedUnlockHandle.IsValid())
	{
		UE_LOGF(LogNavigation, Verbose, "%s", __FUNCTION__);
		
		FTSTicker::RemoveTicker(DelayedUnlockHandle);
		DelayedUnlockHandle.Reset();
	}
}
#endif // WITH_EDITOR

void UNavigationSystemV1::UpdateInvokers()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Navigation_UpdateInvokers);
	
	const UWorld* World = GetWorld();
	const double CurrentTime = World->GetTimeSeconds();
	if (CurrentTime >= NextInvokersUpdateTime)
	{
		InvokerLocations.Reset();
		InvokersSeedBounds.Reset();

		if (Invokers.Num() > 0)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_NavSys_Clusterize);

			const bool bCheckMaximumDistanceFromSeeds = (InvokersMaximumDistanceFromSeed != -1) && World->IsGameWorld();
			TArray<FVector, TInlineAllocator<32>> SeedLocations;
			if (bCheckMaximumDistanceFromSeeds)
			{
				GetInvokerSeedLocations(*World, SeedLocations);

				// Fill seed bounds
				for (const FVector SeedLocation : SeedLocations)
				{
					InvokersSeedBounds.Emplace(
						FVector(SeedLocation.X-InvokersMaximumDistanceFromSeed, SeedLocation.Y-InvokersMaximumDistanceFromSeed, SeedLocation.Z-InvokersMaximumDistanceFromSeed),
						FVector(SeedLocation.X+InvokersMaximumDistanceFromSeed, SeedLocation.Y+InvokersMaximumDistanceFromSeed, SeedLocation.Z+InvokersMaximumDistanceFromSeed));
				}
			}

#if ENABLE_VISUAL_LOG
			const double StartTime = FPlatformTime::Seconds();
#endif // ENABLE_VISUAL_LOG

			InvokerLocations.Reserve(Invokers.Num());

			for (auto ItemIterator = Invokers.CreateIterator(); ItemIterator; ++ItemIterator)
			{
				FVector InvokerLocation;
				if (!ItemIterator->Value.GetLocation(InvokerLocation))
				{
					ItemIterator.RemoveCurrent();
					continue;
				}

				const float GenerationRadius = ItemIterator->Value.GenerationRadius;
				bool bKeep = !bCheckMaximumDistanceFromSeeds;

				double ClosestDistanceSq = DBL_MAX;
				if (bCheckMaximumDistanceFromSeeds)
				{
					const double CheckDistanceSq = FMath::Square(InvokersMaximumDistanceFromSeed + GenerationRadius);

					// Check if the invoker is close enough
					for (const FVector SeedLocation : SeedLocations)
					{
						const double InvokerDistanceToSeedSq = FVector::DistSquared(SeedLocation, InvokerLocation);
						if (InvokerDistanceToSeedSq <= CheckDistanceSq)
						{
							bKeep = true;
							break;
						}
						else
						{
							ClosestDistanceSq = FMath::Min(InvokerDistanceToSeedSq, ClosestDistanceSq);
						}
					}
				}

				if (bKeep)
				{
					InvokerLocations.Add(FNavigationInvokerRaw(InvokerLocation, GenerationRadius, ItemIterator->Value.RemovalRadius,
						ItemIterator->Value.SupportedAgents, ItemIterator->Value.Priority));
				}
				else
				{
					UE_LOGF(LogNavInvokers, Verbose, "Invoker %ls ignored because it's too far from any seed location. Closest seed at %.0f.",
						*ItemIterator->Value.GetName(), FMath::Sqrt(ClosestDistanceSq));
				}
			}

#if ENABLE_VISUAL_LOG
			const double CachingFinishTime = FPlatformTime::Seconds();
			UE_VLOG(this, LogNavInvokers, Log, TEXT("Caching time %fms"), (CachingFinishTime - StartTime) * 1000);

			for (const auto& InvokerData : InvokerLocations)
			{
				UE_VLOG_CYLINDER(this, LogNavInvokers, Log, InvokerData.Location, InvokerData.Location + FVector(0, 0, 20), InvokerData.RadiusMax, FColorList::Blue, TEXT(""));
				UE_VLOG_CYLINDER(this, LogNavInvokers, Log, InvokerData.Location, InvokerData.Location + FVector(0, 0, 20), InvokerData.RadiusMin, FColorList::CadetBlue, TEXT("Priority %u"), uint32{EnumToUnderlyingType(InvokerData.Priority)});
			}
#endif // ENABLE_VISUAL_LOG
		}

		UpdateNavDataActiveTiles();

		// once per second
		NextInvokersUpdateTime = CurrentTime + ActiveTilesUpdateInterval;
	}

#if !UE_BUILD_SHIPPING
#if CSV_PROFILER_STATS
	if (FCsvProfiler::Get()->IsCapturing())
	{
		TArray<int32, TInlineAllocator<8>> InvokerCounts;
		InvokerCounts.InsertZeroed(0, NavDataSet.Num());
	
		for (int32 NavDataIndex = 0; NavDataIndex < NavDataSet.Num(); NavDataIndex++)
		{
			const ANavigationData* NavData = NavDataSet[NavDataIndex].Get();
			if (NavData)
			{
				const int32 NavDataSupportedAgentIndex = GetSupportedAgentIndex(NavData);	

				for (auto ItemIterator = InvokerLocations.CreateIterator(); ItemIterator; ++ItemIterator)
				{
					const FNavAgentSelector& InvokerSupportedAgents = ItemIterator->SupportedAgents;
					if (InvokerSupportedAgents.Contains(NavDataSupportedAgentIndex))
					{
						InvokerCounts[NavDataIndex]++;
					}
				}

				const FString StatName = FString::Printf(TEXT("InvokerCount_%s"), *NavData->GetName()); 
				FCsvProfiler::RecordCustomStat(*StatName, CSV_CATEGORY_INDEX(NavInvokers), InvokerCounts[NavDataIndex], ECsvCustomStatOp::Set);
			}

			FCsvProfiler::RecordCustomStat(TEXT("InvokersFarAway"), CSV_CATEGORY_INDEX(NavInvokers), Invokers.Num() - InvokerLocations.Num(), ECsvCustomStatOp::Set);
		}		
	}
#endif // CSV_PROFILER_STATS
#endif // !UE_BUILD_SHIPPING
}

#if !UE_BUILD_SHIPPING
void UNavigationSystemV1::DebugLogInvokers(FOutputDevice& OutputDevice) const
{
	OutputDevice.Logf(ELogVerbosity::Log, TEXT("Logging %d Invokers:"), Invokers.Num());
	for (auto It = Invokers.CreateConstIterator(); It; ++It)
	{
		const FNavigationInvoker& Invoker = It.Value();
		OutputDevice.Logf(ELogVerbosity::Log, TEXT("- %s: Radius[Generation:%s Removal:%s] Agents:%d Priority:%s"),
			*GetNameSafe(It.Key()),
			*FString::SanitizeFloat(Invoker.GenerationRadius),
			*FString::SanitizeFloat(Invoker.RemovalRadius),
			Invoker.SupportedAgents.GetAgentBits(),
			*UEnum::GetValueAsString(Invoker.Priority));
	}
}
#endif // !UE_BUILD_SHIPPING

void UNavigationSystemV1::UpdateNavDataActiveTiles()
{
#if WITH_RECAST
	const double UpdateStartTime = FPlatformTime::Seconds();
	for (TActorIterator<ARecastNavMesh> It(GetWorld()); It; ++It)
	{
		It->UpdateActiveTiles(InvokerLocations);
	}
	const double UpdateEndTime = FPlatformTime::Seconds();
	UE_VLOG(this, LogNavInvokers, Log, TEXT("Marking tiles to update %fms (%d invokers)"), (UpdateEndTime - UpdateStartTime) * 1000, InvokerLocations.Num());
#endif
}

void UNavigationSystemV1::DirtyTilesInBuildBounds()
{
#if WITH_RECAST
	UE_VLOG(this, LogNavigation, Log, TEXT("SetupTilesFromBuildBounds"));
	for (TActorIterator<ARecastNavMesh> It(GetWorld()); It; ++It)
	{
		It->DirtyTilesInBounds(BuildBounds);
	}
#endif // WITH_RECAST
}

void UNavigationSystemV1::RegisterNavigationInvoker(AActor* Invoker, float TileGenerationRadius, float TileRemovalRadius)
{
	if (Invoker != nullptr)
	{
		// The FNavAgentSelector class is not yet exposed in BP so we use the default value to specify that we want to generate the navmesh for all agents
		RegisterInvoker(*Invoker, TileGenerationRadius, TileRemovalRadius, FNavAgentSelector(), ENavigationInvokerPriority::Default);
	}
}

void UNavigationSystemV1::UnregisterNavigationInvoker(AActor* Invoker)
{
	if (Invoker != nullptr)
	{
		UnregisterInvoker(*Invoker);
	}
}

// Deprecated
bool UNavigationSystemV1::K2_GetRandomPointInNavigableRadius(UObject* WorldContextObject, const FVector& Origin, FVector& RandomLocation, float Radius, ANavigationData* NavData, TSubclassOf<UNavigationQueryFilter> FilterClass)
{
	return K2_GetRandomLocationInNavigableRadius(WorldContextObject, Origin, RandomLocation, Radius, NavData, FilterClass);
}

void UNavigationSystemV1::VerifyNavigationRenderingComponents(const bool bShow)
{
	// make sure nav mesh has a rendering component
	ANavigationData* const NavData = GetDefaultNavDataInstance(FNavigationSystem::DontCreate);

	if (NavData && NavData->RenderingComp == nullptr)
	{
		NavData->RenderingComp = NavData->ConstructRenderingComponent();
		if (NavData->RenderingComp)
		{
			NavData->RenderingComp->SetVisibility(bShow);
			NavData->RenderingComp->RegisterComponent();
		}
	}

	if (NavData == nullptr)
	{
		UE_LOGF(LogNavigation, Warning, "No NavData found when calling UNavigationSystemV1::VerifyNavigationRenderingComponents()");
	}
}

#if !UE_BUILD_SHIPPING
void UNavigationSystemV1::GetOnScreenMessages(TMultiMap<FCoreDelegates::EOnScreenMessageSeverity, FText>& OutMessages)
{
	// check navmesh
#if WITH_EDITOR
	const bool bIsNavigationAutoUpdateEnabled = GetIsAutoUpdateEnabled();
#else
	const bool bIsNavigationAutoUpdateEnabled = true;
#endif

	// Don't display "navmesh needs to be rebuilt" on-screen editor message in partitioned world. 
	// It's not meaningful since loading and unloading parts of the world triggers it.
	if (!UWorld::IsPartitionedWorld(GetWorld())
		&& IsNavigationDirty()
		&& ((FNavigationSystem::IsEditorRunMode(OperationMode) && !bIsNavigationAutoUpdateEnabled) || !SupportsNavigationGeneration() || !CanRebuildDirtyNavigation()))
	{
		OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Error
			, LOCTEXT("NAVMESHERROR", "NAVMESH NEEDS TO BE REBUILT"));
	}
}
#endif // !UE_BUILD_SHIPPING

INavigationDataInterface* UNavigationSystemV1::GetNavDataForActor(const AActor& Actor)
{
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(Actor.GetWorld());
	ANavigationData* NavData = nullptr;
	const INavAgentInterface* AsNavAgent = CastChecked<INavAgentInterface>(&Actor);
	if (AsNavAgent)
	{
		const FNavAgentProperties& AgentProps = AsNavAgent->GetNavAgentPropertiesRef();
		NavData = NavSys->GetNavDataForProps(AgentProps, AsNavAgent->GetNavAgentLocation());
	}
	if (NavData == nullptr)
	{
		NavData = NavSys->GetDefaultNavDataInstance(FNavigationSystem::DontCreate);
	}

	return NavData;
}

int UNavigationSystemV1::GetNavigationBoundsForNavData(const ANavigationData& NavData, TArray<FBox>& OutBounds, ULevel* InLevel) const
{
	const int InitialBoundsCount = OutBounds.Num();
	OutBounds.Reserve(InitialBoundsCount + RegisteredNavBounds.Num());
	const int32 AgentIndex = GetSupportedAgentIndex(&NavData);

	if (AgentIndex != INDEX_NONE)
	{
		for (const FNavigationBounds& NavigationBounds : RegisteredNavBounds)
		{
			if ((InLevel == nullptr || NavigationBounds.Level == InLevel)
				&& NavigationBounds.SupportedAgents.Contains(AgentIndex))
			{
				OutBounds.Add(NavigationBounds.AreaBox);
			}
		}
	}

	return OutBounds.Num() - InitialBoundsCount;
}

const FNavDataConfig& UNavigationSystemV1::GetDefaultSupportedAgent()
{
	static const FNavDataConfig DefaultAgent;
	const UNavigationSystemV1* NavSysCDO = GetDefault<UNavigationSystemV1>();
	check(NavSysCDO);
	return NavSysCDO->SupportedAgents.Num() > 0
		? NavSysCDO->GetDefaultSupportedAgentConfig()
		: DefaultAgent;
}

const FNavDataConfig& UNavigationSystemV1::GetBiggestSupportedAgent(const UWorld* World) 
{
	const UNavigationSystemV1* NavSys = nullptr;
	if (World != nullptr)
	{
		NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(World);		
	}

	if (NavSys == nullptr)
	{
		// If no world is available, use the CDO.
		NavSys = GetDefault<UNavigationSystemV1>();
	}
	check(NavSys);

	if (NavSys->GetSupportedAgents().IsEmpty())
	{
		static const FNavDataConfig DefaultAgent;
		return DefaultAgent;
	}

	const FNavDataConfig* BiggestAgent = nullptr;
	for (const FNavDataConfig& Config : NavSys->GetSupportedAgents())
	{
		if (BiggestAgent == nullptr || Config.AgentRadius > BiggestAgent->AgentRadius)
		{
			BiggestAgent = &Config; 
		}
	}

	return *BiggestAgent;
}

#if WITH_EDITOR
double UNavigationSystemV1::GetWorldPartitionNavigationDataBuilderOverlap(const UWorld& World) 
{
	const UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(&World);
	if (NavSys == nullptr)
	{
		// If no world is available, use the CDO.
		NavSys = GetDefault<UNavigationSystemV1>();
	}
	check(NavSys);

	double MaxOverlap = 0;
	for (const ANavigationData* NavData : NavSys->NavDataSet)
	{
		if (NavData)
		{
			MaxOverlap = FMath::Max(MaxOverlap, NavData->GetWorldPartitionNavigationDataBuilderOverlap());
		}
	}

	return MaxOverlap;
}
#endif //WITH_EDITOR

const FNavDataConfig& UNavigationSystemV1::GetDefaultSupportedAgentConfig() const 
{ 
	static const FNavDataConfig DefaultAgent;

	int32 FirstValidIndex = INDEX_NONE;
	for (int32 AgentIndex = 0; AgentIndex < SupportedAgents.Num(); ++AgentIndex)
	{
		if (SupportedAgentsMask.Contains(AgentIndex))
		{
			if ((DefaultAgentName == NAME_None || SupportedAgents[AgentIndex].Name == DefaultAgentName))
			{
				return SupportedAgents[AgentIndex];
			}
			FirstValidIndex = (FirstValidIndex == INDEX_NONE) ? AgentIndex : FirstValidIndex;			
		}
	}

	// if not found, get the first one allowed
	return FirstValidIndex != INDEX_NONE ? SupportedAgents[FirstValidIndex] : DefaultAgent;;
}

void UNavigationSystemV1::OverrideSupportedAgents(const TArray<FNavDataConfig>& NewSupportedAgents)
{
	UE_CLOGF(bWorldInitDone, LogNavigation, Warning, "Trying to override NavigationSystem\'s SupportedAgents past the World\'s initialization");

	SupportedAgentsMask.Empty();

	// reset the SupportedAgents 
	const UNavigationSystemV1* NavSysCDO = GetClass()->GetDefaultObject<UNavigationSystemV1>();
	SupportedAgents = NavSysCDO->SupportedAgents;

	for (const FNavDataConfig& Agent : NewSupportedAgents)
	{
		for(int32 AgentIndex = 0; AgentIndex < SupportedAgents.Num(); AgentIndex++)
		{
			if (SupportedAgents[AgentIndex].IsEquivalent(Agent))
			{
				SupportedAgentsMask.Set(AgentIndex);
				break;
			}
		}
	}

	SupportedAgentsMask.MarkInitialized();

	ApplySupportedAgentsFilter();
}

void UNavigationSystemV1::ApplySupportedAgentsFilter()
{
	// reset the SupportedAgents 
	const UNavigationSystemV1* NavSysCDO = GetClass()->GetDefaultObject<UNavigationSystemV1>();
	SupportedAgents = NavSysCDO->SupportedAgents;
	// make sure there's at least one supported navigation agent size
	if (SupportedAgents.Num() == 0)
	{
		SupportedAgents.Add(UE::Navigation::Private::GetFallbackNavDataConfig());
	}

	// make all SupportedAgents filtered out by SupportedAgentsMask invalid by
	// clearing out their NavDataClass
	for (int32 AgentIndex = 0; AgentIndex < SupportedAgents.Num(); AgentIndex++)
	{
		if (SupportedAgentsMask.Contains(AgentIndex) == false)
		{
			SupportedAgents[AgentIndex].Invalidate();
		}
	}
}

void UNavigationSystemV1::UnregisterUnusedNavData()
{
	for (int32 AgentIndex = 0; AgentIndex < SupportedAgents.Num(); AgentIndex++)
	{
		if (SupportedAgentsMask.Contains(AgentIndex) == false)
		{
			// if we already have navdata for this agent we need to remove it
			ANavigationData* NavData = GetNavDataForAgentName(SupportedAgents[AgentIndex].Name);
			if (NavData)
			{
				UnregisterNavData(NavData);
			}
		}
	}
}

void UNavigationSystemV1::SetSupportedAgentsMask(const FNavAgentSelector& InSupportedAgentsMask)
{
	SupportedAgentsMask = InSupportedAgentsMask;
	ApplySupportedAgentsFilter();
}

void UNavigationSystemV1::Configure(const UNavigationSystemConfig& Config)
{
	if (Config.DefaultAgentName != NAME_None)
	{
		DefaultAgentName = Config.DefaultAgentName;
	}
	SetSupportedAgentsMask(Config.SupportedAgentsMask);

	if (DefaultAgentName == NAME_None)
	{
		if (SupportedAgents.Num() == 1)
		{
			DefaultAgentName = SupportedAgents[0].Name;
		}
		else // pick the first available one
		{
			for (int32 AgentIndex = 0; AgentIndex < SupportedAgents.Num(); ++AgentIndex)
			{
				if (SupportedAgents[AgentIndex].IsValid())
				{
					DefaultAgentName = SupportedAgents[AgentIndex].Name;
					break;
				}
			}
		}
	}
}

void UNavigationSystemV1::AppendConfig(const UNavigationSystemConfig& NewConfig)
{
	if (NewConfig.SupportedAgentsMask.IsSame(SupportedAgentsMask) == false)
	{
		bool bAgentsAdded = false;
		for (int AgentIndex = 0; AgentIndex < SupportedAgents.Num(); ++AgentIndex)
		{
			if (NewConfig.SupportedAgentsMask.Contains(AgentIndex) == true
				&& SupportedAgentsMask.Contains(AgentIndex) == false)
			{
				SupportedAgentsMask.Set(AgentIndex);
				bAgentsAdded = true;
			}
		}

		if (bAgentsAdded)
		{
			ApplySupportedAgentsFilter();
			// @todo consider updating the octree, it might be missing data for the new agent(s)
		}

		if (DefaultAgentName == NAME_None)
		{
			DefaultAgentName = NewConfig.DefaultAgentName;
		}
	}
}

//----------------------------------------------------------------------//
// UNavigationSystemModuleConfig
//----------------------------------------------------------------------//
UNavigationSystemModuleConfig::UNavigationSystemModuleConfig(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bDirtyNavDataPackageOnNavigationGenerationFinished(false)
{
}

void UNavigationSystemModuleConfig::PostInitProperties()
{
	Super::PostInitProperties();

	const UNavigationSystemV1* NavSysCDO = GetDefault<UNavigationSystemV1>();
	if (NavSysCDO)
	{
		UpdateWithNavSysCDO(*NavSysCDO);
	}
}

void UNavigationSystemModuleConfig::UpdateWithNavSysCDO(const UNavigationSystemV1& NavSysCDO)
{
	UClass* MyClass = NavigationSystemClass.ResolveClass();
	if (MyClass != nullptr && MyClass->IsChildOf(NavSysCDO.GetClass()))
	{
		// note that we're not longer copying bStrictlyStatic due to UE-91171
		// Copying NavSysCDO.bStaticRuntimeNavigation resulted in copying 'true' 
		// between unrelated maps
		bCreateOnClient = NavSysCDO.bAllowClientSideNavigation;
		bAutoSpawnMissingNavData = NavSysCDO.bAutoCreateNavigationData;
		bSpawnNavDataInNavBoundsLevel = NavSysCDO.bSpawnNavDataInNavBoundsLevel;
		bDirtyNavDataPackageOnNavigationGenerationFinished = NavSysCDO.bDirtyNavDataPackageOnNavigationGenerationFinished;
	}
}

UNavigationSystemBase* UNavigationSystemModuleConfig::CreateAndConfigureNavigationSystem(UWorld& World) const
{
	// This should be handled by ShouldCreateNavigationSystemInstance
	// called from the base class below but this is an early out.
	if (bCreateOnClient == false && World.GetNetMode() == NM_Client)
	{
		return nullptr;
	}

	UNavigationSystemBase* NewNavSys = Super::CreateAndConfigureNavigationSystem(World);
	UNavigationSystemV1* NavSysInstance = Cast<UNavigationSystemV1>(NewNavSys);
	UE_CLOGF(NavSysInstance == nullptr && NewNavSys != nullptr, LogNavigation, Error
		, "Unable to spawn navigation system instance of class %ls - unable to cast to UNavigationSystemV1"
		, *NavigationSystemClass.GetAssetName()
	);
	
	if (NavSysInstance)
	{
		NavSysInstance->bAutoCreateNavigationData = bAutoSpawnMissingNavData;
		NavSysInstance->bSpawnNavDataInNavBoundsLevel = bSpawnNavDataInNavBoundsLevel;
		NavSysInstance->bDirtyNavDataPackageOnNavigationGenerationFinished = bDirtyNavDataPackageOnNavigationGenerationFinished;
		NavSysInstance->ConfigureAsStatic(bStrictlyStatic);
	}

	return NavSysInstance;
}

#if WITH_EDITOR
void UNavigationSystemModuleConfig::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName NAME_NavigationSystemClass = GET_MEMBER_NAME_CHECKED(UNavigationSystemConfig, NavigationSystemClass);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		FName PropName = PropertyChangedEvent.Property->GetFName();
		if (PropName == NAME_NavigationSystemClass)
		{
			if (NavigationSystemClass.IsValid() == false)
			{
				NavigationSystemClass = *GEngine->NavigationSystemClass;
			}
			else
			{
				NavigationSystemClass.TryLoad();
				TSubclassOf<UNavigationSystemBase> NavSysClass = NavigationSystemClass.ResolveClass();
				const UNavigationSystemV1* NavSysCDO = *NavSysClass
					? NavSysClass->GetDefaultObject<UNavigationSystemV1>()
					: (UNavigationSystemV1*)nullptr;
				if (NavSysCDO)
				{
					UpdateWithNavSysCDO(*NavSysCDO);
				}
			}
		}
	}
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
