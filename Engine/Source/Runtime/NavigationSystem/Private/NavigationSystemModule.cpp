// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationSystemModule.h"
#include "EngineDefines.h"
#include "AI/NavigationSystemBase.h"
#include "Misc/CoreDelegates.h"
#include "Templates/SubclassOf.h"
#include "Engine/World.h"
#include "NavLinkCustomInterface.h"
#include "NavMesh/NavMeshBoundsVolume.h"

#define LOCTEXT_NAMESPACE "NavigationSystem"

DEFINE_LOG_CATEGORY_STATIC(LogNavSysModule, Log, All);

class FNavigationSystemModule : public INavSysModule
{
	// Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
	//virtual UNavigationSystemBase* CreateNavigationSystemInstance(UWorld& World) override;
	// End IModuleInterface

private:
#if WITH_EDITOR 
	static FDelegateHandle OnPreWorldInitializationHandle;
	static FDelegateHandle OnPostEngineInitHandle;
#endif
};

IMPLEMENT_MODULE(FNavigationSystemModule, NavigationSystem)

#if WITH_EDITOR 
FDelegateHandle FNavigationSystemModule::OnPreWorldInitializationHandle;
FDelegateHandle FNavigationSystemModule::OnPostEngineInitHandle;
#endif

void FNavigationSystemModule::StartupModule()
{ 
	// mz@todo bind to all the delegates in FNavigationSystem
#if WITH_EDITOR 
	OnPreWorldInitializationHandle = FWorldDelegates::OnPreWorldInitialization.AddStatic(&INavLinkCustomInterface::OnPreWorldInitialization);
	OnPostEngineInitHandle = FCoreDelegates::GetOnPostEngineInit().AddStatic(&ANavMeshBoundsVolume::OnPostEngineInit);
#endif
}

void FNavigationSystemModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
#if WITH_EDITOR
	FCoreDelegates::GetOnPostEngineInit().Remove(OnPostEngineInitHandle);
	FWorldDelegates::OnPreWorldInitialization.Remove(OnPreWorldInitializationHandle);
#endif
}

//UNavigationSystemBase* FNavigationSystemModule::CreateNavigationSystemInstance(UWorld& World)
//{
//	UE_LOGF(LogNavSysModule, Log, "Creating NavigationSystem for world %ls", *World.GetName());
//	
//	/*TSubclassOf<UNavigationSystemBase> NavSystemClass = LoadClass<UNavigationSystemBase>(NULL, *UNavigationSystemBase::GetNavigationSystemClassName().ToString(), NULL, LOAD_None, NULL);
//	return NewObject<UNavigationSystemBase>(&World, NavSystemClass);*/
//	return nullptr;
//}

#undef LOCTEXT_NAMESPACE
