// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationSystemTypes.h"
#include "NavLinkCustomInterface.h"
#include "NavigationSystem.h"
#include "NavMesh/RecastGeometryExport.h"
#include "Engine/World.h"
#include "NavCollision.h"
#include "PhysicsEngine/BodySetup.h"
#include "Components/StaticMeshComponent.h"
#include "VisualLogger/VisualLogger.h"

namespace
{
	FORCEINLINE_DEBUGGABLE FVector::FReal RawGeometryFall(const AActor* Querier, const FVector& FallStart, const FVector::FReal FallLimit)
	{
		FVector::FReal FallDownHeight = 0.;

		UE_VLOG_SEGMENT(Querier, LogNavigation, Log, FallStart, FallStart + FVector(0, 0, -FallLimit)
			, FColor::Red, TEXT("TerrainTrace"));

		FCollisionQueryParams TraceParams(SCENE_QUERY_STAT(RawGeometryFall), true, Querier);
		FHitResult Hit;
		const bool bHit = Querier->GetWorld()->LineTraceSingleByChannel(Hit, FallStart, FallStart + FVector(0., 0., -FallLimit), ECC_WorldStatic, TraceParams);
		if (bHit)
		{
			UE_VLOG_LOCATION(Querier, LogNavigation, Log, Hit.Location, 15, FColor::Red, TEXT("%s")
				, *Hit.HitObjectHandle.GetName());

			if (Cast<UStaticMeshComponent>(Hit.Component.Get()))
			{
				const FVector Loc = Hit.ImpactPoint;
				FallDownHeight = FallStart.Z - Loc.Z;
			}
		}

		return FallDownHeight;
	}
}


namespace NavigationHelper
{
	void GatherCollision(UBodySetup* RigidBody, TNavStatArray<FVector>& OutVertexBuffer, TNavStatArray<int32>& OutIndexBuffer, const FTransform& LocalToWorld, FBox& OutBounds)
	{
		if (RigidBody == NULL)
		{
			return;
		}
#if WITH_RECAST
		FRecastGeometryExport::ExportRigidBodyGeometry(*RigidBody, OutVertexBuffer, OutIndexBuffer, OutBounds, LocalToWorld);
#endif // WITH_RECAST
	}

	void GatherCollision(UBodySetup* RigidBody, UNavCollision* NavCollision)
	{
		if (RigidBody == NULL || NavCollision == NULL)
		{
			return;
		}
#if WITH_RECAST
		FRecastGeometryExport::ExportRigidBodyGeometry(*RigidBody
			, NavCollision->GetMutableTriMeshCollision().VertexBuffer, NavCollision->GetMutableTriMeshCollision().IndexBuffer
			, NavCollision->GetMutableConvexCollision().VertexBuffer, NavCollision->GetMutableConvexCollision().IndexBuffer
			, NavCollision->ConvexShapeIndices
			, NavCollision->Bounds);
#endif // WITH_RECAST
	}

	void GatherCollision(const FKAggregateGeom& AggGeom, UNavCollision& NavCollision)
	{
#if WITH_RECAST
		FRecastGeometryExport::ExportAggregatedGeometry(
			AggGeom,
			NavCollision.GetMutableConvexCollision().VertexBuffer,
			 NavCollision.GetMutableConvexCollision().IndexBuffer,
			 NavCollision.ConvexShapeIndices,
			 NavCollision.Bounds);
#endif // WITH_RECAST
	}

	FNavLinkOwnerData::FNavLinkOwnerData(const AActor& InActor)
	{
		Actor = &InActor;
		LinkToWorld = InActor.GetActorTransform();
	}

	FNavLinkOwnerData::FNavLinkOwnerData(const USceneComponent& InComponent)
	{
		Actor = InComponent.GetOwner();
		LinkToWorld = InComponent.GetComponentTransform();
	}

	void DefaultNavLinkProcessorImpl(FCompositeNavModifier* OUT CompositeModifier, const FNavLinkOwnerData& OwnerData, const TArray<FNavigationLink>& IN NavLinks)
	{
		FSimpleLinkNavModifier SimpleLink(NavLinks, OwnerData.LinkToWorld);

		// adjust links
		for (int32 LinkIndex = 0; LinkIndex < SimpleLink.Links.Num(); ++LinkIndex)
		{
			FNavigationLink& Link = SimpleLink.Links[LinkIndex];

			// this one needs adjusting
			if (Link.Direction == ENavLinkDirection::RightToLeft)
			{
				Swap(Link.Left, Link.Right);
			}

			if (Link.MaxFallDownLength > 0)
			{
				const FVector WorldRight = OwnerData.LinkToWorld.TransformPosition(Link.Right);
				const FVector::FReal FallDownHeight = RawGeometryFall(OwnerData.Actor, WorldRight, Link.MaxFallDownLength);

				if (FallDownHeight > 0.)
				{
					// @todo maybe it's a good idea to clear ModifiedLink.MaxFallDownLength here
					UE_VLOG_SEGMENT(OwnerData.Actor, LogNavigation, Log, WorldRight, WorldRight + FVector(0, 0, -FallDownHeight), FColor::Green, TEXT("FallDownHeight %d"), LinkIndex);

					Link.Right.Z -= FallDownHeight;
				}
			}

			if (Link.LeftProjectHeight > 0)
			{
				const FVector WorldLeft = OwnerData.LinkToWorld.TransformPosition(Link.Left);
				const FVector::FReal FallDownHeight = RawGeometryFall(OwnerData.Actor, WorldLeft, Link.LeftProjectHeight);

				if (FallDownHeight > 0.)
				{
					// @todo maybe it's a good idea to clear ModifiedLink.LeftProjectHeight here
					UE_VLOG_SEGMENT(OwnerData.Actor, LogNavigation, Log, WorldLeft, WorldLeft + FVector(0, 0, -FallDownHeight), FColor::Green, TEXT("LeftProjectHeight %d"), LinkIndex);

					Link.Left.Z -= FallDownHeight;
				}
			}
		}

		CompositeModifier->Add(SimpleLink);
	}

	void DefaultNavLinkSegmentProcessorImpl(FCompositeNavModifier* OUT CompositeModifier, const FNavLinkOwnerData& OwnerData, const TArray<FNavigationSegmentLink>& IN NavLinks)
	{
		FSimpleLinkNavModifier SimpleLink(NavLinks, OwnerData.LinkToWorld);

		// adjust links if needed
		for (int32 LinkIndex = 0; LinkIndex < SimpleLink.SegmentLinks.Num(); ++LinkIndex)
		{
			FNavigationSegmentLink& Link = SimpleLink.SegmentLinks[LinkIndex];

			// this one needs adjusting
			if (Link.Direction == ENavLinkDirection::RightToLeft)
			{
				Swap(Link.LeftStart, Link.RightStart);
				Swap(Link.LeftEnd, Link.RightEnd);
			}

			if (Link.MaxFallDownLength > 0)
			{
				const FVector WorldRightStart = OwnerData.LinkToWorld.TransformPosition(Link.RightStart);
				const FVector WorldRightEnd = OwnerData.LinkToWorld.TransformPosition(Link.RightEnd);

				const FVector::FReal FallDownHeightStart = RawGeometryFall(OwnerData.Actor, WorldRightStart, Link.MaxFallDownLength);
				const FVector::FReal FallDownHeightEnd = RawGeometryFall(OwnerData.Actor, WorldRightEnd, Link.MaxFallDownLength);

				if (FallDownHeightStart > 0.)
				{
					// @todo maybe it's a good idea to clear ModifiedLink.MaxFallDownLength here
					UE_VLOG_SEGMENT(OwnerData.Actor, LogNavigation, Log, WorldRightStart, WorldRightStart + FVector(0, 0, -FallDownHeightStart), FColor::Green, TEXT("FallDownHeightStart %d"), LinkIndex);

					Link.RightStart.Z -= FallDownHeightStart;
				}
				if (FallDownHeightEnd > 0.)
				{
					// @todo maybe it's a good idea to clear ModifiedLink.MaxFallDownLength here
					UE_VLOG_SEGMENT(OwnerData.Actor, LogNavigation, Log, WorldRightEnd, WorldRightEnd + FVector(0, 0, -FallDownHeightEnd), FColor::Green, TEXT("FallDownHeightEnd %d"), LinkIndex);

					Link.RightEnd.Z -= FallDownHeightEnd;
				}
			}
		}

		CompositeModifier->Add(SimpleLink);
	}

	FNavLinkProcessorDataDelegate NavLinkProcessor = FNavLinkProcessorDataDelegate::CreateStatic(DefaultNavLinkProcessorImpl);
	FNavLinkSegmentProcessorDataDelegate NavLinkSegmentProcessor = FNavLinkSegmentProcessorDataDelegate::CreateStatic(DefaultNavLinkSegmentProcessorImpl);

	void ProcessNavLinkAndAppend(FCompositeNavModifier* OUT CompositeModifier, const AActor* Actor, const TArray<FNavigationLink>& IN NavLinks)
	{
		if (Actor)
		{
			ProcessNavLinkAndAppend(CompositeModifier, FNavLinkOwnerData(*Actor), NavLinks);
		}
	}

	void ProcessNavLinkAndAppend(FCompositeNavModifier* OUT CompositeModifier, const FNavLinkOwnerData& OwnerData, const TArray<FNavigationLink>& IN NavLinks)
	{
		SCOPE_CYCLE_COUNTER(STAT_Navigation_AdjustingNavLinks);

		if (NavLinks.Num())
		{
			check(NavLinkProcessor.IsBound());
			NavLinkProcessor.Execute(CompositeModifier, OwnerData, NavLinks);
		}
	}

	void ProcessNavLinkSegmentAndAppend(FCompositeNavModifier* OUT CompositeModifier, const AActor* Actor, const TArray<FNavigationSegmentLink>& IN NavLinks)
	{
		if (Actor)
		{
			ProcessNavLinkSegmentAndAppend(CompositeModifier, FNavLinkOwnerData(*Actor), NavLinks);
		}
	}

	void ProcessNavLinkSegmentAndAppend(FCompositeNavModifier* OUT CompositeModifier, const FNavLinkOwnerData& OwnerData, const TArray<FNavigationSegmentLink>& IN NavLinks)
	{
		SCOPE_CYCLE_COUNTER(STAT_Navigation_AdjustingNavLinks);

		if (NavLinks.Num())
		{
			check(NavLinkSegmentProcessor.IsBound());
			NavLinkSegmentProcessor.Execute(CompositeModifier, OwnerData, NavLinks);
		}
	}

	void SetNavLinkProcessorDelegate(const FNavLinkProcessorDataDelegate& NewDelegate)
	{
		check(NewDelegate.IsBound());
		NavLinkProcessor = NewDelegate;
	}

	void SetNavLinkSegmentProcessorDelegate(const FNavLinkSegmentProcessorDataDelegate& NewDelegate)
	{
		check(NewDelegate.IsBound());
		NavLinkSegmentProcessor = NewDelegate;
	}

	bool IsBodyNavigationRelevant(const UBodySetup& BodySetup)
	{
		const bool bBodyHasGeometry = (BodySetup.AggGeom.GetElementCount() > 0 || BodySetup.TriMeshGeometries.Num() > 0);

		// has any colliding geometry
		return bBodyHasGeometry
			// AND blocks any of Navigation-relevant 
			&& (BodySetup.DefaultInstance.GetResponseToChannel(ECC_Pawn) == ECR_Block || BodySetup.DefaultInstance.GetResponseToChannel(ECC_Vehicle) == ECR_Block)
			// AND has full colliding capabilities 
			&& BodySetup.DefaultInstance.GetCollisionEnabled() == ECollisionEnabled::QueryAndPhysics;
	}
}

#include "NavLinkHostInterface.h"
//----------------------------------------------------------------------//
// interfaces
//----------------------------------------------------------------------//
#include "NavigationPathGenerator.h"
UNavigationPathGenerator::UNavigationPathGenerator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

uint32 INavLinkCustomInterface::NextUniqueId = 1;

UNavLinkHostInterface::UNavLinkHostInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UNavLinkCustomInterface::UNavLinkCustomInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UObject* INavLinkCustomInterface::GetLinkOwner() const
{
	return Cast<UObject>((INavLinkCustomInterface*)this);
}

uint32 INavLinkCustomInterface::GetUniqueId()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_LOGF(LogNavLink, VeryVerbose, "%s id: %u.", __FUNCTION__, NextUniqueId);
	return NextUniqueId++;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void INavLinkCustomInterface::UpdateUniqueId(FNavLinkId AlreadyUsedId)
{
	// Only update NextUniqueId for old style incremental Ids.
	if (AlreadyUsedId.IsLegacyId())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE_CLOGF(AlreadyUsedId.GetId() + 1 > NextUniqueId, LogNavLink, VeryVerbose, "%s, updating NextUniqueId to: %llu.", __FUNCTION__, AlreadyUsedId.GetId() + 1)
		const uint64 NextId = FMath::Max((uint64)NextUniqueId, AlreadyUsedId.GetId() + 1);
		ensureMsgf(NextId <= TNumericLimits<uint32>::Max(), TEXT("Overflowing uint32 using legacy nav link id system!"));

		NextUniqueId = (uint32)NextId;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

FNavigationLink INavLinkCustomInterface::GetModifier(const INavLinkCustomInterface* CustomNavLink)
{
	FNavigationLink LinkMod;
	LinkMod.SetAreaClass(CustomNavLink->GetLinkAreaClass());
	LinkMod.NavLinkId = CustomNavLink->GetId();

	ENavLinkDirection::Type LinkDirection = ENavLinkDirection::BothWays;
	CustomNavLink->GetLinkData(LinkMod.Left, LinkMod.Right, LinkDirection);
	CustomNavLink->GetSupportedAgents(LinkMod.SupportedAgents);
	LinkMod.Direction = LinkDirection;

	return LinkMod;
}

void INavLinkCustomInterface::OnPreWorldInitialization(UWorld* World, const FWorldInitializationValues IVS)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ResetUniqueId();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void INavLinkCustomInterface::ResetUniqueId()
{
	UE_LOGF(LogNavLink, VeryVerbose, "Reset navlink id.");
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	NextUniqueId = 1;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
