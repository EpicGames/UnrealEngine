// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationTestingActor.h"
#include "NavigationSystem.h"
#include "Engine/CollisionProfile.h"
#include "EngineUtils.h"
#if WITH_EDITOR
#include "ObjectEditorUtils.h"
#endif // WITH_EDITOR
#include "NavMesh/NavTestRenderingComponent.h"
#include "NavigationInvokerComponent.h"
#include "NavMesh/RecastNavMesh.h"
#include "Components/CapsuleComponent.h"
#include "NavigationData.h"
#include "NavFilters/NavigationQueryFilter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationTestingActor)

void FNavTestTickHelper::Tick(float DeltaTime)
{
#if WITH_EDITOR
	if (Owner.IsValid())
	{
		Owner->TickMe();
	}
#endif // WITH_EDITOR
}

TStatId FNavTestTickHelper::GetStatId() const 
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FNavTestTickHelper, STATGROUP_Tickables);
}

ANavigationTestingActor::ANavigationTestingActor(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	EdRenderComp = CreateDefaultSubobject<UNavTestRenderingComponent>(TEXT("EdRenderComp"));

#if WITH_RECAST
	TickHelper = NULL;
#endif // WITH_RECAST
#endif // WITH_EDITORONLY_DATA

	bIsEditorOnlyActor = true;
	NavAgentProps.AgentRadius = 34.f;
	NavAgentProps.AgentHeight = 144.f;
	CostLimitFactor = FLT_MAX;
	MinimumCostLimit = 0.f;
	ShowStepIndex = -1;
	bShowNodePool = true;
	bShowBestPath = true;
	bShowDiffWithPreviousStep = false;
	bShouldBeVisibleInGame = false;
	TextCanvasOffset = FVector2D::ZeroVector;
	bGatherDetailedInfo = true;
	bDrawDistanceToWall = false;
	ClosestWallLocation = FNavigationSystem::InvalidLocation;
	RaycastHitLocation = FNavigationSystem::InvalidLocation;
	bNavDataIsReadyInRadius = false;
	bNavDataIsReadyToQueryTargetActor = false;
	bRaycastToQueryTargetActorResult = false;
	bRaycastToQueryTargetEndsInCorridor = false;
	OffsetFromCornersDistance = 0.f;

	QueryingExtent = FVector(DEFAULT_NAV_QUERY_EXTENT_HORIZONTAL, DEFAULT_NAV_QUERY_EXTENT_HORIZONTAL, DEFAULT_NAV_QUERY_EXTENT_VERTICAL);
	bRequireNavigableEndLocation = true;

	CapsuleComponent = CreateDefaultSubobject<UCapsuleComponent>(TEXT("CollisionCylinder"));
	CapsuleComponent->InitCapsuleSize(NavAgentProps.AgentRadius, NavAgentProps.AgentHeight / 2);
	CapsuleComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	CapsuleComponent->CanCharacterStepUpOn = ECB_No;
	CapsuleComponent->SetShouldUpdatePhysicsVolume(true);
	CapsuleComponent->SetCanEverAffectNavigation(false);

	RootComponent = CapsuleComponent;

	InvokerComponent = CreateDefaultSubobject<UNavigationInvokerComponent>(TEXT("InvokerComponent"));
	InvokerComponent->bAutoActivate = bActAsNavigationInvoker;

	PathObserver = FNavigationPath::FPathObserverDelegate::FDelegate::CreateUObject(this, &ANavigationTestingActor::OnPathEvent);
}

ANavigationTestingActor::~ANavigationTestingActor()
{
#if WITH_RECAST && WITH_EDITORONLY_DATA
	delete TickHelper;
#endif // WITH_RECAST && WITH_EDITORONLY_DATA
}

void ANavigationTestingActor::BeginDestroy()
{
	LastPath.Reset();
	if (OtherActor && OtherActor->OtherActor == this)
	{
		OtherActor->OtherActor = NULL;
		OtherActor->LastPath.Reset();
	}
	if (QueryTargetActor)
	{
		if (USceneComponent* RootComp = QueryTargetActor->GetRootComponent())
		{
			RootComp->TransformUpdated.RemoveAll(this);
		}
	}
	Super::BeginDestroy();
}

#if WITH_EDITOR
void ANavigationTestingActor::PreEditChange(FProperty* PropertyThatWillChange)
{
	static const FName NAME_OtherActor = GET_MEMBER_NAME_CHECKED(ANavigationTestingActor, OtherActor);
	static const FName NAME_QueryTargetActor = GET_MEMBER_NAME_CHECKED(ANavigationTestingActor, QueryTargetActor);

	if (PropertyThatWillChange)
	{
		const FName ChangedPropName = PropertyThatWillChange->GetFName();
		if (ChangedPropName == NAME_OtherActor && OtherActor && OtherActor->OtherActor == this)
		{
			OtherActor->OtherActor = NULL;
			OtherActor->LastPath.Reset();
			LastPath.Reset();
	#if WITH_EDITORONLY_DATA
			OtherActor->EdRenderComp->MarkRenderStateDirty();
			EdRenderComp->MarkRenderStateDirty();
	#endif
		}
		else if (ChangedPropName == NAME_QueryTargetActor && QueryTargetActor)
		{
			if (USceneComponent* RootComp = QueryTargetActor->GetRootComponent())
			{
				RootComp->TransformUpdated.RemoveAll(this);
			}
		}
	}

	Super::PreEditChange(PropertyThatWillChange);
}

void ANavigationTestingActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	static const FName NAME_ShouldBeVisibleInGame = GET_MEMBER_NAME_CHECKED(ANavigationTestingActor, bShouldBeVisibleInGame);
	static const FName NAME_OtherActor = GET_MEMBER_NAME_CHECKED(ANavigationTestingActor, OtherActor);
	static const FName NAME_QueryTargetActor = GET_MEMBER_NAME_CHECKED(ANavigationTestingActor, QueryTargetActor);
	static const FName NAME_IsSearchStart = GET_MEMBER_NAME_CHECKED(ANavigationTestingActor, bSearchStart);
	static const FName NAME_InvokerComponent = GET_MEMBER_NAME_CHECKED(ANavigationTestingActor, InvokerComponent);
	static const FName NAME_FilterClass = GET_MEMBER_NAME_CHECKED(ANavigationTestingActor, FilterClass);

	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		const FName ChangedPropName = PropertyChangedEvent.Property->GetFName();
		const FName ChangedCategory = FObjectEditorUtils::GetCategoryFName(PropertyChangedEvent.Property);

		if (ChangedPropName == GET_MEMBER_NAME_CHECKED(FNavAgentProperties,AgentRadius) ||
			ChangedPropName == GET_MEMBER_NAME_CHECKED(FNavAgentProperties,AgentHeight) ||
			ChangedPropName == GET_MEMBER_NAME_CHECKED(FNavAgentProperties,PreferredNavData))
		{
			MyNavData = NULL;
			UpdateNavData();

			CapsuleComponent->SetCapsuleSize(NavAgentProps.AgentRadius, NavAgentProps.AgentHeight/2);
		}
		else if (ChangedPropName == GET_MEMBER_NAME_CHECKED(ANavigationTestingActor,QueryingExtent))
		{
			UpdateNavData();

			UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
			if (NavSys)
			{
				FNavLocation NavLoc;
				bProjectedLocationValid = NavSys->ProjectPointToNavigation(GetActorLocation(), NavLoc, QueryingExtent, MyNavData);
				ProjectedLocation = NavLoc.Location;
			}
		}
		else if (ChangedPropName == NAME_ShouldBeVisibleInGame)
		{
			SetHidden(!bShouldBeVisibleInGame);
		}
		else if (ChangedCategory == TEXT("Debug"))
		{
#if WITH_EDITORONLY_DATA
			EdRenderComp->MarkRenderStateDirty();
#endif
		}
		else if (ChangedCategory == TEXT("Query"))
		{
			if (ChangedPropName == NAME_QueryTargetActor && QueryTargetActor)
			{
				if (USceneComponent* RootComp = QueryTargetActor->GetRootComponent())
				{
					RootComp->TransformUpdated.AddUObject(this, &ANavigationTestingActor::OnQueryTargetActorTransformUpdated);
				}
			}

			UpdateLocalQueries();
			UpdateTargetActorQueries();

			if (ChangedPropName == NAME_FilterClass)
			{
				UpdatePathfinding();
			}
		}
		else if (ChangedCategory == TEXT("Pathfinding"))
		{
			if (ChangedPropName == NAME_OtherActor)
			{
				if (OtherActor != NULL)
				{
					ANavigationTestingActor* OtherActorsOldOtherActor = OtherActor->OtherActor;

					OtherActor->OtherActor = this;
					bSearchStart = !OtherActor->bSearchStart;

#if WITH_EDITORONLY_DATA
					if (bSearchStart)
					{
						OtherActor->EdRenderComp->MarkRenderStateDirty();
					}
					else
					{
						EdRenderComp->MarkRenderStateDirty();
					}
#endif

					if (OtherActorsOldOtherActor)
					{
						OtherActorsOldOtherActor->OtherActor = NULL;
						OtherActorsOldOtherActor->LastPath.Reset();
#if WITH_EDITORONLY_DATA
						OtherActorsOldOtherActor->EdRenderComp->MarkRenderStateDirty();
#endif
					}
				}
			}
			else if (ChangedPropName == NAME_IsSearchStart)
			{
				if (OtherActor != NULL)
				{
					OtherActor->bSearchStart = !bSearchStart;
					OtherActor->LastPath.Reset();
					if (OtherActor->EdRenderComp)
					{
						OtherActor->EdRenderComp->MarkRenderStateDirty();
					}
				}

				LastPath.Reset();
				if (EdRenderComp)
				{
					EdRenderComp->MarkRenderStateDirty();
				}
			}

			UpdatePathfinding();
		}
		else if (NAME_InvokerComponent == ChangedPropName)
		{
			InvokerComponent->SetActive(bActAsNavigationInvoker);
		}
	}
}

void ANavigationTestingActor::PostEditMove(bool bFinished)
{
	Super::PostEditMove(bFinished);

	// project location to navmesh
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	if (NavSys)
	{
		FNavLocation NavLoc;
		bProjectedLocationValid = NavSys->ProjectPointToNavigation(GetActorLocation(), NavLoc, QueryingExtent, MyNavData);
		ProjectedLocation = NavLoc.Location;

		if (bSearchStart || (OtherActor != NULL && OtherActor->bSearchStart))
		{
			UpdatePathfinding();
		}

		UpdateLocalQueries();
		UpdateTargetActorQueries();
	}
}

void ANavigationTestingActor::PostLoad()
{
	Super::PostLoad();

	InvokerComponent->bAutoActivate = bActAsNavigationInvoker;

	if (QueryTargetActor)
	{
		if (USceneComponent* RootComp = QueryTargetActor->GetRootComponent())
		{
			RootComp->TransformUpdated.AddUObject(this, &ANavigationTestingActor::OnQueryTargetActorTransformUpdated);
		}
	}

#if WITH_RECAST && WITH_EDITORONLY_DATA
	if (GIsEditor)
	{
		TickHelper = new FNavTestTickHelper();
		TickHelper->Owner = this;
	}
#endif

	SetHidden(!bShouldBeVisibleInGame);
}

void ANavigationTestingActor::TickMe()
{
	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	if (NavSys && !NavSys->IsNavigationBuildInProgress())
	{
#if WITH_RECAST && WITH_EDITORONLY_DATA
		delete TickHelper;
		TickHelper = NULL;
#endif

		UpdatePathfinding();
	}
}

#endif // WITH_EDITOR

FVector ANavigationTestingActor::GetNavAgentLocation() const
{ 
	return GetActorLocation(); 
}

void ANavigationTestingActor::UpdateNavData()
{
	if (!MyNavData && GetWorld() && GetWorld()->GetNavigationSystem())
	{
		UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
		if (NavSys)
		{
			MyNavData = NavSys->GetNavDataForProps(NavAgentProps, GetActorLocation());
		}
	}
}

void ANavigationTestingActor::UpdatePathfinding()
{
	PathfindingTime = 0.0f;
	PathCost = 0.;
	bPathSearchOutOfNodes = false;
	bPathIsPartial = false;
	bPathExist = false;
	LastPath.Reset();
	PathfindingSteps = 0;
#if WITH_RECAST && WITH_EDITORONLY_DATA
	DebugSteps.Reset();
#endif
	UpdateNavData();

	if (bSearchStart == false && (OtherActor == NULL || OtherActor->bSearchStart == false))
	{
#if WITH_EDITORONLY_DATA
		if (EdRenderComp)
		{
			EdRenderComp->MarkRenderStateDirty();
		}
#endif // WITH_EDITORONLY_DATA
		return;
	}

	if (OtherActor == NULL)
	{
		ANavigationTestingActor* AlternativeOtherActor = NULL;

		for (TActorIterator<ANavigationTestingActor> It(GetWorld()); It; ++It)
		{
			ANavigationTestingActor* TestActor = *It;
			if (TestActor && TestActor != this)
			{
				if( TestActor->OtherActor == this )
				{
					OtherActor = TestActor;
					break;
				}
				// the other one doesn't have anything set - potential end for us
				else if (bSearchStart && TestActor->OtherActor == NULL)
				{
					AlternativeOtherActor = TestActor;
				}
			}
		}

		// if still empty maybe AlternativeOtherActor can fill in
		if (OtherActor == NULL && AlternativeOtherActor != NULL)
		{
			OtherActor = AlternativeOtherActor;
			AlternativeOtherActor->OtherActor = this;
		}
	}

	if (OtherActor)
	{
		if (bSearchStart)
		{
			SearchPathTo(OtherActor);
		}
		else if (OtherActor)
		{
			OtherActor->SearchPathTo(this);
		}
	}
}

void ANavigationTestingActor::UpdateLocalQueries()
{
	if (bDrawDistanceToWall)
	{
		ClosestWallLocation = FindClosestWallLocation();
	}
	if (bDrawIfNavDataIsReadyInRadius)
	{
		bNavDataIsReadyInRadius = CheckIfNavDataIsReadyInRadius();
	}
}

void ANavigationTestingActor::UpdateTargetActorQueries()
{
	if (bDrawIfNavDataIsReadyToQueryTargetActor)
	{
		bNavDataIsReadyToQueryTargetActor = CheckIfNavDataIsReadyToActor(QueryTargetActor);
	}
	if (bDrawRaycastToQueryTargetActor)
	{
		bRaycastToQueryTargetActorResult = CheckRaycastToActor(QueryTargetActor, RaycastHitLocation, bRaycastToQueryTargetEndsInCorridor);
	}
}

FVector ANavigationTestingActor::FindClosestWallLocation() const
{
#if WITH_EDITORONLY_DATA
	if (EdRenderComp)
	{
		EdRenderComp->MarkRenderStateDirty();
	}
#endif // WITH_EDITORONLY_DATA

#if WITH_RECAST
	ARecastNavMesh* AsRecastNavMesh = Cast<ARecastNavMesh>(MyNavData);
	if (AsRecastNavMesh)
	{
		FVector TmpOutLocation = FNavigationSystem::InvalidLocation;
		AsRecastNavMesh->FindDistanceToWall(GetActorLocation(), UNavigationQueryFilter::GetQueryFilter(*MyNavData, this, FilterClass), FLT_MAX, &TmpOutLocation);
		return TmpOutLocation;
	}
#endif // WITH_RECAST
	
	return FNavigationSystem::InvalidLocation;
}

bool ANavigationTestingActor::CheckIfNavDataIsReadyInRadius()
{
#if WITH_EDITORONLY_DATA
	if (EdRenderComp)
	{
		EdRenderComp->MarkRenderStateDirty();
	}
#endif // WITH_EDITORONLY_DATA
	
#if WITH_RECAST
	UpdateNavData();
	const ARecastNavMesh* RecastNavMesh = Cast<ARecastNavMesh>(MyNavData);
	if (RecastNavMesh)
	{
		return RecastNavMesh->HasCompleteDataInRadius(GetActorLocation(), RadiusUsedToValidateNavData);
	}
#endif // WITH_RECAST
	
	return false;
}

bool ANavigationTestingActor::CheckIfNavDataIsReadyToActor(const AActor* TargetActor)
{
#if WITH_EDITORONLY_DATA
	if (EdRenderComp)
	{
		EdRenderComp->MarkRenderStateDirty();
	}
#endif // WITH_EDITORONLY_DATA

#if WITH_RECAST
	UpdateNavData();
	const ARecastNavMesh* RecastNavMesh = Cast<ARecastNavMesh>(MyNavData);
	if (RecastNavMesh && TargetActor)
	{
		return RecastNavMesh->HasCompleteDataAroundSegment(GetActorLocation(), TargetActor->GetActorLocation(), RadiusUsedToValidateNavData);
	}
#endif // WITH_RECAST

	return false;
}

bool ANavigationTestingActor::CheckRaycastToActor(const AActor* TargetActor, FVector& OutHitLocation, bool& bOutIsRaycastEndInCorridor)
{
	OutHitLocation = FNavigationSystem::InvalidLocation;
#if WITH_EDITORONLY_DATA
	if (EdRenderComp)
	{
		EdRenderComp->MarkRenderStateDirty();
	}
#endif // WITH_EDITORONLY_DATA

	UpdateNavData();
	if (MyNavData && TargetActor)
	{
		FSharedConstNavQueryFilter Filter = UNavigationQueryFilter::GetQueryFilter(*MyNavData, this, FilterClass);
		FNavigationRaycastAdditionalResults AdditionalResults;
		const bool bDidHit = MyNavData->Raycast(GetActorLocation(), TargetActor->GetActorLocation(), OutHitLocation, &AdditionalResults, Filter, this);
		bOutIsRaycastEndInCorridor = AdditionalResults.bIsRayEndInCorridor;
		return bDidHit;
	}

	return false;
}

void ANavigationTestingActor::OnQueryTargetActorTransformUpdated(USceneComponent* InRootComponent, EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	UpdateTargetActorQueries();
}

void ANavigationTestingActor::SearchPathTo(ANavigationTestingActor* Goal)
{
#if WITH_EDITORONLY_DATA
	if (EdRenderComp)
	{
		EdRenderComp->MarkRenderStateDirty();
	}
#endif // WITH_EDITORONLY_DATA

	if (Goal == NULL) 
	{
		return;
	}

	UNavigationSystemV1* NavSys = FNavigationSystem::GetCurrent<UNavigationSystemV1>(GetWorld());
	if (NavSys == nullptr)
	{
		return;
	}
	ANavigationData* NavData = Cast<ANavigationData>(NavSys->GetNavDataForActor(*this));
	if (NavData == nullptr)
	{
		return;
	}

	const double StartTime = FPlatformTime::Seconds();

	FPathFindingQuery Query = BuildPathFindingQuery(Goal);

	FSharedConstNavQueryFilter NavQueryFilter = Query.QueryFilter ? Query.QueryFilter : NavData->GetDefaultQueryFilter();
	if (NavQueryFilter->GetImplementation() == nullptr)
	{
		return;
	}

	if (bBacktracking)
	{
		FSharedNavQueryFilter NavigationFilterCopy = NavQueryFilter->GetCopy();
		NavigationFilterCopy->SetBacktrackingEnabled(true);
		Query.QueryFilter = NavigationFilterCopy;
		NavQueryFilter = NavigationFilterCopy;
	}

	//Apply cost limit factor
	const float HeuristicScale = NavQueryFilter->GetHeuristicScale();
	Query.CostLimit = FPathFindingQuery::ComputeCostLimitFromHeuristic(Query.StartLocation, Query.EndLocation, HeuristicScale, CostLimitFactor, MinimumCostLimit);

	EPathFindingMode::Type Mode = bUseHierarchicalPathfinding ? EPathFindingMode::Hierarchical : EPathFindingMode::Regular;
	FPathFindingResult Result = NavSys->FindPathSync(NavAgentProps, Query, Mode);

	const double EndTime = FPlatformTime::Seconds();
	const double Duration = (EndTime - StartTime);
	PathfindingTime = static_cast<float>(Duration * 1000000.);			// in micro seconds [us]
	bPathIsPartial = Result.IsPartial();
	bPathExist = Result.IsSuccessful();
	bPathSearchOutOfNodes = bPathExist ? Result.Path->DidSearchReachedLimit() : false;
	LastPath = Result.Path;
	PathCost = bPathExist ? Result.Path->GetCost() : 0.;

	if (bPathExist)
	{
		LastPath->AddObserver(PathObserver);

		if (OffsetFromCornersDistance > 0.0f)
		{
			((FNavMeshPath*)LastPath.Get())->OffsetFromCorners(OffsetFromCornersDistance);
		}
	}

#if WITH_RECAST && WITH_EDITORONLY_DATA
	if (bGatherDetailedInfo && !bUseHierarchicalPathfinding)
	{
		ARecastNavMesh* RecastNavMesh = Cast<ARecastNavMesh>(MyNavData);
		if (RecastNavMesh && RecastNavMesh->HasValidNavmesh())
		{
			PathfindingSteps = RecastNavMesh->DebugPathfinding(Query, DebugSteps);
		}
	}
#endif
}

void ANavigationTestingActor::OnPathEvent(FNavigationPath* InvalidatedPath, ENavPathEvent::Type Event)
{
	if (InvalidatedPath == NULL || InvalidatedPath != LastPath.Get())
    {
		return;
    }

	switch (Event)
	{
		case ENavPathEvent::Invalidated:
		{
			UpdatePathfinding();
			break;
		}
		case ENavPathEvent::RePathFailed:
			break;
		case ENavPathEvent::UpdatedDueToGoalMoved:
		case ENavPathEvent::UpdatedDueToNavigationChanged:
		{
		}
		break;
	}
}

FPathFindingQuery ANavigationTestingActor::BuildPathFindingQuery(const ANavigationTestingActor* Goal) const
{
	check(Goal);
	if (MyNavData)
	{
		constexpr float DefaultCostLimit = FLT_MAX;
		const FNavPathSharedPtr NoSharedPath = nullptr;
		return FPathFindingQuery(this, *MyNavData, GetNavAgentLocation(), Goal->GetNavAgentLocation(), UNavigationQueryFilter::GetQueryFilter(*MyNavData, this, FilterClass), NoSharedPath, DefaultCostLimit, bRequireNavigableEndLocation);
	}
	
	return FPathFindingQuery();
}


