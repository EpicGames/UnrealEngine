// Copyright Epic Games, Inc. All Rights Reserved.

#include "SplineNavModifierComponent.h"

#include "AI/NavigationSystemBase.h"
#include "AI/Navigation/NavigationRelevantData.h"
#include "Components/SplineComponent.h"
#include "Curves/BezierUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SplineNavModifierComponent)

namespace
{
	// Subdivide the spline into linear segments, adapting to its curvature (more curvy means more linear segments)
	void SubdivideSpline(TArray<FVector>& OutSubdivisions, const USplineComponent& Spline, const float SubdivisionThreshold)
	{
		// Sample at least 2 points
		const int32 NumSplinePoints = FMath::Max(Spline.GetNumberOfSplinePoints(), 2);

		// The USplineComponent's Hermite spline tangents are 3 times larger than Bezier tangents and we need to convert before tessellation
		constexpr double HermiteToBezierFactor = 3.0;

		// Tessellate the spline segments
		int32 PrevIndex = Spline.IsClosedLoop() ? (NumSplinePoints - 1) : INDEX_NONE;
		for (int32 SplinePointIndex = 0; SplinePointIndex < NumSplinePoints; SplinePointIndex++)
		{
			if (PrevIndex >= 0)
			{
				const FSplinePoint PrevSplinePoint = Spline.GetSplinePointAt(PrevIndex, ESplineCoordinateSpace::Local);
				const FSplinePoint CurrSplinePoint = Spline.GetSplinePointAt(SplinePointIndex, ESplineCoordinateSpace::Local);

				// The first point of the segment is appended before tessellation since UE::CubicBezier::Tessellate does not add it
				OutSubdivisions.Add(PrevSplinePoint.Position);

				// Convert this segment of the spline from Hermite to Bezier and subdivide it 
				UE::CubicBezier::Tessellate(OutSubdivisions,
					PrevSplinePoint.Position,
					PrevSplinePoint.Position + PrevSplinePoint.LeaveTangent / HermiteToBezierFactor,
					CurrSplinePoint.Position - CurrSplinePoint.ArriveTangent / HermiteToBezierFactor,
					CurrSplinePoint.Position,
					SubdivisionThreshold);
			}

			PrevIndex = SplinePointIndex;
		}
	}
}

void USplineNavModifierComponent::CalculateBounds() const
{
	Bounds = FBox(ForceInit);

	if (const USplineComponent* Spline = Cast<USplineComponent>(AttachedSpline.GetComponent(GetOwner())))
	{
		// The largest stroke length is used to expand the bounds
		const double Buffer = FMath::Max(StrokeWidth / 2.0, StrokeHeight / 2.0);
		Bounds = Spline->CalcBounds(SplineTransform).GetBox().ExpandBy(Buffer);
	}
}

void USplineNavModifierComponent::GetNavigationData(FNavigationRelevantData& Data) const
{
	const USplineComponent* Spline = Cast<USplineComponent>(AttachedSpline.GetComponent(GetOwner()));
	if (!Spline)
	{
		return;
	}

	// Build a rectangle in the YZ plane used to sample the spline at each cross section
	constexpr int32 NumCrossSectionVertices = 4;
	const double StrokeHalfWidth = StrokeWidth / 2.0;
	const double StrokeHalfHeight = StrokeHeight / 2.0;
	TStaticArray<FVector, NumCrossSectionVertices> CrossSectionRect;
	CrossSectionRect[0] = FVector(0.0, -StrokeHalfWidth, -StrokeHalfHeight);
	CrossSectionRect[1] = FVector(0.0,  StrokeHalfWidth, -StrokeHalfHeight);
	CrossSectionRect[2] = FVector(0.0,  StrokeHalfWidth,  StrokeHalfHeight);
	CrossSectionRect[3] = FVector(0.0, -StrokeHalfWidth,  StrokeHalfHeight);

	// Vertices (in an arbitrary order) of a prism which will enclose each segment of the spline
	TStaticArray<FVector, NumCrossSectionVertices * 2> Tube;

	// Subdivide the spline so that high curvature sections get smaller and more linear segments than straighter sections
	TArray<FVector> Subdivisions;
	SubdivideSpline(Subdivisions, *Spline, GetSubdivisionThreshold());
	const int32 NumSubdivisions = Subdivisions.Num();

	// Create volumes from the spline subdivisions and use them to mark the nav mesh with the given are
	const FTransform ComponentTransform = Spline->GetComponentTransform();
	int32 PrevIndex = 0;
	for (int32 SubdivisionIndex = 1; SubdivisionIndex < NumSubdivisions; SubdivisionIndex++)
	{
		// Compute the rotation of this tube segment
		const double TubeAngle = (Subdivisions[SubdivisionIndex] - Subdivisions[PrevIndex]).HeadingAngle();
		const FQuat TubeRotation(FVector::UnitZ(), TubeAngle);

		// Compute the vertices of this tube segment
		for (int i = 0; i < NumCrossSectionVertices; i++)
		{
			// For each vertex of the tube segment, first rotate about the positive Z axis, then translate to the subdivision point
			Tube[i] = (TubeRotation * CrossSectionRect[i]) + Subdivisions[PrevIndex];
			Tube[i + NumCrossSectionVertices] = (TubeRotation * CrossSectionRect[i]) + Subdivisions[SubdivisionIndex];
		}

		// From the tube construct a convex hull whose volume will be used to mark the nav mesh with the selected AreaClass
		FAreaNavModifier NavModifier(Tube, ENavigationCoordSystem::Type::Unreal, ComponentTransform, AreaClass);
		if (AreaClassToReplace)
		{
			NavModifier.SetAreaClassToReplace(AreaClassToReplace);
		}
		Data.Modifiers.Add(NavModifier);

		PrevIndex = SubdivisionIndex;
	}
}

USplineNavModifierComponent::USplineNavModifierComponent(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	// Should tick in the editor in order to track whether the spline has updated
	bTickInEditor = true;
	PrimaryComponentTick.bCanEverTick = true;

	// If a spline is already attached, store its update-checking data
	if (const USplineComponent* Spline = Cast<USplineComponent>(AttachedSpline.GetComponent(GetOwner())))
	{
		SplineVersion = Spline->GetVersion();
		SplineTransform = Spline->GetComponentTransform();
	}
#endif // WITH_EDITORONLY_DATA
}

void USplineNavModifierComponent::UpdateNavigationWithComponentData()
{
#if WITH_EDITORONLY_DATA
	CalculateBounds();
	FNavigationSystem::UpdateComponentData(*this);
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITORONLY_DATA
bool USplineNavModifierComponent::IsComponentTickEnabled() const
{
	const UWorld* World = GetWorld();
	return World && !World->IsGameWorld();
}

void USplineNavModifierComponent::TickComponent(const float DeltaTime, const ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (const USplineComponent* Spline = Cast<USplineComponent>(AttachedSpline.GetComponent(GetOwner())))
	{
		// Update spline data, and if anything changed then update nav data 
		if (SplineVersion != INVALID_SPLINE_VERSION)
		{
			bool bRequiresNavigationUpdate = false;
	
			const uint32 NextVersion = Spline->GetVersion();
			if (SplineVersion != NextVersion)
			{
				SplineVersion = NextVersion;
				bRequiresNavigationUpdate = true;
			}

			const FTransform& NextTransform = Spline->GetComponentTransform();
			if (!SplineTransform.Equals(NextTransform))
			{
				SplineTransform = NextTransform;
				bRequiresNavigationUpdate = true;
			}

			// This can be expensive (i.e. updating every tick as the user drags a spline point), so only update nav data if the editor flag is set
			if (bRequiresNavigationUpdate && bUpdateNavDataOnSplineChange)
			{
				UpdateNavigationWithComponentData();
			}
		}
		else
		{
			// The spline just became valid; store its data and use it to update nav data
			SplineVersion = Spline->GetVersion();
			SplineTransform = Spline->GetComponentTransform();

			UpdateNavigationWithComponentData();
		}
	}
	else if (SplineVersion != INVALID_SPLINE_VERSION)
	{
		// The spline just became invalid; reset the version and recompute nav data without the spline
		SplineVersion = INVALID_SPLINE_VERSION;
		UpdateNavigationWithComponentData();
	}
}

#endif // WITH_EDITORONLY_DATA

float USplineNavModifierComponent::GetSubdivisionThreshold() const
{
	switch (SubdivisionLOD)
	{
	case ESubdivisionLOD::Ultra:
		return 10.0f;
	case ESubdivisionLOD::High:
		return 100.0f;
	case ESubdivisionLOD::Medium:
		return 250.0f;
	case ESubdivisionLOD::Low:
	default: // Fallthrough
		return 500.0f;
	}
}