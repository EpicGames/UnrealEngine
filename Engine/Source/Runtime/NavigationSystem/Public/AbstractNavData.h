// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AI/Navigation/NavQueryFilter.h"
#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "Math/Box.h"
#include "Math/Vector.h"
#include "NavigationData.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "AbstractNavData.generated.h"

class UClass;
class UObject;
struct FPathFindingQuery;

struct FAbstractNavigationPath : public FNavigationPath
{
	typedef FNavigationPath Super;

	NAVIGATIONSYSTEM_API FAbstractNavigationPath();

	static NAVIGATIONSYSTEM_API const FNavPathType Type;
};

class FAbstractQueryFilter : public INavigationQueryFilterInterface
{
public:
	virtual void Reset() override {}
	virtual void SetAreaCost(uint8 AreaType, float Cost) override {}
	virtual void SetFixedAreaEnteringCost(uint8 AreaType, float Cost) override {}
	virtual void SetExcludedArea(uint8 AreaType) override {}
	virtual void SetAllAreaCosts(const float* CostArray, const int32 Count) override {}
	virtual void GetAllAreaCosts(float* CostArray, float* FixedCostArray, const int32 Count) const override {}
	virtual void SetBacktrackingEnabled(const bool bBacktracking) override {}
	virtual bool IsBacktrackingEnabled() const override { return false; }
	virtual float GetHeuristicScale() const override { return 1.f; }
	virtual bool IsEqual(const INavigationQueryFilterInterface* Other) const override { return true; }
	virtual void SetIncludeFlags(uint16 Flags) override {}
	virtual uint16 GetIncludeFlags() const override { return 0; }
	virtual void SetExcludeFlags(uint16 Flags) override {}
	virtual uint16 GetExcludeFlags() const override { return 0; }
	virtual FVector GetAdjustedEndLocation(const FVector& EndLocation) const override { return EndLocation; }
	NAVIGATIONSYSTEM_API virtual INavigationQueryFilterInterface* CreateCopy() const override;
};

UCLASS(MinimalAPI)
class AAbstractNavData : public ANavigationData
{
	GENERATED_BODY()

public:
	NAVIGATIONSYSTEM_API AAbstractNavData(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	NAVIGATIONSYSTEM_API virtual void PostLoad() override;

#if WITH_EDITOR
	// Begin AActor overrides
	virtual bool SupportsExternalPackaging() const override { return false; }
	// End AActor overrides
#endif

	// Begin ANavigationData overrides
	virtual void BatchRaycast(TArray<FNavigationRaycastWork>& Workload, FSharedConstNavQueryFilter QueryFilter, const UObject* Querier = NULL) const override {};
	virtual bool FindMoveAlongSurface(const FNavLocation& StartLocation, const FVector& TargetPosition, FNavLocation& OutLocation, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override { return false;  };
	virtual void BatchFindMoveAlongSurface(TArray<FNavigationFindMoveAlongSurfaceWork>& Workload, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override {};
	virtual bool FindOverlappingEdges(const FNavLocation& StartLocation, TConstArrayView<FVector> ConvexPolygon, TArray<FVector>& OutEdges, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override { return false; };
	virtual bool GetPathSegmentBoundaryEdges(const FNavigationPath& Path, const FNavPathPoint& StartPoint, const FNavPathPoint& EndPoint, const TConstArrayView<FVector> SearchArea, TArray<FVector>& OutEdges, const float MaxAreaEnterCost, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override { return false; }
	virtual FBox GetBounds() const override { return FBox(ForceInit); };
	virtual FNavLocation GetRandomPoint(FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override { return FNavLocation();  }
	virtual bool GetRandomReachablePointInRadius(const FVector& Origin, float Radius, FNavLocation& OutResult, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override { return false; }
	virtual bool GetRandomPointInNavigableRadius(const FVector& Origin, float Radius, FNavLocation& OutResult, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override { return false; }
	virtual bool ProjectPoint(const FVector& Point, FNavLocation& OutLocation, const FVector& Extent, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override { return false; }
	virtual void BatchProjectPoints(TArray<FNavigationProjectionWork>& Workload, const FVector& Extent, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override {};
	virtual void BatchProjectPoints(TArray<FNavigationProjectionWork>& Workload, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override {}
	virtual ENavigationQueryResult::Type CalcPathCost(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& OutPathCost, FSharedConstNavQueryFilter QueryFilter = NULL, const UObject* Querier = NULL) const override { return ENavigationQueryResult::Invalid; }
	virtual ENavigationQueryResult::Type CalcPathLength(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& OutPathLength, FSharedConstNavQueryFilter QueryFilter = NULL, const UObject* Querier = NULL) const override { return ENavigationQueryResult::Invalid; }
	virtual ENavigationQueryResult::Type CalcPathLengthAndCost(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& OutPathLength, FVector::FReal& OutPathCost, FSharedConstNavQueryFilter QueryFilter = NULL, const UObject* Querier = NULL) const override { return ENavigationQueryResult::Invalid; }
	virtual bool DoesNodeContainLocation(NavNodeRef NodeRef, const FVector& WorldSpaceLocation) const override { return true; }
	virtual void OnNavAreaAdded(const UClass* NavAreaClass, int32 AgentIndex) override {}
	virtual void OnNavAreaRemoved(const UClass* NavAreaClass) override {};
	virtual bool IsNodeRefValid(NavNodeRef NodeRef) const override { return true; };
	// End ANavigationData overrides

	static NAVIGATIONSYSTEM_API FPathFindingResult FindPathAbstract(const FNavAgentProperties& AgentProperties, const FPathFindingQuery& Query);
	static NAVIGATIONSYSTEM_API bool TestPathAbstract(const FNavAgentProperties& AgentProperties, const FPathFindingQuery& Query, int32* NumVisitedNodes);
	static NAVIGATIONSYSTEM_API bool RaycastAbstract(const ANavigationData* NavDataInstance, const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, FSharedConstNavQueryFilter QueryFilter, const UObject* Querier);
	static NAVIGATIONSYSTEM_API bool RaycastAbstract(const ANavigationData* NavDataInstance, const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, FNavigationRaycastAdditionalResults* AdditionalResults, FSharedConstNavQueryFilter QueryFilter, const UObject* Querier);
};
