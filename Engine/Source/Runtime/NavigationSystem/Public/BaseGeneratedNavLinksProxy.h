// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "NavLinkCustomInterface.h"

#include "BaseGeneratedNavLinksProxy.generated.h"

/**
 * Experimental
 * Base class used to create generated navlinks proxy.
 * The proxy id is used to represent multiple links generated from the same configuration.
 */
UCLASS(Blueprintable, MinimalAPI)
class UBaseGeneratedNavLinksProxy : public UObject, public INavLinkCustomInterface
{
	GENERATED_UCLASS_BODY()
	
	// BEGIN INavLinkCustomInterface
	NAVIGATIONSYSTEM_API virtual void GetLinkData(FVector& LeftPt, FVector& RightPt, ENavLinkDirection::Type& Direction) const override;
	NAVIGATIONSYSTEM_API virtual FNavLinkId GetId() const override;
	NAVIGATIONSYSTEM_API virtual void UpdateLinkId(FNavLinkId ProxyId) override;
	NAVIGATIONSYSTEM_API virtual UObject* GetLinkOwner() const override;
	// END INavLinkCustomInterface

	void SetOwner(UObject* NewOwner) { Owner = NewOwner; }
	
protected:
	/** The LinkID will be the same for all navlinks using the proxy. */
	UPROPERTY(Transient)
	FNavLinkId LinkProxyId;

	/** Proxy owner. */
	UPROPERTY(Transient)
	TObjectPtr<UObject> Owner = nullptr;
};
