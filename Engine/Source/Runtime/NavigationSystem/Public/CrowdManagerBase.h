// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CrowdManagerBase.generated.h"

class ANavigationData;

/** Base class for Crowd Managers. If you want to create a custom crowd manager
 *	implement a class extending this one and set UNavigationSystemV1::CrowdManagerClass
 *	to point at your class */
UCLASS(Abstract, Transient, MinimalAPI)
class UCrowdManagerBase : public UObject
{
	GENERATED_BODY()
public:
	virtual void Tick(float DeltaTime) PURE_VIRTUAL(UCrowdManagerBase::Tick, );

	/** Called by the nav system when a new navigation data instance is registered. */
	virtual void OnNavDataRegistered(ANavigationData& NavDataInstance) PURE_VIRTUAL(UCrowdManagerBase::OnNavDataRegistered, );

	/** Called by the nav system when a navigation data instance is removed. */
	virtual void OnNavDataUnregistered(ANavigationData& NavDataInstance) PURE_VIRTUAL(UCrowdManagerBase::OnNavDataUnregistered, );

	virtual void CleanUp(float DeltaTime) PURE_VIRTUAL(UCrowdManagerBase::CleanUp, );
};

