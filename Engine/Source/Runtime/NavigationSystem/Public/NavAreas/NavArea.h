// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "AI/Navigation/NavAreaBase.h"
#include "AI/Navigation/NavAgentSelector.h"
#include "NavArea.generated.h"

/** Class containing definition of a navigation area */
UCLASS(DefaultToInstanced, abstract, Config=Engine, Blueprintable, MinimalAPI)
class UNavArea : public UNavAreaBase
{
	GENERATED_BODY()

public: 
	/** travel cost multiplier for path distance */
	UPROPERTY(EditAnywhere, Category=NavArea, config, meta=(ClampMin = "0.0"))
	float DefaultCost;

protected:
	/** entering cost */
	UPROPERTY(EditAnywhere, Category=NavArea, config, meta=(ClampMin = "0.0"))
	float FixedAreaEnteringCost;

public:
	/** area color in navigation view */
	UPROPERTY(EditAnywhere, Category=NavArea, config)
	FColor DrawColor;

	/** restrict area only to specified agents */
	UPROPERTY(EditAnywhere, Category=NavArea, config)
	FNavAgentSelector SupportedAgents;

	// DEPRECATED AGENT CONFIG
#if CPP
	union
	{
		struct
		{
#endif
	UPROPERTY(config)
	uint32 bSupportsAgent0 : 1;
	UPROPERTY(config)
	uint32 bSupportsAgent1 : 1;
	UPROPERTY(config)
	uint32 bSupportsAgent2 : 1;
	UPROPERTY(config)
	uint32 bSupportsAgent3 : 1;
	UPROPERTY(config)
	uint32 bSupportsAgent4 : 1;
	UPROPERTY(config)
	uint32 bSupportsAgent5 : 1;
	UPROPERTY(config)
	uint32 bSupportsAgent6 : 1;
	UPROPERTY(config)
	uint32 bSupportsAgent7 : 1;
	UPROPERTY(config)
	uint32 bSupportsAgent8 : 1;
	UPROPERTY(config)
	uint32 bSupportsAgent9 : 1;
	UPROPERTY(config)
	uint32 bSupportsAgent10 : 1;
	UPROPERTY(config)
	uint32 bSupportsAgent11 : 1;
	UPROPERTY(config)
	uint32 bSupportsAgent12 : 1;
	UPROPERTY(config)
	uint32 bSupportsAgent13 : 1;
	UPROPERTY(config)
	uint32 bSupportsAgent14 : 1;
	UPROPERTY(config)
	uint32 bSupportsAgent15 : 1;
#if CPP
		};
		uint32 SupportedAgentsBits;
	};
#endif

	NAVIGATIONSYSTEM_API UNavArea(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	NAVIGATIONSYSTEM_API virtual void FinishDestroy() override;
	NAVIGATIONSYSTEM_API virtual void PostLoad() override;
	NAVIGATIONSYSTEM_API virtual void PostInitProperties() override;
	NAVIGATIONSYSTEM_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	NAVIGATIONSYSTEM_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent);
	NAVIGATIONSYSTEM_API virtual void PostCDOCompiled(const FPostCDOCompiledContext& Context) override;
#endif // WITH_EDITOR

	inline uint16 GetAreaFlags() const { return AreaFlags; }
	inline bool HasFlags(uint16 InFlags) const { return (InFlags & AreaFlags) != 0; }

	inline bool IsSupportingAgent(int32 AgentIndex) const { return SupportedAgents.Contains(AgentIndex); }

	/** called before adding to navigation system */
	virtual void InitializeArea() {};

	/** Get the fixed area entering cost. */
	virtual float GetFixedAreaEnteringCost() { return FixedAreaEnteringCost; }

	/** Retrieved color declared for AreaDefinitionClass */
	static NAVIGATIONSYSTEM_API FColor GetColor(UClass* AreaDefinitionClass);

	/** copy properties from other area */
	NAVIGATIONSYSTEM_API virtual void CopyFrom(TSubclassOf<UNavArea> AreaClass);

protected:

	/** these flags will be applied to navigation data along with AreaID */
	uint16 AreaFlags;
	
	NAVIGATIONSYSTEM_API void RegisterArea();
};
