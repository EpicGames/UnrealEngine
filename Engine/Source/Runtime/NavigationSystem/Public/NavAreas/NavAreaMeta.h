// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavAreas/NavArea.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "NavAreaMeta.generated.h"

class AActor;
class UObject;

/** A convenience class for an area that has IsMetaArea() == true.
 *	Do not use this class when determining whether an area class is "meta". 
 *	Call IsMetaArea instead. */
UCLASS(Abstract, MinimalAPI)
class UNavAreaMeta : public UNavArea
{
	GENERATED_BODY()

public:
	NAVIGATIONSYSTEM_API UNavAreaMeta(const FObjectInitializer& ObjectInitializer);
};
