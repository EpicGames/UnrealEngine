// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "NavAreas/NavAreaMeta.h"
#include "NavAreaMeta_SwitchByAgent.generated.h"

class AActor;

/** Class containing definition of a navigation area */
UCLASS(Abstract, MinimalAPI)
class UNavAreaMeta_SwitchByAgent : public UNavAreaMeta
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=AgentTypes)
	TSubclassOf<UNavArea> Agent0Area;

	UPROPERTY(EditAnywhere, Category=AgentTypes)
	TSubclassOf<UNavArea> Agent1Area;

	UPROPERTY(EditAnywhere, Category=AgentTypes)
	TSubclassOf<UNavArea> Agent2Area;

	UPROPERTY(EditAnywhere, Category=AgentTypes)
	TSubclassOf<UNavArea> Agent3Area;

	UPROPERTY(EditAnywhere, Category=AgentTypes)
	TSubclassOf<UNavArea> Agent4Area;

	UPROPERTY(EditAnywhere, Category=AgentTypes)
	TSubclassOf<UNavArea> Agent5Area;

	UPROPERTY(EditAnywhere, Category=AgentTypes)
	TSubclassOf<UNavArea> Agent6Area;

	UPROPERTY(EditAnywhere, Category=AgentTypes)
	TSubclassOf<UNavArea> Agent7Area;

	UPROPERTY(EditAnywhere, Category=AgentTypes)
	TSubclassOf<UNavArea> Agent8Area;

	UPROPERTY(EditAnywhere, Category=AgentTypes)
	TSubclassOf<UNavArea> Agent9Area;

	UPROPERTY(EditAnywhere, Category=AgentTypes)
	TSubclassOf<UNavArea> Agent10Area;

	UPROPERTY(EditAnywhere, Category=AgentTypes)
	TSubclassOf<UNavArea> Agent11Area;

	UPROPERTY(EditAnywhere, Category=AgentTypes)
	TSubclassOf<UNavArea> Agent12Area;

	UPROPERTY(EditAnywhere, Category=AgentTypes)
	TSubclassOf<UNavArea> Agent13Area;

	UPROPERTY(EditAnywhere, Category=AgentTypes)
	TSubclassOf<UNavArea> Agent14Area;

	UPROPERTY(EditAnywhere, Category=AgentTypes)
	TSubclassOf<UNavArea> Agent15Area;

	NAVIGATIONSYSTEM_API virtual TSubclassOf<UNavAreaBase> PickAreaClassForAgent(const AActor& Actor, const FNavAgentProperties& NavAgent) const override;

#if WITH_EDITOR
	/** setup AgentXArea properties */
	NAVIGATIONSYSTEM_API virtual void UpdateAgentConfig();

	static bool bUpdateAgentConfigCallbackRegistered;
#endif
};
