// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavAreas/NavArea.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "NavArea_Default.generated.h"

class UObject;

/** Regular navigation area, applied to entire navigation data by default */
UCLASS(Config=Engine, MinimalAPI)
class UNavArea_Default : public UNavArea
{
	GENERATED_UCLASS_BODY()
};
