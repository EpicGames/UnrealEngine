// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AI/Navigation/NavigationTypes.h"
#include "CoreMinimal.h"
#include "NavAreas/NavArea.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "NavArea_LowHeight.generated.h"

class UObject;

/** Special area that can be generated in spaces with insufficient free height above. Cannot be traversed by anyone. */
UCLASS(Config = Engine, MinimalAPI)
class UNavArea_LowHeight : public UNavArea
{
	GENERATED_UCLASS_BODY()
public:
	virtual bool IsLowArea() const override { return true; }
};
