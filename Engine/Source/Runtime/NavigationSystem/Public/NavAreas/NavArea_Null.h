// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavAreas/NavArea.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "NavArea_Null.generated.h"

class UObject;

/** In general represents an empty area, that cannot be traversed by anyone. Ever.*/
UCLASS(Config=Engine, MinimalAPI)
class UNavArea_Null : public UNavArea
{
	GENERATED_UCLASS_BODY()
};
