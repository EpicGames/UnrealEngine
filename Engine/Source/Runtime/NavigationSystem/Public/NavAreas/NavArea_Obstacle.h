// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavAreas/NavArea.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "NavArea_Obstacle.generated.h"

class UObject;

/** In general represents a high cost area, that shouldn't be traversed by anyone unless no other path exist.*/
UCLASS(Config = Engine, MinimalAPI)
class UNavArea_Obstacle : public UNavArea
{
	GENERATED_UCLASS_BODY()
};
