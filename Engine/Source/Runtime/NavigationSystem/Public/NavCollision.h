// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "Serialization/BulkData.h"
#include "AI/Navigation/NavigationTypes.h"
#include "AI/Navigation/NavCollisionBase.h"
#include "NavCollision.generated.h"

class FPrimitiveDrawInterface;
struct FCompositeNavModifier;
struct FNavigableGeometryExport;

USTRUCT()
struct FNavCollisionCylinder
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=Cylinder)
	FVector Offset = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, Category=Cylinder)
	float Radius = 0.f;

	UPROPERTY(EditAnywhere, Category=Cylinder)
	float Height = 0.f;
};

USTRUCT()
struct FNavCollisionBox
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=Box)
	FVector Offset = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, Category=Box)
	FVector Extent = FVector::ZeroVector;
};

UCLASS(config=Engine, MinimalAPI)
class UNavCollision : public UNavCollisionBase
{
	GENERATED_UCLASS_BODY()

	TNavStatArray<int32> ConvexShapeIndices;

	FBox Bounds;

	/** list of nav collision cylinders */
	UPROPERTY(EditAnywhere, Category=Navigation)
	TArray<FNavCollisionCylinder> CylinderCollision;

	/** list of nav collision boxes */
	UPROPERTY(EditAnywhere, Category=Navigation)
	TArray<FNavCollisionBox> BoxCollision;

	/** Navigation area type for this mesh's collision geometry.
	 *
	 *  Area usages (mutually exclusive):
	 *
	 *  1) Dynamic Obstacles
	 *     - GetNavigationModifier() creates convex area modifiers with AreaClass
	 *     - These paint obstacle volumes into the navmesh during generation
	 *     - Used for dynamic obstacles that can move/appear/disappear at runtime
	 *     - Area class applied via nav modifier system
	 *
	 *  2) Surface Area
	 *     - GetNavigationModifier() adds a FAreaNavModifier
	 *     - AreaClass flows to FRecastRawGeometryElement during collision export
	 *     - Applied during triangle rasterization
	 *     - Area class applied during voxelization, not as post-process modifier
	 */
	UPROPERTY(EditAnywhere, Category = Navigation, meta = (EditCondition = "bIsDynamicObstacle || bUseSurfaceArea"))
	TSubclassOf<class UNavArea> AreaClass;

	/** If set, convex collisions will be exported offline for faster runtime navmesh building (increases memory usage) */
	UPROPERTY(EditAnywhere, Category=Navigation, config)
	uint32 bGatherConvexGeometry : 1;

	/** If false, will not create nav collision when connecting as a client */
	UPROPERTY(EditAnywhere, Category=Navigation, config)
	uint32 bCreateOnClient : 1;

	/** if set, convex geometry will be rebuilt instead of using cooked data */
	uint32 bForceGeometryRebuild : 1;

	/** Guid of associated BodySetup */
	FGuid BodySetupGuid;

	/** Cooked data for each format */
	FFormatContainer CookedFormatData;

	//~ Begin UObject Interface.
	NAVIGATIONSYSTEM_API virtual void PostInitProperties() override;
	NAVIGATIONSYSTEM_API virtual void Serialize(FArchive& Ar) override;
	NAVIGATIONSYSTEM_API virtual void PostLoad() override;
	NAVIGATIONSYSTEM_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
#if WITH_EDITOR
	NAVIGATIONSYSTEM_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	NAVIGATIONSYSTEM_API virtual bool NeedsLoadForTargetPlatform(const class ITargetPlatform* TargetPlatform) const override;
	virtual bool NeedsLoadForClient() const override { return bCreateOnClient; }
	//~ End UObject Interface.

	NAVIGATIONSYSTEM_API FGuid GetGuid() const;

	/** Tries to read data from DDC, and if that fails gathers navigation
	 *	collision data, stores it and uploads to DDC */
	NAVIGATIONSYSTEM_API virtual void Setup(class UBodySetup* BodySetup) override;

	NAVIGATIONSYSTEM_API virtual FBox GetBounds() const override;

	/** copy user settings from other nav collision data */
	NAVIGATIONSYSTEM_API void CopyUserSettings(const UNavCollision& OtherData);

	/** show cylinder and box collisions */
	NAVIGATIONSYSTEM_API virtual void DrawSimpleGeom(FPrimitiveDrawInterface* PDI, const FTransform& Transform, const FColor Color) override;

	/** Get data for dynamic obstacle and collisions with surface navarea. */
	NAVIGATIONSYSTEM_API virtual void GetNavigationModifier(FCompositeNavModifier& Modifier, const FTransform& LocalToWorld) override;

	/** Returns true if this collision has a surface area class set for collision triangles. */
	NAVIGATIONSYSTEM_API virtual bool HasSurfaceAreaClass() const override;

	/** Export collision data */
	NAVIGATIONSYSTEM_API virtual bool ExportGeometry(const FTransform& LocalToWorld, FNavigableGeometryExport& GeoExport) const override;

	/** Read collisions data */
	NAVIGATIONSYSTEM_API void GatherCollision();

#if WITH_EDITOR
	NAVIGATIONSYSTEM_API virtual void InvalidateCollision() override;
#endif // WITH_EDITOR

protected:
	NAVIGATIONSYSTEM_API void ClearCollision();

#if WITH_EDITOR
	NAVIGATIONSYSTEM_API void InvalidatePhysicsData();
#endif // WITH_EDITOR
	NAVIGATIONSYSTEM_API FByteBulkData* GetCookedData(FName Format);
};
