// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineDefines.h"


#if UE_ENABLE_DEBUG_DRAWING
#include "DebugRenderSceneProxy.h"

typedef FDebugRenderSceneProxy::FMesh FNavDebugMeshData;
#endif //!UE_BUILD_SHIPPING