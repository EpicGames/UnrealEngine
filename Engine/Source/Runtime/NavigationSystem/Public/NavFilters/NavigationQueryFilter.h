// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "NavAreas/NavArea.h"
#include "AI/Navigation/NavQueryFilter.h"
#include "NavigationQueryFilter.generated.h"

class ANavigationData;

USTRUCT()
struct FNavigationFilterArea
{
	GENERATED_USTRUCT_BODY()

	/** navigation area class */
	UPROPERTY(EditAnywhere, Category=Area)
	TSubclassOf<UNavArea> AreaClass;

	/** override for travel cost */
	UPROPERTY(EditAnywhere, Category=Area, meta=(EditCondition="bOverrideTravelCost",ClampMin=0.001))
	float TravelCostOverride;

	/** override for entering cost */
	UPROPERTY(EditAnywhere, Category=Area, meta=(EditCondition="bOverrideEnteringCost",ClampMin=0))
	float EnteringCostOverride;

	/** mark as excluded */
	UPROPERTY(EditAnywhere, Category=Area)
	uint32 bIsExcluded : 1;

	UPROPERTY(EditAnywhere, Category=Area, meta=(InlineEditConditionToggle))
	uint32 bOverrideTravelCost : 1;

	UPROPERTY(EditAnywhere, Category=Area, meta=(InlineEditConditionToggle))
	uint32 bOverrideEnteringCost : 1;

	FNavigationFilterArea()
	{
		FMemory::Memzero(*this);
		TravelCostOverride = 1.f;
	}
};

// 
// Use UNavigationSystemV1.DescribeFilterFlags() to setup user friendly names of flags
// 
USTRUCT()
struct FNavigationFilterFlags
{
	GENERATED_USTRUCT_BODY()

#if CPP
	union
	{
		struct
		{
#endif
	UPROPERTY(EditAnywhere, Category=Flags)
	uint32 bNavFlag0 : 1;
	UPROPERTY(EditAnywhere, Category=Flags)
	uint32 bNavFlag1 : 1;
	UPROPERTY(EditAnywhere, Category=Flags)
	uint32 bNavFlag2 : 1;
	UPROPERTY(EditAnywhere, Category=Flags)
	uint32 bNavFlag3 : 1;
	UPROPERTY(EditAnywhere, Category=Flags)
	uint32 bNavFlag4 : 1;
	UPROPERTY(EditAnywhere, Category=Flags)
	uint32 bNavFlag5 : 1;
	UPROPERTY(EditAnywhere, Category=Flags)
	uint32 bNavFlag6 : 1;
	UPROPERTY(EditAnywhere, Category=Flags)
	uint32 bNavFlag7 : 1;
	UPROPERTY(EditAnywhere, Category=Flags)
	uint32 bNavFlag8 : 1;
	UPROPERTY(EditAnywhere, Category=Flags)
	uint32 bNavFlag9 : 1;
	UPROPERTY(EditAnywhere, Category=Flags)
	uint32 bNavFlag10 : 1;
	UPROPERTY(EditAnywhere, Category=Flags)
	uint32 bNavFlag11 : 1;
	UPROPERTY(EditAnywhere, Category=Flags)
	uint32 bNavFlag12 : 1;
	UPROPERTY(EditAnywhere, Category=Flags)
	uint32 bNavFlag13 : 1;
	UPROPERTY(EditAnywhere, Category=Flags)
	uint32 bNavFlag14 : 1;
	UPROPERTY(EditAnywhere, Category=Flags)
	uint32 bNavFlag15 : 1;
#if CPP
		};
		uint16 Packed = 0;
	};
#endif
};

/** Class containing definition of a navigation query filter */
UCLASS(Abstract, Blueprintable, MinimalAPI)
class UNavigationQueryFilter : public UObject
{
	GENERATED_UCLASS_BODY()
	
	/** list of overrides for navigation areas */
	UPROPERTY(EditAnywhere, Category=Filter, meta = (TitleProperty = "AreaClass"))
	TArray<FNavigationFilterArea> Areas;

	/** required flags of navigation nodes */
	UPROPERTY(EditAnywhere, Category=Filter)
	FNavigationFilterFlags IncludeFlags;

	/** forbidden flags of navigation nodes */
	UPROPERTY(EditAnywhere, Category=Filter)
	FNavigationFilterFlags ExcludeFlags;

	/** get filter for given navigation data and initialize on first access */
	NAVIGATIONSYSTEM_API FSharedConstNavQueryFilter GetQueryFilter(const ANavigationData& NavData, const UObject* Querier) const;
	
	/** helper functions for accessing filter */
	static NAVIGATIONSYSTEM_API FSharedConstNavQueryFilter GetQueryFilter(const ANavigationData& NavData, TSubclassOf<UNavigationQueryFilter> FilterClass);
	static NAVIGATIONSYSTEM_API FSharedConstNavQueryFilter GetQueryFilter(const ANavigationData& NavData, const UObject* Querier, TSubclassOf<UNavigationQueryFilter> FilterClass);

	template<class T>
	static FSharedConstNavQueryFilter GetQueryFilter(const ANavigationData& NavData, TSubclassOf<UNavigationQueryFilter> FilterClass = T::StaticClass())
	{
		return GetQueryFilter(NavData, FilterClass);
	}

#if WITH_EDITOR
	NAVIGATIONSYSTEM_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

protected:

	/** if set, filter will not be cached by navigation data and can be configured per Querier */
	uint32 bInstantiateForQuerier : 1;

	/** if set to true GetSimpleFilterForAgent will be called when determining the actual filter class to be used */
	uint32 bIsMetaFilter : 1;

	/** helper functions for adding area overrides */
	NAVIGATIONSYSTEM_API void AddTravelCostOverride(TSubclassOf<UNavArea> AreaClass, float TravelCost);
	NAVIGATIONSYSTEM_API void AddEnteringCostOverride(TSubclassOf<UNavArea> AreaClass, float EnteringCost);
	NAVIGATIONSYSTEM_API void AddExcludedArea(TSubclassOf<UNavArea> AreaClass);

	/** find index of area data */
	NAVIGATIONSYSTEM_API int32 FindAreaOverride(TSubclassOf<UNavArea> AreaClass) const;
	
	/** setup filter for given navigation data, use to create custom filters */
	NAVIGATIONSYSTEM_API virtual void InitializeFilter(const ANavigationData& NavData, const UObject* Querier, FNavigationQueryFilter& Filter) const;

	virtual TSubclassOf<UNavigationQueryFilter> GetSimpleFilterForAgent(const UObject& Querier) const { return nullptr; }
};
