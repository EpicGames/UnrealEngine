// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavFilters/NavigationQueryFilter.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "RecastFilter_UseDefaultArea.generated.h"

class ANavigationData;
class UObject;
struct FNavigationQueryFilter;

/** Regular navigation area, applied to entire navigation data by default */
UCLASS(MinimalAPI)
class URecastFilter_UseDefaultArea : public UNavigationQueryFilter
{
	GENERATED_UCLASS_BODY()

	virtual void InitializeFilter(const ANavigationData& NavData, const UObject* Querier, FNavigationQueryFilter& Filter) const override;
};
