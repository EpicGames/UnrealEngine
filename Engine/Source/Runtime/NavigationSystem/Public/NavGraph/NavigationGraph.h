// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "NavigationData.h"
#include "NavigationGraph.generated.h"


USTRUCT()
struct FNavGraphEdge
{
	GENERATED_USTRUCT_BODY()

	enum {
		EgdeFlagsCount = 7
	};

	struct FNavGraphNode* Start;

	struct FNavGraphNode* End;

	int32 Flags : EgdeFlagsCount;
	uint32 bEnabled : 1;

	FNavGraphEdge()
		: Start(NULL)
		, End(NULL)
		, Flags(0)
		, bEnabled(false)
	{
	}
};


USTRUCT()
struct FNavGraphNode
{
	GENERATED_USTRUCT_BODY()

	/** Who's this node referring to? This will most commonly point to an actor or a component */
	UPROPERTY()
	TObjectPtr<UObject> Owner;

	enum {
		InitialEdgesCount = 4
	};

	TArray<FNavGraphEdge> Edges;
	// Location to be added here
	// Radius might be needed as well

	FNavGraphNode();
};

/** currently abstract since it's not full implemented */
UCLASS(config=Engine, MinimalAPI, abstract)
class ANavigationGraph : public ANavigationData
{
	GENERATED_UCLASS_BODY()

public:
};
