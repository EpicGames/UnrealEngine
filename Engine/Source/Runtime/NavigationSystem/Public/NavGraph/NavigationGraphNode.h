// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"

#include "NavigationGraphNode.generated.h"

UCLASS(config=Engine, MinimalAPI, NotBlueprintable, abstract)
class ANavigationGraphNode : public AActor
{
	GENERATED_UCLASS_BODY()
};
