// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Components/SceneComponent.h"
#include "NavGraph/NavigationGraph.h"
#include "NavigationGraphNodeComponent.generated.h"

UCLASS(config=Engine, MinimalAPI, HideCategories=(Mobility))
class UNavigationGraphNodeComponent : public USceneComponent
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FNavGraphNode Node;

	UPROPERTY()
	TObjectPtr<UNavigationGraphNodeComponent> NextNodeComponent;

	UPROPERTY()
	TObjectPtr<UNavigationGraphNodeComponent> PrevNodeComponent;

	//~ Begin UObject Interface.
	virtual void BeginDestroy() override;
};
