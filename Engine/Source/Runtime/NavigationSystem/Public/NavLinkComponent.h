// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Components/PrimitiveComponent.h"
#include "AI/Navigation/NavLinkDefinition.h"
#include "NavLinkHostInterface.h"
#include "NavLinkComponent.generated.h"

class FPrimitiveSceneProxy;
struct FNavigationRelevantData;

UCLASS(ClassGroup = (Navigation), meta = (BlueprintSpawnableComponent), hidecategories = (Activation), MinimalAPI)
class UNavLinkComponent : public UPrimitiveComponent, public INavLinkHostInterface
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Navigation)
	TArray<FNavigationLink> Links;

	NAVIGATIONSYSTEM_API virtual void GetNavigationData(FNavigationRelevantData& Data) const override;
	NAVIGATIONSYSTEM_API virtual bool IsNavigationRelevant() const override;

	virtual bool GetNavigationLinksClasses(TArray<TSubclassOf<class UNavLinkDefinition> >& OutClasses) const override { return false; }
	NAVIGATIONSYSTEM_API virtual bool GetNavigationLinksArray(TArray<FNavigationLink>& OutLink, TArray<FNavigationSegmentLink>& OutSegments) const override;

	NAVIGATIONSYSTEM_API virtual FBoxSphereBounds CalcBounds(const FTransform &LocalToWorld) const override;
	NAVIGATIONSYSTEM_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual bool ShouldRecreateProxyOnUpdateTransform() const override { return true; }

#if WITH_EDITOR
	NAVIGATIONSYSTEM_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	NAVIGATIONSYSTEM_API virtual void PostEditUndo() override;
	NAVIGATIONSYSTEM_API virtual void PostEditImport() override;
#endif // WITH_EDITOR

	NAVIGATIONSYSTEM_API virtual void OnRegister() override;

protected:
	NAVIGATIONSYSTEM_API void InitializeLinksAreaClasses();
};
