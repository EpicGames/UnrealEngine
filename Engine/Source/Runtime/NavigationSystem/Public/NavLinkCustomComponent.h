// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "NavAreas/NavArea.h"
#include "AI/Navigation/NavLinkDefinition.h"
#include "AI/Navigation/NavAgentSelector.h"
#include "NavLinkCustomInterface.h"
#include "NavRelevantComponent.h"
#include "NavLinkCustomComponent.generated.h"


struct FNavigationRelevantData;

/**
 *  Encapsulates NavLinkCustomInterface interface, can be used with Actors not relevant for navigation
 *  
 *  Additional functionality:
 *  - can be toggled
 *  - can create obstacle area for easier/forced separation of link end points
 *  - can broadcast state changes to nearby agents
 */

UCLASS(MinimalAPI)
class UNavLinkCustomComponent : public UNavRelevantComponent, public INavLinkCustomInterface
{
	GENERATED_UCLASS_BODY()

	DECLARE_DELEGATE_ThreeParams(FOnMoveReachedLink, UNavLinkCustomComponent* /*ThisComp*/, UObject* /*PathComp*/, const FVector& /*DestPoint*/);
	DECLARE_DELEGATE_TwoParams(FBroadcastFilter, UNavLinkCustomComponent* /*ThisComp*/, TArray<UObject*>& /*NotifyList*/);

	// BEGIN INavLinkCustomInterface
	NAVIGATIONSYSTEM_API virtual void GetLinkData(FVector& LeftPt, FVector& RightPt, ENavLinkDirection::Type& Direction) const override;
	NAVIGATIONSYSTEM_API virtual void GetSupportedAgents(FNavAgentSelector& OutSupportedAgents) const override;
	NAVIGATIONSYSTEM_API virtual TSubclassOf<UNavArea> GetLinkAreaClass() const override;
	NAVIGATIONSYSTEM_API virtual FNavLinkAuxiliaryId GetAuxiliaryId() const override;
	NAVIGATIONSYSTEM_API virtual FNavLinkId GetId() const override;
	NAVIGATIONSYSTEM_API virtual void UpdateLinkId(FNavLinkId NewUniqueId) override;
	NAVIGATIONSYSTEM_API virtual bool IsLinkPathfindingAllowed(const UObject* Querier) const override;
	NAVIGATIONSYSTEM_API virtual bool OnLinkMoveStarted(UObject* PathComp, const FVector& DestPoint) override;
	NAVIGATIONSYSTEM_API virtual void OnLinkMoveFinished(UObject* PathComp) override;
	// END INavLinkCustomInterface

	//~ Begin UNavRelevantComponent Interface
	NAVIGATIONSYSTEM_API virtual void GetNavigationData(FNavigationRelevantData& Data) const override;
	NAVIGATIONSYSTEM_API virtual void CalcAndCacheBounds() const override;
	//~ End UNavRelevantComponent Interface

	//~ Begin UActorComponent Interface
	NAVIGATIONSYSTEM_API virtual void OnRegister() override;
	NAVIGATIONSYSTEM_API virtual void OnUnregister() override;
	NAVIGATIONSYSTEM_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;
	//~ End UActorComponent Interface

	NAVIGATIONSYSTEM_API void ApplyComponentInstanceData(struct FNavLinkCustomInstanceData* ComponentInstanceData);

	//~ Begin UObject Interface
	NAVIGATIONSYSTEM_API virtual void Serialize(FArchive& Ar) override;
	NAVIGATIONSYSTEM_API virtual void PostLoad() override;
#if WITH_EDITOR
	NAVIGATIONSYSTEM_API virtual void PostEditImport() override;
#endif
	//~ End UObject Interface

	/** set basic link data: end points and direction */
	NAVIGATIONSYSTEM_API void SetLinkData(const FVector& RelativeStart, const FVector& RelativeEnd, ENavLinkDirection::Type Direction);
	NAVIGATIONSYSTEM_API virtual FNavigationLink GetLinkModifier() const;

	/** set area class to use when link is enabled */
	NAVIGATIONSYSTEM_API void SetEnabledArea(TSubclassOf<UNavArea> AreaClass);
	TSubclassOf<UNavArea> GetEnabledArea() const { return EnabledAreaClass; }

	/** set area class to use when link is disabled */
	NAVIGATIONSYSTEM_API void SetDisabledArea(TSubclassOf<UNavArea> AreaClass);
	TSubclassOf<UNavArea> GetDisabledArea() const { return DisabledAreaClass; }

	void SetSupportedAgents(const FNavAgentSelector& InSupportedAgents) { SupportedAgents = InSupportedAgents; }
	FNavAgentSelector GetSupportedAgents() const { return SupportedAgents; }

	/** add box obstacle during generation of navigation data
	  * this can be used to create empty area under doors */
	NAVIGATIONSYSTEM_API void AddNavigationObstacle(TSubclassOf<UNavArea> AreaClass, const FVector& BoxExtent, const FVector& BoxOffset = FVector::ZeroVector);

	/** removes simple obstacle */
	NAVIGATIONSYSTEM_API void ClearNavigationObstacle();

	/** set properties of trigger around link entry point(s), that will notify nearby agents about link state change */
	NAVIGATIONSYSTEM_API void SetBroadcastData(float Radius, ECollisionChannel TraceChannel = ECC_Pawn, float Interval = 0.0f);

	NAVIGATIONSYSTEM_API void SendBroadcastWhenEnabled(bool bEnabled);
	NAVIGATIONSYSTEM_API void SendBroadcastWhenDisabled(bool bEnabled);

	/** set delegate to filter  */
	NAVIGATIONSYSTEM_API void SetBroadcastFilter(FBroadcastFilter const& InDelegate);

	/** change state of smart link (used area class) */
	NAVIGATIONSYSTEM_API void SetEnabled(bool bNewEnabled);
	bool IsEnabled() const { return bLinkEnabled; }
	
	/** set delegate to notify about reaching this link during path following */
	NAVIGATIONSYSTEM_API void SetMoveReachedLink(FOnMoveReachedLink const& InDelegate);

	/** check is any agent is currently moving though this link */
	NAVIGATIONSYSTEM_API bool HasMovingAgents() const;

	/** get link start point in world space */
	NAVIGATIONSYSTEM_API FVector GetStartPoint() const;

	/** get link end point in world space */
	NAVIGATIONSYSTEM_API FVector GetEndPoint() const;

	TSubclassOf<UNavArea> GetObstacleAreaClass() const { return ObstacleAreaClass; }

	//////////////////////////////////////////////////////////////////////////
	// helper functions for setting delegates

	template< class UserClass >	
	inline void SetMoveReachedLink(UserClass* TargetOb, typename FOnMoveReachedLink::TMethodPtr< UserClass > InFunc)
	{
		SetMoveReachedLink(FOnMoveReachedLink::CreateUObject(TargetOb, InFunc));
	}
	template< class UserClass >	
	inline void SetMoveReachedLink(UserClass* TargetOb, typename FOnMoveReachedLink::TConstMethodPtr< UserClass > InFunc)
	{
		SetMoveReachedLink(FOnMoveReachedLink::CreateUObject(TargetOb, InFunc));
	}

	template< class UserClass >	
	inline void SetBroadcastFilter(UserClass* TargetOb, typename FBroadcastFilter::TMethodPtr< UserClass > InFunc)
	{
		SetBroadcastFilter(FBroadcastFilter::CreateUObject(TargetOb, InFunc));
	}
	template< class UserClass >	
	inline void SetBroadcastFilter(UserClass* TargetOb, typename FBroadcastFilter::TConstMethodPtr< UserClass > InFunc)
	{
		SetBroadcastFilter(FBroadcastFilter::CreateUObject(TargetOb, InFunc));
	}

protected:
#if WITH_EDITOR
	NAVIGATIONSYSTEM_API void OnNavAreaRegistered(const UWorld& World, const UClass* NavAreaClass);
	NAVIGATIONSYSTEM_API void OnNavAreaUnregistered(const UWorld& World, const UClass* NavAreaClass);
#endif // WITH_EDITOR

protected:

	UE_DEPRECATED(5.3, "LinkIds are now based on FNavLinkId. Use CustomLinkId instead. This Id is no longer used by the engine.")
	UPROPERTY()
	uint32 NavLinkUserId;

	/** link Id assigned by navigation system */
	UPROPERTY()
	FNavLinkId CustomLinkId;

	/** 
	 *  Assigned in the constructor. This uniquely identifies a component in an Actor, but will not be unique between duplicate level instances.
	 *  containing the same Actor.
	 *  This is Hashed with the Actor Instance FGuid to create the CustomLinkId so that Actors with more than one UNavLinkCustomComponent can have a 
	 *  completely unique ID per UNavLinkCustomComponent even across level instances.
	 **/
	UPROPERTY()
	FNavLinkAuxiliaryId AuxiliaryCustomLinkId;

	/** area class to use when link is enabled */
	UPROPERTY(EditAnywhere, Category=SmartLink)
	TSubclassOf<UNavArea> EnabledAreaClass;

	/** area class to use when link is disabled */
	UPROPERTY(EditAnywhere, Category=SmartLink)
	TSubclassOf<UNavArea> DisabledAreaClass;

	/** restrict area only to specified agents */
	UPROPERTY(EditAnywhere, Category=SmartLink)
	FNavAgentSelector SupportedAgents;

	/** start point, relative to owner */
	UPROPERTY(EditAnywhere, Category=SmartLink)
	FVector LinkRelativeStart;

	/** end point, relative to owner */
	UPROPERTY(EditAnywhere, Category=SmartLink)
	FVector LinkRelativeEnd;

	/** direction of link */
	UPROPERTY(EditAnywhere, Category=SmartLink)
	TEnumAsByte<ENavLinkDirection::Type> LinkDirection;

	/** is link currently in enabled state? (area class) */
	UPROPERTY(EditAnywhere, Category=SmartLink)
	uint32 bLinkEnabled : 1;

	/** should link notify nearby agents when it changes state to enabled */
	UPROPERTY(EditAnywhere, Category=Broadcast)
	uint32 bNotifyWhenEnabled : 1;

	/** should link notify nearby agents when it changes state to disabled */
	UPROPERTY(EditAnywhere, Category=Broadcast)
	uint32 bNotifyWhenDisabled : 1;

	/** if set, box obstacle area will be added to generation */
	UPROPERTY(EditAnywhere, Category=Obstacle)
	uint32 bCreateBoxObstacle : 1;

	/** offset of simple box obstacle */
	UPROPERTY(EditAnywhere, Category=Obstacle)
	FVector ObstacleOffset;

	/** extent of simple box obstacle */
	UPROPERTY(EditAnywhere, Category=Obstacle)
	FVector ObstacleExtent;

	/** area class for simple box obstacle */
	UPROPERTY(EditAnywhere, Category=Obstacle)
	TSubclassOf<UNavArea> ObstacleAreaClass;

	/** radius of state change broadcast */
	UPROPERTY(EditAnywhere, Category=Broadcast)
	float BroadcastRadius;

	/** interval for state change broadcast (0 = single broadcast) */
	UPROPERTY(EditAnywhere, Category=Broadcast, Meta=(UIMin="0.0", ClampMin="0.0"))
	float BroadcastInterval;

	/** trace channel for state change broadcast */
	UPROPERTY(EditAnywhere, Category=Broadcast)
	TEnumAsByte<ECollisionChannel> BroadcastChannel;

	/** delegate to call when link is reached */
	FBroadcastFilter OnBroadcastFilter;

	/** list of agents moving though this link */
	TArray<TWeakObjectPtr<UObject> > MovingAgents;

	/** delegate to call when link is reached */
	FOnMoveReachedLink OnMoveReachedLink;

	/** Handle for efficient management of BroadcastStateChange timer */
	FTimerHandle TimerHandle_BroadcastStateChange;

	/** notify nearby agents about link changing state */
	NAVIGATIONSYSTEM_API void BroadcastStateChange();
	
	/** gather agents to notify about state change */
	NAVIGATIONSYSTEM_API void CollectNearbyAgents(TArray<UObject*>& NotifyList);

#if WITH_EDITOR
	FDelegateHandle OnNavAreaRegisteredDelegateHandle;
	FDelegateHandle OnNavAreaUnregisteredDelegateHandle;
#endif // WITH_EDITOR
};

/** Used to store navlink data during RerunConstructionScripts */
USTRUCT()
struct FNavLinkCustomInstanceData : public FActorComponentInstanceData
{
	GENERATED_BODY()

public:
	FNavLinkCustomInstanceData() = default;
	FNavLinkCustomInstanceData(const UNavLinkCustomComponent* SourceComponent)
		: FActorComponentInstanceData(SourceComponent)
	{}

	virtual ~FNavLinkCustomInstanceData() = default;

	virtual bool ContainsData() const override
	{
		return true;
	}

	virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase) override
	{
		Super::ApplyToComponent(Component, CacheApplyPhase);

		if (CacheApplyPhase == ECacheApplyPhase::PostUserConstructionScript)
		{
			CastChecked<UNavLinkCustomComponent>(Component)->ApplyComponentInstanceData(this);
		}
	}

	UPROPERTY()
	FNavLinkId CustomLinkId;

	UPROPERTY()
	FNavLinkAuxiliaryId AuxiliaryCustomLinkId;
};
