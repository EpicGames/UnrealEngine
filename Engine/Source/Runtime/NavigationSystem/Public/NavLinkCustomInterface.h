// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "UObject/Interface.h"
#include "NavAreas/NavArea.h"
#include "AI/Navigation/NavLinkDefinition.h"
#include "Engine/WorldInitializationValues.h"
#include "NavLinkCustomInterface.generated.h"

/** 
 *  Interface for custom navigation links
 *
 *  They can affect path finding requests without navmesh rebuilds (e.g. opened/closed doors),
 *  allows updating their area class without navmesh rebuilds (e.g. dynamic path cost)
 *  and give hooks for supporting custom movement (e.g. ladders),
 *
 *  Owner is responsible for registering and unregistering links in NavigationSystem:
 *  - RegisterCustomLink
 *  - UnregisterCustomLink
 *
 *  See also: NavLinkCustomComponent
 */

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UNavLinkCustomInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class INavLinkCustomInterface
{
	GENERATED_IINTERFACE_BODY()

	/** Get basic link data: two points (relative to owner) and direction */
	virtual void GetLinkData(FVector& LeftPt, FVector& RightPt, ENavLinkDirection::Type& Direction) const {};

	/** Get agents supported by this link */
	virtual void GetSupportedAgents(FNavAgentSelector& OutSupportedAgents) const {};

	/** Get basic link data: area class (null = default walkable) */
	virtual TSubclassOf<UNavArea> GetLinkAreaClass() const { return nullptr; }

	virtual FNavLinkAuxiliaryId GetAuxiliaryId() const { return FNavLinkAuxiliaryId::Invalid; }

	UE_DEPRECATED(5.3, "LinkIds are now based on a FNavLinkId. Call GetId() instead. This function only returns Invalid Id.")
	virtual uint32 GetLinkId() const final { return static_cast<uint32>(FNavLinkId::Invalid.GetId()); }

	/** Get unique ID number for custom link
	 *  Owner should get its unique ID by calling INavLinkCustomInterface::GetUniqueId() and store it
	 */
	virtual FNavLinkId GetId() const { return FNavLinkId::Invalid; }

	UE_DEPRECATED(5.3, "LinkIds are now based on a FNavLinkId. Call the version of this function that takes a FNavLinkId. This function now has no effect.")
	virtual void UpdateLinkId(uint32 NewUniqueId) final {}

	/** Update unique ID number for custom link by navigation system. */
	virtual void UpdateLinkId(FNavLinkId NewUniqueId) {}

	/** Get object owner of navigation link, used for creating containers with multiple links */
	NAVIGATIONSYSTEM_API virtual UObject* GetLinkOwner() const;

	/** Check if link allows path finding
	 *  Querier is usually an AIController trying to find path
	 */
	virtual bool IsLinkPathfindingAllowed(const UObject* Querier) const { return true; }

	/** Notify called when agent starts using this link for movement.
	 *  returns true = custom movement, path following will NOT update velocity until FinishUsingCustomLink() is called on it
	 */
	virtual bool OnLinkMoveStarted(class UObject* PathComp, const FVector& DestPoint) { return false; }

	/** Notify called when agent finishes using this link for movement */
	virtual void OnLinkMoveFinished(class UObject* PathComp) {}

	/** Whether or not this link has custom reach conditions that need to override the default reach checks done by the path following component. */
	virtual bool IsLinkUsingCustomReachCondition(const UObject* PathComp) const { return false; }

	/** Function that replaces the default reach check when IsLinkUsingCustomReachCondition is true. 
	 *  Returns true if CurrentLocation has reached the start of the link.
	 */
	virtual bool HasReachedLinkStart(const UObject* PathComp, const FVector& CurrentLocation, const FNavPathPoint& LinkStart, const FNavPathPoint& LinkEnd) const { return true; }

	UE_DEPRECATED(5.3, "LinkIds are now based on FNavLinkId using FNavLinkId::GenerateUniqueId(). This function will still generate an incremental Id however it does not work well in all circumstances.")
	static NAVIGATIONSYSTEM_API uint32 GetUniqueId();

	/** Helper function: bump unique ID numbers above given one */
	UE_DEPRECATED(5.3, "LinkIds are now based on FNavLinkId. If your project is still using any of the old incremental Ids (saved in actors in levels or licensee code) then this function must be called still (typically by existing engine code), otherwise it is not necessary.")
	static NAVIGATIONSYSTEM_API void UpdateUniqueId(FNavLinkId AlreadyUsedId);

	UE_DEPRECATED(5.3, "LinkIds are now based on FNavLinkId. You may need to call the other version of this function that takes a FNavLinkId. This function has no effect.")
	static void UpdateUniqueId(uint32 AlreadyUsedId) {}

	/** Helper function: create modifier for navigation data export */
	static NAVIGATIONSYSTEM_API FNavigationLink GetModifier(const INavLinkCustomInterface* CustomNavLink);
	
	UE_DEPRECATED(5.3, "LinkIds are now based on a FNavLinkId Hash. If your project is still using any of the old incremental Ids then this function must be called still (typically by existing engine code), otherwise it is not necessary.")
	static NAVIGATIONSYSTEM_API void ResetUniqueId();

	static NAVIGATIONSYSTEM_API void OnPreWorldInitialization(UWorld* World, const FWorldInitializationValues IVS);

	UE_DEPRECATED(5.3, "LinkIds are now based on FNavLinkId using FNavLinkId::GenerateUniqueId().")
	static uint32 NextUniqueId;
};

