// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "UObject/Interface.h"
#include "AI/Navigation/NavLinkDefinition.h"
#include "NavLinkHostInterface.generated.h"

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UNavLinkHostInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class INavLinkHostInterface
{
	GENERATED_IINTERFACE_BODY()
		
	/**
	 *	Retrieves UNavLinkDefinition derived UClasses hosted by this interface implementer
	 */
	virtual bool GetNavigationLinksClasses(TArray<TSubclassOf<class UNavLinkDefinition> >& OutClasses) const PURE_VIRTUAL(INavLinkHostInterface::GetNavigationLinksClasses,return false;);

	/** 
	 *	_Optional_ way of retrieving navigation link data - if INavLinkHostInterface 
	 *	implementer defines custom navigation links then it can just retrieve 
	 *	a list of links
	 */
	virtual bool GetNavigationLinksArray(TArray<FNavigationLink>& OutLink, TArray<FNavigationSegmentLink>& OutSegments) const { return false; }
};
