// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Components/PrimitiveComponent.h"
#include "NavLinkRenderingComponent.generated.h"

class FPrimitiveSceneProxy;
struct FConvexVolume;
struct FEngineShowFlags;

UCLASS(hidecategories=Object, editinlinenew, MinimalAPI)
class UNavLinkRenderingComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()
		
	//~ Begin UPrimitiveComponent Interface
	NAVIGATIONSYSTEM_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;

	/** Should recreate proxy one very update */
	virtual bool ShouldRecreateProxyOnUpdateTransform() const override { return true; }
	virtual bool GetIgnoreBoundsForEditorFocus() const override { return true; }
#if WITH_EDITOR
	NAVIGATIONSYSTEM_API virtual bool ComponentIsTouchingSelectionBox(const FBox& InSelBBox, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const override;
	NAVIGATIONSYSTEM_API virtual bool ComponentIsTouchingSelectionFrustum(const FConvexVolume& InFrustum, const bool bConsiderOnlyBSP, const bool bMustEncompassEntireComponent) const override;
#endif
	//~ End UPrimitiveComponent Interface

	//~ Begin USceneComponent Interface
	NAVIGATIONSYSTEM_API virtual FBoxSphereBounds CalcBounds(const FTransform &LocalToWorld) const override;
	//~ End USceneComponent Interface
};

