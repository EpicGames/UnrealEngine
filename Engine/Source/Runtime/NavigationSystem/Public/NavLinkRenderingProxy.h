// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavAreas/NavArea.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "AI/Navigation/NavLinkDefinition.h"

class FMaterialRenderProxy;
class FMeshElementCollector;
class FPrimitiveDrawInterface;
class UPrimitiveComponent;

class FNavLinkRenderingProxy : public FPrimitiveSceneProxy
{
private:
	AActor* LinkOwnerActor;
	class INavLinkHostInterface* LinkOwnerHost;

public:
	NAVIGATIONSYSTEM_API SIZE_T GetTypeHash() const override;

	struct FNavLinkDrawing
	{
		FNavLinkDrawing() {}
		FNavLinkDrawing(const FTransform& InLocalToWorld, const FNavigationLink& Link)
			: Left(InLocalToWorld.TransformPosition(Link.Left))
			, Right(InLocalToWorld.TransformPosition(Link.Right))
			, Direction(Link.Direction)
			, Color(UNavArea::GetColor(Link.GetAreaClass()))
			, SnapRadius(Link.SnapRadius)
			, SnapHeight(Link.bUseSnapHeight ? Link.SnapHeight : -1.0f)
			, SupportedAgentsBits(Link.SupportedAgents.PackedBits)
		{}

		FVector Left;
		FVector Right;
		ENavLinkDirection::Type Direction;
		FColor Color;
		float SnapRadius;
		float SnapHeight;
		uint32 SupportedAgentsBits;
	};
	struct FNavLinkSegmentDrawing
	{
		FNavLinkSegmentDrawing() {}
		FNavLinkSegmentDrawing(const FTransform& InLocalToWorld, const FNavigationSegmentLink& Link)
			: LeftStart(InLocalToWorld.TransformPosition(Link.LeftStart))
			, LeftEnd(InLocalToWorld.TransformPosition(Link.LeftEnd))
			, RightStart(InLocalToWorld.TransformPosition(Link.RightStart))
			, RightEnd(InLocalToWorld.TransformPosition(Link.RightEnd))
			, Direction(Link.Direction)
			, Color(UNavArea::GetColor(Link.GetAreaClass()))
			, SnapRadius(Link.SnapRadius)
			, SnapHeight(Link.bUseSnapHeight ? Link.SnapHeight : -1.0f)
			, SupportedAgentsBits(Link.SupportedAgents.PackedBits)
		{}

		FVector LeftStart, LeftEnd;
		FVector RightStart, RightEnd;
		ENavLinkDirection::Type Direction;
		FColor Color;
		float SnapRadius;
		float SnapHeight;
		uint32 SupportedAgentsBits;
	};

private:
	TArray<FNavLinkDrawing> OffMeshPointLinks;
	TArray<FNavLinkSegmentDrawing> OffMeshSegmentLinks;

public:
	/** Initialization constructor. */
	NAVIGATIONSYSTEM_API FNavLinkRenderingProxy(const UPrimitiveComponent* InComponent);
	NAVIGATIONSYSTEM_API virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;
	NAVIGATIONSYSTEM_API virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
	NAVIGATIONSYSTEM_API virtual uint32 GetMemoryFootprint( void ) const override;
	NAVIGATIONSYSTEM_API uint32 GetAllocatedSize( void ) const;
	NAVIGATIONSYSTEM_API void StorePointLinks(const FTransform& LocalToWorld, const TArray<FNavigationLink>& LinksArray);
	NAVIGATIONSYSTEM_API void StoreSegmentLinks(const FTransform& LocalToWorld, const TArray<FNavigationSegmentLink>& LinksArray);

	static NAVIGATIONSYSTEM_API void GetLinkMeshes(const TArray<FNavLinkDrawing>& OffMeshPointLinks, const TArray<FNavLinkSegmentDrawing>& OffMeshSegmentLinks, TArray<float>& StepHeights, FMaterialRenderProxy* const MeshColorInstance, int32 ViewIndex, FMeshElementCollector& Collector, uint32 AgentMask);

	/** made static to allow consistent navlinks drawing even if something is drawing links without FNavLinkRenderingProxy */
	static NAVIGATIONSYSTEM_API void DrawLinks(FPrimitiveDrawInterface* PDI, TArray<FNavLinkDrawing>& OffMeshPointLinks, TArray<FNavLinkSegmentDrawing>& OffMeshSegmentLinks, TArray<float>& StepHeights, FMaterialRenderProxy* const MeshColorInstance, uint32 AgentMask);
};
