// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "AI/Navigation/NavLinkDefinition.h"
#include "NavLinkTrivial.generated.h"

UCLASS(MinimalAPI)
class UNavLinkTrivial : public UNavLinkDefinition
{
	GENERATED_UCLASS_BODY()
};
