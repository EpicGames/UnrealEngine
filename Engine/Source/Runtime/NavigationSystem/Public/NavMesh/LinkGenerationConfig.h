// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SubclassOf.h"
#include "AI/Navigation/NavigationTypes.h"

#include "LinkGenerationConfig.generated.h"

class UBaseGeneratedNavLinksProxy;
class UNavAreaBase;

#if WITH_RECAST
struct dtNavLinkBuilderJumpConfig;
#endif //WITH_RECAST

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class ENavLinkBuilderFlags : uint16
{
	CreateCenterPointLink		= 1 << 0,
	CreateExtremityLink			= 1 << 1,
};
ENUM_CLASS_FLAGS(ENavLinkBuilderFlags);

/** Experimental configuration to generate vertical links. */
USTRUCT()
struct FNavLinkGenerationJumpConfig
{
	GENERATED_BODY()

	NAVIGATIONSYSTEM_API FNavLinkGenerationJumpConfig();

	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	~FNavLinkGenerationJumpConfig() = default;
	FNavLinkGenerationJumpConfig(const FNavLinkGenerationJumpConfig&) = default;
	FNavLinkGenerationJumpConfig(FNavLinkGenerationJumpConfig&&) = default;
	FNavLinkGenerationJumpConfig& operator=(const FNavLinkGenerationJumpConfig&) = default;
	FNavLinkGenerationJumpConfig& operator=(FNavLinkGenerationJumpConfig&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Should this config be used to generate links. */
	UPROPERTY(EditAnywhere, Config, Category = Settings)
	bool bEnabled = true;

	/** A name for this config */
	UPROPERTY(EditAnywhere, Config, Category = Settings)
	FName Name;
	
	/** Horizontal length of the jump.
	 * How far from the starting point we will look for ground. */ 
	UPROPERTY(EditAnywhere, Config, Category = Settings, meta=(UIMin=0, ClampMin=0, UIMax=10000))
	float JumpLength = 150.f; 

	/** How far from the navmesh edge is the jump started. */
	UPROPERTY(EditAnywhere, Config, Category = Settings, meta=(UIMin=0, ClampMin=0))
	float JumpDistanceFromEdge = 10.f; 

	/** How far below the starting height we want to look for landing ground.
	 * A negative value can be used to generate a trajectory landing above the starting height
	 * (if negative, make sure to use a JumpHeight value big enough).
	 */
	UPROPERTY(EditAnywhere, Config, Category = Settings)
	float JumpMaxDepth = 150.f;

	/** Peak height relative to the height of the starting point. */
	UPROPERTY(EditAnywhere, Config, Category = Settings, meta=(UIMin=0, ClampMin=0))
	float JumpHeight = 50.f;

	/** Tolerance at both ends of the jump to find ground. */
	UPROPERTY(EditAnywhere, Config, Category = Settings, meta=(UIMin=0, ClampMin=0))
	float JumpEndsHeightTolerance = 80.f;

	/** Value multiplied by CellSize to find the distance between sampling trajectories. Default is 1.
     *  Larger values improve generation speed but might introduce sampling errors.  */
	UPROPERTY(EditAnywhere, Config, Category = Settings, meta=(UIMin=1, ClampMin=1))
	float SamplingSeparationFactor = 1.f;
	
	/** When filtering similar links, it's the distance used to compare between segment endpoints to match similar links.
	 * Use greater distance for more filtering (0 to deactivate filtering). */
	UPROPERTY(EditAnywhere, Config, Category = Settings, meta=(UIMin=0, ClampMin=0))
	float FilterDistanceThreshold = 80.f;

	/** Flags used to indicate how links will be added. */
	UPROPERTY(EditAnywhere, Config, Category = Settings, meta = (Bitmask, BitmaskEnum = "/Script/NavigationSystem.ENavLinkBuilderFlags"))
	uint16 LinkBuilderFlags = (uint16)ENavLinkBuilderFlags::CreateCenterPointLink;

#if WITH_EDITORONLY_DATA
	/** Area class for links generated by this configuration. */
	UE_DEPRECATED(5.6, "AreaClass is deprecated")
	UPROPERTY(Config, meta = (DeprecatedProperty, DeprecationMessage = "Use DownDirectionAreaClass and UpDirectionAreaClass instead."))
	TSubclassOf<UNavAreaBase> AreaClass_DEPRECATED;
#endif //WITH_EDITORONLY_DATA

	/**
	 * Area class for downward traversal of links generated by this configuration.
	 * @note If the value matches UpDirectionAreaClass, a single bidirectional link will be used to represent links generated by this configuration.
	 * If the value is null, no link will be generated in this direction
	 */
	UPROPERTY(EditAnywhere, Config, Category = Settings)
	TSubclassOf<UNavAreaBase> DownDirectionAreaClass;

	/**
	 * Area class for upward traversal of links generated by this configuration.
	 * @note If the value matches DownDirectionAreaClass, a single bidirectional link will be used to represent links generated by this configuration.
	 * If the value is null, no link will be generated in this direction
	 */
	UPROPERTY(EditAnywhere, Config, Category = Settings)
	TSubclassOf<UNavAreaBase> UpDirectionAreaClass;

	/** Class used to handle links made with this configuration.
	 * Using this allows to implement custom behaviors when using navlinks, for example during the pathfollow.
	 * Note that having a proxy is not required for successful navlink pathfinding,
	 * but it does allow for custom behavior at the start and the end of a given navlink.
	 * This implies that using LinkProxyClass is optional, and it can remain empty (the default value).
	 * @see INavLinkCustomInterface 
	 * @see UGeneratedNavLinksProxy
	 */
	UPROPERTY(EditAnywhere, Category= Settings)
	TSubclassOf<UBaseGeneratedNavLinksProxy> LinkProxyClass;

	/** Identifier used identify the current proxy handler. All links generated through this config will use the same handler. */
	UPROPERTY()
	FNavLinkId LinkProxyId;

	/** Current proxy. The proxy instance is build from the LinkProxyClass (provided it's not null).
	 * A proxy will be created if a @see LinkProxyClass is used.
	 */
	UPROPERTY(SkipSerialization)
	TObjectPtr<UBaseGeneratedNavLinksProxy> LinkProxy = nullptr;

	/** Is the link proxy registered to the navigation system CustomNavLinksMap.
	 * Registration occurs on PostRegisterAllComponents or on PostLoadPreRebuild if a new proxy was created. */
	UPROPERTY(SkipSerialization)
	bool bLinkProxyRegistered = false;

	/** Implemented for deprecated property cleanup purposes. */
	bool Serialize(FArchive& Ar);

#if WITH_RECAST	
	/** Copy configuration to dtNavLinkBuilderJumpConfig. */
	void CopyToDetourConfig(dtNavLinkBuilderJumpConfig& OutDetourConfig) const;
#endif //WITH_RECAST
};

template<> struct TStructOpsTypeTraits<FNavLinkGenerationJumpConfig> : public TStructOpsTypeTraitsBase2<FNavLinkGenerationJumpConfig>
{
	enum
	{
		WithSerializer = true
	};
};


PRAGMA_DISABLE_DEPRECATION_WARNINGS
struct dtNavLinkBuilderJumpDownConfig;

/** Experimental configuration to generate vertical links. */
USTRUCT()
struct UE_DEPRECATED(5.7, "Use FNavLinkGenerationJumpConfig instead.") FNavLinkGenerationJumpDownConfig
{
	GENERATED_BODY()

	FNavLinkGenerationJumpDownConfig();

	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	~FNavLinkGenerationJumpDownConfig() = default;
	FNavLinkGenerationJumpDownConfig(const FNavLinkGenerationJumpDownConfig&) = default;
	FNavLinkGenerationJumpDownConfig(FNavLinkGenerationJumpDownConfig&&) = default;
	FNavLinkGenerationJumpDownConfig& operator=(const FNavLinkGenerationJumpDownConfig&) = default;
	FNavLinkGenerationJumpDownConfig& operator=(FNavLinkGenerationJumpDownConfig&&) = default;

	/** Should this config be used to generate links. */
	UPROPERTY(Config)		// @todo Hidden for now. Since there is currently only one config, it has no real usage.
	bool bEnabled = true;
	
	/** Horizontal length of the jump.
	 * How far from the starting point we will look for ground. */ 
	UPROPERTY(EditAnywhere, Config, Category = Settings, meta=(UIMin=0, ClampMin=0))
	float JumpLength = 150.f; 

	/** How far from the navmesh edge is the jump started. */
	UPROPERTY(EditAnywhere, Config, Category = Settings, meta=(UIMin=0, ClampMin=0))
	float JumpDistanceFromEdge = 10.f; 

	/** How far below the starting height we want to look for landing ground. */
	UPROPERTY(EditAnywhere, Config, Category = Settings, meta=(UIMin=0, ClampMin=0))
	float JumpMaxDepth = 150.f;

	/** Peak height relative to the height of the starting point. */
	UPROPERTY(EditAnywhere, Config, Category = Settings, meta=(UIMin=0, ClampMin=0))
	float JumpHeight = 50.f;

	/** Tolerance at both ends of the jump to find ground. */
	UPROPERTY(EditAnywhere, Config, Category = Settings, meta=(UIMin=0, ClampMin=0))
	float JumpEndsHeightTolerance = 80.f;

	/** Value multiplied by CellSize to find the distance between sampling trajectories. Default is 1.
     *  Larger values improve generation speed but might introduce sampling errors.  */
	UPROPERTY(EditAnywhere, Config, Category = Settings, meta=(UIMin=1, ClampMin=1))
	float SamplingSeparationFactor = 1.f;
	
	/** When filtering similar links, it's the distance used to compare between segment endpoints to match similar links.
	 * Use greater distance for more filtering (0 to deactivate filtering). */
	UPROPERTY(EditAnywhere, Config, Category = Settings, meta=(UIMin=0, ClampMin=0))
	float FilterDistanceThreshold = 80.f;

	/** Flags used to indicate how links will be added. */
	UPROPERTY(EditAnywhere, Config, Category = Settings, meta = (Bitmask, BitmaskEnum = "/Script/NavigationSystem.ENavLinkBuilderFlags"))
	uint16 LinkBuilderFlags = (uint16)ENavLinkBuilderFlags::CreateCenterPointLink;

#if WITH_EDITORONLY_DATA
	/** Area class for links generated by this configuration. */
	UE_DEPRECATED(5.6, "AreaClass is deprecated")
	UPROPERTY(Config, meta = (DeprecatedProperty, DeprecationMessage = "Use DownDirectionAreaClass and UpDirectionAreaClass instead."))
	TSubclassOf<UNavAreaBase> AreaClass_DEPRECATED;
#endif //WITH_EDITORONLY_DATA

	/**
	 * Area class for downward traversal of links generated by this configuration.
	 * @note If the value matches UpDirectionAreaClass, a single bidirectional link will be used to represent links generated by this configuration.
	 * If the value is null, no link will be generated in this direction
	 */
	UPROPERTY(EditAnywhere, Config, Category = Settings)
	TSubclassOf<UNavAreaBase> DownDirectionAreaClass;

	/**
	 * Area class for upward traversal of links generated by this configuration.
	 * @note If the value matches DownDirectionAreaClass, a single bidirectional link will be used to represent links generated by this configuration.
	 * If the value is null, no link will be generated in this direction
	 */
	UPROPERTY(EditAnywhere, Config, Category = Settings)
	TSubclassOf<UNavAreaBase> UpDirectionAreaClass;

	/** Class used to handle links made with this configuration.
	 * Using this allows to implement custom behaviors when using navlinks, for example during the pathfollow.
	 * Note that having a proxy is not required for successful navlink pathfinding,
	 * but it does allow for custom behavior at the start and the end of a given navlink.
	 * This implies that using LinkProxyClass is optional, and it can remain empty (the default value).
	 * @see INavLinkCustomInterface 
	 * @see UGeneratedNavLinksProxy
	 */
	UPROPERTY(EditAnywhere, Category= Settings)
	TSubclassOf<UBaseGeneratedNavLinksProxy> LinkProxyClass;

	/** Identifier used identify the current proxy handler. All links generated through this config will use the same handler. */
	UPROPERTY()
	FNavLinkId LinkProxyId;

	/** Current proxy. The proxy instance is build from the LinkProxyClass (provided it's not null).
	 * A proxy will be created if a @see LinkProxyClass is used.
	 */
	UPROPERTY(Transient)
	TObjectPtr<UBaseGeneratedNavLinksProxy> LinkProxy = nullptr;

	/** Is the link proxy registered to the navigation system CustomNavLinksMap.
	 * Registration occurs on PostRegisterAllComponents or on PostLoadPreRebuild if a new proxy was created. */
	UPROPERTY(Transient)
	bool bLinkProxyRegistered = false;

	/** Implemented for deprecated property cleanup purposes. */
	bool Serialize(FArchive& Ar);

#if WITH_RECAST	
	/** Copy configuration to dtNavLinkBuilderJumpDownConfig. */
	void CopyToDetourConfig(dtNavLinkBuilderJumpDownConfig& OutDetourConfig) const;
#endif //WITH_RECAST
};

template<> struct TStructOpsTypeTraits<FNavLinkGenerationJumpDownConfig> : public TStructOpsTypeTraitsBase2<FNavLinkGenerationJumpDownConfig>
{
	enum
	{
		WithSerializer = true
	};
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS