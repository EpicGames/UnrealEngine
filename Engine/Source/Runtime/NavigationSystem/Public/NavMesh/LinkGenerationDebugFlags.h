// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "LinkGenerationDebugFlags.generated.h"

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class ELinkGenerationDebugFlags : uint16
{
	WalkableSurface				    = 1 << 0,
	WalkableBorders				    = 1 << 1,
	SelectedEdge				    = 1 << 2,
	SelectedEdgeTrajectory		    = 1 << 3,
	SelectedEdgeLandingSamples	    = 1 << 4,
	SelectedEdgeCollisions		    = 1 << 5,
	SelectedEdgeCollisionsSamples	= 1 << 6,
	Links						    = 1 << 7,
	FilteredLinks				    = 1 << 8,
};
ENUM_CLASS_FLAGS(ELinkGenerationDebugFlags);
