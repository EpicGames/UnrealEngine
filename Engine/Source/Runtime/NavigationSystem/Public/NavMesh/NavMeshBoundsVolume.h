// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * this volume only blocks the path builder - it has no gameplay collision
 *
 */

#pragma once

#include "UObject/ObjectMacros.h"
#include "GameFramework/Volume.h"
#include "AI/Navigation/NavAgentSelector.h"
#include "NavMeshBoundsVolume.generated.h"


UCLASS(MinimalAPI)
class ANavMeshBoundsVolume : public AVolume
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Navigation)
	FNavAgentSelector SupportedAgents;

	/* If true, then this volume will have a BodyInstance and calls to EncompassesPoint will function as expected. */
	UPROPERTY(EditAnywhere, Category = Navigation)
	uint8 bAllowPhysicsOverlap : 1 = false;

	//~ Begin AActor Interface
	NAVIGATIONSYSTEM_API virtual void PostRegisterAllComponents() override;
	NAVIGATIONSYSTEM_API virtual void PostUnregisterAllComponents() override;
	//~ End AActor Interface
#if WITH_EDITOR
	//~ Begin UObject Interface
	NAVIGATIONSYSTEM_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
	NAVIGATIONSYSTEM_API virtual void PostEditUndo() override;
	//~ End UObject Interface

	static NAVIGATIONSYSTEM_API void OnPostEngineInit();
#endif // WITH_EDITOR
};

