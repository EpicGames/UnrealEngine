// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "NavigationPath.h"
//#include "NavigationPath.generated.h"

#define RECAST_STRAIGHTPATH_OFFMESH_CONNECTION 0x04

/** Helper to translate FNavPathPoint.Flags. */
struct FNavMeshNodeFlags
{
	/** Extra node information (like "path start", "off-mesh connection"). */
	uint8 PathFlags;
	/** Area type after this node. */
	uint8 Area;
	/** Area flags for this node. */
	uint16 AreaFlags;

	FNavMeshNodeFlags() : PathFlags(0), Area(0), AreaFlags(0) {}
	FNavMeshNodeFlags(const uint32 Flags) : PathFlags((uint8)Flags), Area((uint8)(Flags >> 8)), AreaFlags((uint16)(Flags >> 16)) {}
	uint32 Pack() const { return PathFlags | ((uint32)Area << 8) | ((uint32)AreaFlags << 16); }
	bool IsNavLink() const { return (PathFlags & RECAST_STRAIGHTPATH_OFFMESH_CONNECTION) != 0; }

	FNavMeshNodeFlags& AddAreaFlags(const uint16 InAreaFlags)
	{
		AreaFlags = (AreaFlags | InAreaFlags);
		return *this;
	}
};


struct FNavMeshPath : public FNavigationPath
{
	typedef FNavigationPath Super;

	NAVIGATIONSYSTEM_API FNavMeshPath();
	NAVIGATIONSYSTEM_API ~FNavMeshPath();

	NAVIGATIONSYSTEM_API FNavMeshPath(const FNavMeshPath&);
	NAVIGATIONSYSTEM_API FNavMeshPath(FNavMeshPath&& Other);
	NAVIGATIONSYSTEM_API FNavMeshPath& operator=(const FNavMeshPath& Other);
	NAVIGATIONSYSTEM_API FNavMeshPath& operator=(FNavMeshPath&& Other);

	inline void SetWantsStringPulling(const bool bNewWantsStringPulling) { bWantsStringPulling = bNewWantsStringPulling; }
	inline bool WantsStringPulling() const { return bWantsStringPulling; }
	inline bool IsStringPulled() const { return bStringPulled; }

	/** find string pulled path from PathCorridor */
	NAVIGATIONSYSTEM_API void PerformStringPulling(const FVector& StartLoc, const FVector& EndLoc);

	inline void SetWantsPathCorridor(const bool bNewWantsPathCorridor) { bWantsPathCorridor = bNewWantsPathCorridor; }
	inline bool WantsPathCorridor() const { return bWantsPathCorridor; }

	inline const TArray<FNavigationPortalEdge>& GetPathCorridorEdges() const { return bCorridorEdgesGenerated ? PathCorridorEdges : GeneratePathCorridorEdges(); }
	inline void SetPathCorridorEdges(const TArray<FNavigationPortalEdge>& InPathCorridorEdges) { PathCorridorEdges = InPathCorridorEdges; bCorridorEdgesGenerated = true; }

	inline void OnPathCorridorUpdated() { bCorridorEdgesGenerated = false; }

	NAVIGATIONSYSTEM_API virtual void DebugDraw(const ANavigationData* NavData, const FColor PathColor, UCanvas* Canvas, const bool bPersistent, const float LifeTime, const uint32 NextPathPointIndex = 0) const override;

	NAVIGATIONSYSTEM_API bool ContainsWithSameEnd(const FNavMeshPath* Other) const;

	NAVIGATIONSYSTEM_API void OffsetFromCorners(FVector::FReal Distance);

	NAVIGATIONSYSTEM_API void ApplyFlags(int32 NavDataFlags);

	NAVIGATIONSYSTEM_API virtual void ResetForRepath() override;

	/** get flags of path point or corridor poly (depends on bStringPulled flag) */
	NAVIGATIONSYSTEM_API bool GetNodeFlags(int32 NodeIdx, FNavMeshNodeFlags& Flags) const;

	/** get cost of path, starting from next poly in corridor */
	virtual FVector::FReal GetCostFromNode(NavNodeRef PathNode) const override { return GetCostFromIndex(PathCorridor.Find(PathNode) + 1); }

	/** get cost of path, starting from given point */
	virtual FVector::FReal GetCostFromIndex(int32 PathPointIndex) const override
	{
		FVector::FReal TotalCost = 0.f;
		const FVector::FReal* Cost = PathCorridorCost.GetData();
		for (int32 PolyIndex = PathPointIndex; PolyIndex < PathCorridorCost.Num(); ++PolyIndex, ++Cost)
		{
			TotalCost += *Cost;
		}

		return TotalCost;
	}

	inline FVector::FReal GetTotalPathLength() const
	{
		return bStringPulled ? GetStringPulledLength(0) : GetPathCorridorLength(0);
	}

	inline int32 GetNodeRefIndex(const NavNodeRef NodeRef) const { return PathCorridor.Find(NodeRef); }

	/** check if path (all polys in corridor) contains given node */
	virtual bool ContainsNode(NavNodeRef NodeRef) const override { return PathCorridor.Contains(NodeRef); }

	virtual bool ContainsCustomLink(FNavLinkId UniqueLinkId) const override { return CustomNavLinkIds.Contains(UniqueLinkId); }
	virtual bool ContainsAnyCustomLink() const override { return CustomNavLinkIds.Num() > 0; }

	NAVIGATIONSYSTEM_API bool IsPathSegmentANavLink(const int32 PathSegmentStartIndex) const;

	NAVIGATIONSYSTEM_API virtual bool DoesIntersectBox(const FBox& Box, uint32 StartingIndex = 0, int32* IntersectingSegmentIndex = NULL, FVector* AgentExtent = NULL) const override;
	NAVIGATIONSYSTEM_API virtual bool DoesIntersectBox(const FBox& Box, const FVector& AgentLocation, uint32 StartingIndex = 0, int32* IntersectingSegmentIndex = NULL, FVector* AgentExtent = NULL) const override;
	/** retrieves normalized direction vector to given path segment. If path is not string pulled navigation corridor is being used */
	NAVIGATIONSYSTEM_API virtual FVector GetSegmentDirection(uint32 SegmentEndIndex) const override;

	NAVIGATIONSYSTEM_API void Invert();

private:
	NAVIGATIONSYSTEM_API bool DoesPathIntersectBoxImplementation(const FBox& Box, const FVector& StartLocation, uint32 StartingIndex, int32* IntersectingSegmentIndex, FVector* AgentExtent) const;
	NAVIGATIONSYSTEM_API void InternalResetNavMeshPath();

public:

#if ENABLE_VISUAL_LOG
	NAVIGATIONSYSTEM_API virtual void DescribeSelfToVisLog(struct FVisualLogEntry* Snapshot) const override;
	NAVIGATIONSYSTEM_API virtual FString GetDescription() const override;
#endif // ENABLE_VISUAL_LOG

protected:
	/** calculates total length of string pulled path. Does not generate string pulled
	*	path if it's not already generated (see bWantsStringPulling and bStrigPulled)
	*	Internal use only */
	NAVIGATIONSYSTEM_API FVector::FReal GetStringPulledLength(const int32 StartingPoint) const;

	/** calculates estimated length of path expressed as sequence of navmesh edges.
	*	It basically sums up distances between every subsequent nav edge pair edge middles.
	*	Internal use only */
	NAVIGATIONSYSTEM_API FVector::FReal GetPathCorridorLength(const int32 StartingEdge) const;

	/** it's only const to be callable in const environment. It's not supposed to be called directly externally anyway,
	*	just as part of retrieving corridor on demand or generating it in internal processes. It fills a mutable
	*	array. */
	NAVIGATIONSYSTEM_API const TArray<FNavigationPortalEdge>& GeneratePathCorridorEdges() const;

public:

	/** sequence of navigation mesh poly ids representing an obstacle-free navigation corridor */
	TArray<NavNodeRef> PathCorridor;

	/** for every poly in PathCorridor stores traversal cost from previous navpoly */
	TArray<FVector::FReal> PathCorridorCost;

	/** set of unique link Ids */
	UE_DEPRECATED(5.3, "LinkIds are now based on FNavLinkId. Use CustomNavLinkIds instead. CustomLinkIds array is no longer populated or used in the engine")
	TArray<uint32> CustomLinkIds;

	TArray<FNavLinkId> CustomNavLinkIds;

private:
	/** sequence of FVector pairs where each pair represents navmesh portal edge between two polygons navigation corridor.
	*	Note, that it should always be accessed via GetPathCorridorEdges() since PathCorridorEdges content is generated
	*	on first access */
	mutable TArray<FNavigationPortalEdge> PathCorridorEdges;

	/** transient variable indicating whether PathCorridorEdges contains up to date information */
	mutable uint32 bCorridorEdgesGenerated : 1;

public:
	/** is this path generated on dynamic navmesh (i.e. one attached to moving surface) */
	uint32 bDynamic : 1;

protected:
	/** does this path contain string pulled path? If true then NumPathVerts > 0 and
	*	OutPathVerts contains valid data. If false there's only navigation corridor data
	*	available.*/
	uint32 bStringPulled : 1;

	/** If set to true path instance will contain a string pulled version. Otherwise only
	*	navigation corridor will be available. Defaults to true */
	uint32 bWantsStringPulling : 1;

	/** If set to true path instance will contain path corridor generated as a part
	*	pathfinding call (i.e. without the need to generate it with GeneratePathCorridorEdges */
	uint32 bWantsPathCorridor : 1;

public:
	static NAVIGATIONSYSTEM_API const FNavPathType Type;
};
