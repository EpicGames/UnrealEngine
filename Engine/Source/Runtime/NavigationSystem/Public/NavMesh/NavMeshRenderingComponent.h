// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "PrimitiveViewRelevance.h"
#include "MaterialShared.h"
#include "DynamicMeshBuilder.h"
#include "DebugRenderSceneProxy.h"
#include "Debug/DebugDrawComponent.h"
#include "MeshBatch.h"
#include "LocalVertexFactory.h"
#include "Math/GenericOctree.h"
#include "StaticMeshResources.h"
#include "NavigationSystemTypes.h"
#include "Templates/UnrealTemplate.h"
#include "NavMeshRenderingComponent.generated.h"

class APlayerController;
class ARecastNavMesh;
class FColoredMaterialRenderProxy;
class FMeshElementCollector;
class FPrimitiveDrawInterface;
class UCanvas;
class UNavMeshRenderingComponent;

enum class ENavMeshDetailFlags : uint8
{
	TriangleEdges,
	PolyEdges,
	BoundaryEdges,
	FilledPolys,
	TileBounds,
	PathCollidingGeometry,
	TileLabels,
	PolygonLabels,
	PolygonCost,
	PolygonFlags,
	PolygonAreaIDs,
	PathLabels,
	NavLinks,
	FailedNavLinks,
	Clusters,
	NavOctree,
	NavOctreeDetails,
	MarkForbiddenPolys,
	TileBuildTimes,
	TileBuildTimesHeatMap,
	TileResolutions,
	OnScreenInformation
};

// exported to API for GameplayDebugger module
struct FNavMeshSceneProxyData : public TSharedFromThis<FNavMeshSceneProxyData, ESPMode::ThreadSafe>
{
	struct FDebugMeshData
	{
		TArray<FDynamicMeshVertex> Vertices;
		TArray<uint32> Indices;
		FColor ClusterColor;
	};
	TArray<FDebugMeshData> MeshBuilders;

	struct FDebugPoint
	{
		FDebugPoint(const FVector& InPosition, const FColor& InColor, const float InSize) : Position(InPosition), Color(InColor), Size(InSize) {}
		FVector Position;
		FColor Color;
		float Size = 0.f;
	};

	TArray<FDebugRenderSceneProxy::FDebugLine> ThickLineItems;
	TArray<FDebugRenderSceneProxy::FDebugLine> TileEdgeLines;
	TArray<FDebugRenderSceneProxy::FDebugLine> NavMeshEdgeLines;
	TArray<FDebugRenderSceneProxy::FDebugLine> NavLinkLines;
	TArray<FDebugRenderSceneProxy::FDebugLine> ClusterLinkLines;
	TArray<FDebugRenderSceneProxy::FDebugLine> AuxLines;
	TArray<FDebugPoint> AuxPoints;
	TArray<FDebugRenderSceneProxy::FDebugBox> AuxBoxes;
	TArray<FDebugRenderSceneProxy::FMesh> Meshes;

	struct FDebugText
	{
		FVector Location;
		FString Text;

		FDebugText() {}
		FDebugText(const FVector& InLocation, const FString& InText) : Location(InLocation), Text(InText) {}
		FDebugText(const FString& InText) : Location(FNavigationSystem::InvalidLocation), Text(InText) {}
	};
	TArray<FDebugText> DebugLabels;
	
	TArray<FBoxCenterAndExtent>	OctreeBounds;

	FBox Bounds;
	FVector NavMeshDrawOffset;
	uint32 bDataGathered : 1;
	uint32 bNeedsNewData : 1;
	int32 NavDetailFlags;

	FNavMeshSceneProxyData() : NavMeshDrawOffset(0, 0, 10.f),
		bDataGathered(false), bNeedsNewData(true), NavDetailFlags(0) {}

	NAVIGATIONSYSTEM_API void Reset();
	NAVIGATIONSYSTEM_API void Serialize(FArchive& Ar);
	NAVIGATIONSYSTEM_API uint32 GetAllocatedSize() const;

#if WITH_RECAST
	UE_DEPRECATED(5.5, "Use the version of this function that takes an array of FNavTileRefs instead")
	NAVIGATIONSYSTEM_API void GatherData(const ARecastNavMesh* NavMesh, int32 InNavDetailFlags, const TArray<int32>& TileSet);
	NAVIGATIONSYSTEM_API void GatherData(const ARecastNavMesh* NavMesh, int32 InNavDetailFlags, const TArray<struct FNavTileRef>& TileSet);

#if RECAST_INTERNAL_DEBUG_DATA
	NAVIGATIONSYSTEM_API void AddMeshForInternalData(const struct FRecastInternalDebugData& InInternalData);
#endif //RECAST_INTERNAL_DEBUG_DATA

#endif
};

// exported to API for GameplayDebugger module
class FNavMeshSceneProxy final : public FDebugRenderSceneProxy, public FNoncopyable
{
	friend class FNavMeshDebugDrawDelegateHelper;
public:
	NAVIGATIONSYSTEM_API virtual SIZE_T GetTypeHash() const override;

	NAVIGATIONSYSTEM_API FNavMeshSceneProxy(const UPrimitiveComponent* InComponent, FNavMeshSceneProxyData* InProxyData, bool ForceToRender = false);
	NAVIGATIONSYSTEM_API virtual ~FNavMeshSceneProxy() override;

	NAVIGATIONSYSTEM_API virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;

protected:
	NAVIGATIONSYSTEM_API virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;

	virtual uint32 GetMemoryFootprint(void) const override { return sizeof(*this) + GetAllocatedSizeInternal(); }
	NAVIGATIONSYSTEM_API uint32 GetAllocatedSizeInternal(void) const;

private:			
	FNavMeshSceneProxyData ProxyData;

	FDynamicMeshIndexBuffer32 IndexBuffer;
	FStaticMeshVertexBuffers VertexBuffers;
	FLocalVertexFactory VertexFactory;

	TArray<TUniquePtr<FColoredMaterialRenderProxy>> MeshColors;
	TArray<FMeshBatchElement> MeshBatchElements;

	FDebugDrawDelegate DebugTextDrawingDelegate;
	FDelegateHandle DebugTextDrawingDelegateHandle;
	TWeakObjectPtr<UNavMeshRenderingComponent> RenderingComponent;
	uint32 bForceRendering : 1;
	uint32 bSkipDistanceCheck : 1;
	uint32 bUseThickLines : 1;
};

#if WITH_RECAST && UE_ENABLE_DEBUG_DRAWING
class FNavMeshDebugDrawDelegateHelper : public FDebugDrawDelegateHelper
{
	typedef FDebugDrawDelegateHelper Super;

public:
	FNavMeshDebugDrawDelegateHelper()
		: bForceRendering(false)
		, bNeedsNewData(false)
	{
	}

	void SetupFromProxy(const FNavMeshSceneProxy* InSceneProxy)
	{
		DebugLabels.Reset();
		DebugLabels.Append(InSceneProxy->ProxyData.DebugLabels);
		bForceRendering = InSceneProxy->bForceRendering;
		bNeedsNewData = InSceneProxy->ProxyData.bNeedsNewData;
	}

	void Reset()
	{
		DebugLabels.Reset();
		bNeedsNewData = true;
	}

protected:
	NAVIGATIONSYSTEM_API virtual void DrawDebugLabels(UCanvas* Canvas, APlayerController*) override;

private:
	TArray<FNavMeshSceneProxyData::FDebugText> DebugLabels;
	uint32 bForceRendering : 1;
	uint32 bNeedsNewData : 1;
};
#endif

UCLASS(editinlinenew, ClassGroup = Debug, MinimalAPI)
class UNavMeshRenderingComponent : public UDebugDrawComponent
{
	GENERATED_UCLASS_BODY()

public:
	void ForceUpdate() { bForceUpdate = true; }
	bool IsForcingUpdate() const { return bForceUpdate; }

	static NAVIGATIONSYSTEM_API bool IsNavigationShowFlagSet(const UWorld* World);

protected:
	NAVIGATIONSYSTEM_API virtual void OnRegister()  override;
	NAVIGATIONSYSTEM_API virtual void OnUnregister()  override;

#if UE_ENABLE_DEBUG_DRAWING
  	NAVIGATIONSYSTEM_API virtual FDebugRenderSceneProxy* CreateDebugSceneProxy() override;
#if WITH_RECAST
	virtual FDebugDrawDelegateHelper& GetDebugDrawDelegateHelper() override { return NavMeshDebugDrawDelegateManager; }
#endif // WITH_RECAST
#endif // UE_ENABLE_DEBUG_DRAWING

	NAVIGATIONSYSTEM_API virtual FBoxSphereBounds CalcBounds(const FTransform &LocalToWorld) const override;

	/** Gathers drawable information from NavMesh and puts it in OutProxyData. 
	 *	Override to add additional information to OutProxyData.*/
	NAVIGATIONSYSTEM_API virtual void GatherData(const ARecastNavMesh& NavMesh, FNavMeshSceneProxyData& OutProxyData) const;

	NAVIGATIONSYSTEM_API void TimerFunction();

protected:
	uint32 bCollectNavigationData : 1;
	uint32 bForceUpdate : 1;
	FTimerHandle TimerHandle;

protected:
#if WITH_RECAST && UE_ENABLE_DEBUG_DRAWING
	FNavMeshDebugDrawDelegateHelper NavMeshDebugDrawDelegateManager;
#endif
};

namespace FNavMeshRenderingHelpers
{
	NAVIGATIONSYSTEM_API void AddVertex(FNavMeshSceneProxyData::FDebugMeshData& MeshData, const FVector& Pos, const FColor Color = FColor::White);

	NAVIGATIONSYSTEM_API void AddTriangleIndices(FNavMeshSceneProxyData::FDebugMeshData& MeshData, int32 V0, int32 V1, int32 V2);
}
