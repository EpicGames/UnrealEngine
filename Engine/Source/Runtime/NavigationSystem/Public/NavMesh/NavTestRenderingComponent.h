// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debug/DebugDrawComponent.h"
#include "AI/Navigation/NavigationTypes.h" // NavNodeRef

#include "NavTestRenderingComponent.generated.h"

#if UE_ENABLE_DEBUG_DRAWING

class FPrimitiveSceneProxy;
class ANavigationTestingActor;
class APlayerController;
class FMeshElementCollector;
class UCanvas;
class UNavTestRenderingComponent;
struct FNodeDebugData;

class FNavTestSceneProxy final : public FDebugRenderSceneProxy
{
	friend class FNavTestDebugDrawDelegateHelper;

public:
	virtual SIZE_T GetTypeHash() const override;

	struct FNodeDebugData
	{
		NavNodeRef PolyRef;
		FVector Position;
		FString Desc;
		FSetElementId ParentId;
		uint32 bClosedSet : 1;
		uint32 bBestPath : 1;
		uint32 bModified : 1;
		uint32 bOffMeshLink : 1;

		inline bool operator==(const FNodeDebugData& Other) const
		{
			return PolyRef == Other.PolyRef;
		}
		inline friend uint32 GetTypeHash(const FNodeDebugData& Other)
		{
			return ::GetTypeHash(Other.PolyRef);
		}
	};

	explicit FNavTestSceneProxy(const UNavTestRenderingComponent* InComponent);

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;

	void GatherPathPoints();

	void GatherPathStep();

	inline static bool LocationInView(const FVector& Location, const FSceneView* View)
	{
		return View->ViewFrustum.IntersectBox(Location, FVector::ZeroVector);
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;

	virtual uint32 GetMemoryFootprint() const override { return sizeof(*this) + GetAllocatedSizeInternal(); }

private:
	uint32 GetAllocatedSizeInternal() const;

	FVector3f NavMeshDrawOffset;

	uint32 bShowBestPath : 1;
	uint32 bShowNodePool : 1;
	uint32 bShowDiff : 1;

	ANavigationTestingActor* NavTestActor;
	TArray<FVector> PathPoints;
	TArray<FString> PathPointFlags;

	TArray<FDynamicMeshVertex> OpenSetVerts;
	TArray<uint32> OpenSetIndices;
	TArray<FDynamicMeshVertex> ClosedSetVerts;
	TArray<uint32> ClosedSetIndices;
	TSet<FNodeDebugData> NodeDebug;
	FSetElementId BestNodeId;

	FVector ClosestWallLocation;
};

class FNavTestDebugDrawDelegateHelper : public FDebugDrawDelegateHelper
{
	typedef FDebugDrawDelegateHelper Super;

public:
	FNavTestDebugDrawDelegateHelper(): bShowBestPath(false), bShowDiff(false)
	{
	}

	void SetupFromProxy(const FNavTestSceneProxy* InSceneProxy);

protected:
	virtual void DrawDebugLabels(UCanvas* Canvas, APlayerController*) override;

private:
	TSet<FNavTestSceneProxy::FNodeDebugData> NodeDebug;
	ANavigationTestingActor* NavTestActor = nullptr;
	TArray<FVector> PathPoints;
	TArray<FString> PathPointFlags;
	FSetElementId BestNodeId;
	uint32 bShowBestPath : 1;
	uint32 bShowDiff : 1;
};

#endif // UE_ENABLE_DEBUG_DRAWING

UCLASS(ClassGroup = Debug)
class UNavTestRenderingComponent: public UDebugDrawComponent
{
	GENERATED_BODY()

protected:

	virtual FBoxSphereBounds CalcBounds(const FTransform &LocalToWorld) const override;

#if UE_ENABLE_DEBUG_DRAWING
	virtual FDebugRenderSceneProxy* CreateDebugSceneProxy() override;
	virtual FDebugDrawDelegateHelper& GetDebugDrawDelegateHelper() override { return NavTestDebugDrawDelegateHelper; }
private:
	FNavTestDebugDrawDelegateHelper NavTestDebugDrawDelegateHelper;
#endif // UE_ENABLE_DEBUG_DRAWING
};

