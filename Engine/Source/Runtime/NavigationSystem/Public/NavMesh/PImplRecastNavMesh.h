// Copyright Epic Games, Inc. All Rights Reserved.

//
// Private implementation for communication with Recast library
// 
// All functions should be called through RecastNavMesh actor to make them thread safe!
//

#pragma once 

#include "UObject/WeakObjectPtr.h"
#include "AI/Navigation/NavigationTypes.h"
#include "NavMesh/RecastNavMesh.h"
#include "NavMesh/RecastQueryFilter.h"
#include "AI/NavigationSystemBase.h"
#include "VisualLogger/VisualLogger.h"

#if RECAST_INTERNAL_DEBUG_DATA
#include "NavMesh/RecastInternalDebugData.h"
#endif

#if WITH_RECAST
#include "Detour/DetourNavMesh.h"
#include "Detour/DetourNavMeshQuery.h"
#endif

class FRecastNavMeshGenerator;
struct FNavLinkId;

#if WITH_RECAST

#define RECAST_VERY_SMALL_AGENT_RADIUS 0.0f

/** Engine Private! - Private Implementation details of ARecastNavMesh */
class FPImplRecastNavMesh
{
public:

	/** Constructor */
	NAVIGATIONSYSTEM_API FPImplRecastNavMesh(ARecastNavMesh* Owner);

	/** Dtor */
	NAVIGATIONSYSTEM_API ~FPImplRecastNavMesh();

	/**
	 * Serialization.
	 * @param Ar - The archive with which to serialize.
	 * @returns true if serialization was successful.
	 */
	NAVIGATIONSYSTEM_API void Serialize(FArchive& Ar, int32 NavMeshVersion);

	/* Gather debug geometry.
	 * @params OutGeometry Output geometry.
	 * @params TileIndex Used to collect geometry for a specific tile, INDEX_NONE will gather all tiles.
	 * @return True if done collecting.
	 */
	UE_DEPRECATED(5.5, "Use the version of this function that takes a FNavTileRef instead.")
	NAVIGATIONSYSTEM_API bool GetDebugGeometryForTile(FRecastDebugGeometry& OutGeometry, int32 TileIndex) const;

	/* Gather debug geometry.
	 * @params OutGeometry Output geometry.
	 * @params TileRef Used to collect geometry for a specific tile, an invalid FNavTileRef will gather all tiles.
	 * @return True if done collecting.
	 */
	NAVIGATIONSYSTEM_API bool GetDebugGeometryForTile(FRecastDebugGeometry& OutGeometry, FNavTileRef TileRef) const;
	
	/** Returns bounding box for the whole navmesh. */
	NAVIGATIONSYSTEM_API FBox GetNavMeshBounds() const;

	/** Retrieves XY coordinates of tile specified by position */
	NAVIGATIONSYSTEM_API bool GetNavMeshTileXY(const FVector& Point, int32& OutX, int32& OutY) const;

	/** Retrieves list of tiles that intersect specified bounds */
	NAVIGATIONSYSTEM_API void GetNavMeshTilesIn(const TArray<FBox>& InclusionBounds, TArray<FNavTileRef>& OutRefs) const;

	/** Retrieves number of tiles in this navmesh */
	inline int32 GetNavMeshTilesCount() const { return DetourNavMesh ? DetourNavMesh->getMaxTiles() : 0; }

	/** Supported queries */

	/** Generates path from the given query. Synchronous. */
	NAVIGATIONSYSTEM_API ENavigationQueryResult::Type FindPath(const FVector& StartLoc, const FVector& EndLoc, const FVector::FReal CostLimit, const bool bRequireNavigableEndLocation, FNavMeshPath& Path, const FNavigationQueryFilter& Filter, const UObject* Owner) const;

	/** Check if path exists */
	NAVIGATIONSYSTEM_API ENavigationQueryResult::Type TestPath(const FVector& StartLoc, const FVector& EndLoc, const bool bRequireNavigableEndLocation, const FNavigationQueryFilter& Filter, const UObject* Owner, int32* NumVisitedNodes = 0) const;

	template< typename TRecastAStar, typename TRecastAStartGraph, typename TRecastGraphAStarFilter, typename TRecastAStarResult >
	ENavigationQueryResult::Type FindPathCustomAStar(TRecastAStartGraph& RecastGraphWrapper, TRecastAStar& AStarAlgo, const FVector& StartLoc, const FVector& EndLoc, const FVector::FReal CostLimit, FNavMeshPath& Path, const FNavigationQueryFilter& Filter, const UObject* Owner) const;

	/** Checks if the whole segment is in navmesh */
	NAVIGATIONSYSTEM_API void Raycast(const FVector& StartLoc, const FVector& EndLoc, const FNavigationQueryFilter& InQueryFilter, const UObject* Owner,
		ARecastNavMesh::FRaycastResult& RaycastResult, NavNodeRef StartNode = INVALID_NAVNODEREF) const;

	/** Generates path from given query and collect data for every step of A* algorithm */
	NAVIGATIONSYSTEM_API int32 DebugPathfinding(const FVector& StartLoc, const FVector& EndLoc, const FVector::FReal CostLimit, const bool bRequireNavigableEndLocation, const FNavigationQueryFilter& Filter, const UObject* Owner, TArray<FRecastDebugPathfindingData>& Steps);

	/** Returns a random location on the navmesh. */
	NAVIGATIONSYSTEM_API FNavLocation GetRandomPoint(const FNavigationQueryFilter& Filter, const UObject* Owner) const;

#if WITH_NAVMESH_CLUSTER_LINKS
	/** Check if path exists using cluster graph */
	NAVIGATIONSYSTEM_API ENavigationQueryResult::Type TestClusterPath(const FVector& StartLoc, const FVector& EndLoc, int32* NumVisitedNodes = 0) const;

	/** Returns a random location on the navmesh within cluster */
	NAVIGATIONSYSTEM_API bool GetRandomPointInCluster(NavNodeRef ClusterRef, FNavLocation& OutLocation) const;
#endif // WITH_NAVMESH_CLUSTER_LINKS

	/**	Tries to move current nav location towards target constrained to navigable area. Faster than ProjectPointToNavmesh.
	 *	@param OutLocation if successful this variable will be filed with result
	 *	@return true if successful, false otherwise
	 */
	NAVIGATIONSYSTEM_API bool FindMoveAlongSurface(const FNavLocation& StartLocation, const FVector& TargetPosition, FNavLocation& OutLocation, const FNavigationQueryFilter& Filter, const UObject* Owner) const;

	NAVIGATIONSYSTEM_API bool ProjectPointToNavMesh(const FVector& Point, FNavLocation& Result, const FVector& Extent, const FNavigationQueryFilter& Filter, const UObject* Owner) const;
	
	/** Project single point and grab all vertical intersections */
	NAVIGATIONSYSTEM_API bool ProjectPointMulti(const FVector& Point, TArray<FNavLocation>& OutLocations, const FVector& Extent,
		FVector::FReal MinZ, FVector::FReal MaxZ, const FNavigationQueryFilter& Filter, const UObject* Owner) const;

	/** Returns nearest navmesh polygon to Loc, or INVALID_NAVNODEREF if Loc is not on the navmesh. */
	NAVIGATIONSYSTEM_API NavNodeRef FindNearestPoly(FVector const& Loc, FVector const& Extent, const FNavigationQueryFilter& Filter, const UObject* Owner) const;

	/** Finds the polygons along the navigation graph that touch the specified circle. Return true if found any. */
	NAVIGATIONSYSTEM_API bool FindPolysAroundCircle(const FVector& CenterPos, const NavNodeRef CenterNodeRef, const FVector::FReal Radius, const FNavigationQueryFilter& Filter, const UObject* Owner, TArray<NavNodeRef>* OutPolys = nullptr, TArray<NavNodeRef>* OutPolysParent = nullptr, TArray<float>* OutPolysCost = nullptr, int32* OutPolysCount = nullptr) const;

	/** Retrieves all polys within given pathing distance from StartLocation.
	 *	@NOTE query is not using string-pulled path distance (for performance reasons),
	 *		it measured distance between middles of portal edges, do you might want to 
	 *		add an extra margin to PathingDistance */
	NAVIGATIONSYSTEM_API bool GetPolysWithinPathingDistance(FVector const& StartLoc, const FVector::FReal PathingDistance,
		const FNavigationQueryFilter& Filter, const UObject* Owner,
		TArray<NavNodeRef>& FoundPolys, FRecastDebugPathfindingData* DebugData) const;

	//@todo document
	NAVIGATIONSYSTEM_API void GetEdgesForPathCorridor(const TArray<NavNodeRef>* PathCorridor, TArray<FNavigationPortalEdge>* PathCorridorEdges) const;

	/** finds stringpulled path from given corridor */
	NAVIGATIONSYSTEM_API bool FindStraightPath(const FVector& StartLoc, const FVector& EndLoc, const TArray<NavNodeRef>& PathCorridor, TArray<FNavPathPoint>& PathPoints, TArray<FNavLinkId>* CustomLinks = NULL) const;

	/** Filters nav polys in PolyRefs with Filter */
	NAVIGATIONSYSTEM_API bool FilterPolys(TArray<NavNodeRef>& PolyRefs, const FRecastQueryFilter* Filter, const UObject* Owner) const;

	/** Get all polys from tile */
	UE_DEPRECATED(5.5, "Use the version of this function in ARecastNavMesh that takes a FNavTileRef instead.")
	NAVIGATIONSYSTEM_API bool GetPolysInTile(int32 TileIndex, TArray<FNavPoly>& Polys) const;

	/** Updates area on polygons creating point-to-point connection with given UserId */
	NAVIGATIONSYSTEM_API void UpdateNavigationLinkArea(FNavLinkId UserId, uint8 AreaType, uint16 PolyFlags) const;

#if WITH_NAVMESH_SEGMENT_LINKS
	/** Updates area on polygons creating segment-to-segment connection with given UserId */
	NAVIGATIONSYSTEM_API void UpdateSegmentLinkArea(int32 UserId, uint8 AreaType, uint16 PolyFlags) const;

	/** Creates segment link connections for the give tile */
	NAVIGATIONSYSTEM_API void ProcessSegmentLinksForTile(FNavTileRef TileRef) const;

	/** Creates segment link connections for the give tile and stores the list of indices skipped to avoid generating duplicate links */
	NAVIGATIONSYSTEM_API void ProcessSegmentLinksForTile(FNavTileRef TileRef, uint32 MaxSkippedNeigborTiles, dtTileRef* OutSkippedNeigborTiles, uint32& OutNumSkippedNeigborTiles) const;
#endif // WITH_NAVMESH_SEGMENT_LINKS

	/** Retrieves center of the specified polygon. Returns false on error. */
	NAVIGATIONSYSTEM_API bool GetPolyCenter(NavNodeRef PolyID, FVector& OutCenter) const;
	/** Retrieves the vertices for the specified polygon. Returns false on error. */
	NAVIGATIONSYSTEM_API bool GetPolyVerts(NavNodeRef PolyID, TArray<FVector>& OutVerts) const;
	/** Retrieves a random point inside the specified polygon. Returns false on error. */
	NAVIGATIONSYSTEM_API bool GetRandomPointInPoly(NavNodeRef PolyID, FVector& OutPoint) const;
	/** Retrieves the surface area of the specified polygon. Returns 0 on error. */
	NAVIGATIONSYSTEM_API FVector::FReal GetPolySurfaceArea(NavNodeRef PolyID) const;
	/** Retrieves the flags for the specified polygon. Returns false on error. */
	NAVIGATIONSYSTEM_API bool GetPolyData(NavNodeRef PolyID, uint16& Flags, uint8& AreaType) const;
	/** Retrieves area ID for the specified polygon. */
	NAVIGATIONSYSTEM_API uint32 GetPolyAreaID(NavNodeRef PolyID) const;
	/** Sets area ID for the specified polygon. */
	NAVIGATIONSYSTEM_API void SetPolyAreaID(NavNodeRef PolyID, uint8 AreaID);
	/** Finds all polys connected with specified one */
	NAVIGATIONSYSTEM_API bool GetPolyNeighbors(NavNodeRef PolyID, TArray<FNavigationPortalEdge>& Neighbors) const;
	/** Finds all polys connected with specified one, results expressed as array of NavNodeRefs */
	NAVIGATIONSYSTEM_API bool GetPolyNeighbors(NavNodeRef PolyID, TArray<NavNodeRef>& Neighbors) const;
	/** Finds all polys connected with specified one */
	NAVIGATIONSYSTEM_API bool GetPolyEdges(NavNodeRef PolyID, TArray<FNavigationPortalEdge>& Edges) const;
	/** Finds all wall segments for the specified polygon (walls or area borders) */
	NAVIGATIONSYSTEM_API bool GetPolyWallSegments(NavNodeRef PolyID, const FNavigationQueryFilter& InQueryFilter, const UObject* QueryOwner, TArray<FNavigationPortalEdge>& OutNeighbors) const;
	/** Finds closest point constrained to given poly */
	NAVIGATIONSYSTEM_API bool GetClosestPointOnPoly(NavNodeRef PolyID, const FVector& TestPt, FVector& PointOnPoly) const;
	/** Decode poly ID into tile index and poly index */
	NAVIGATIONSYSTEM_API bool GetPolyTileIndex(NavNodeRef PolyID, uint32& PolyIndex, uint32& TileIndex) const;
	/** Decode poly ID into FNavTileRef and poly index */
	NAVIGATIONSYSTEM_API bool GetPolyTileRef(NavNodeRef PolyId, uint32& OutPolyIndex, FNavTileRef& OutTileRef) const;
	/** Retrieves user ID for given offmesh link poly */
	UE_DEPRECATED(5.3, "Please use GetNavLinkUserId() instead. This function only returns Invalid.")
	uint32 GetLinkUserId(NavNodeRef LinkPolyID) const
	{
		return static_cast<int32>(FNavLinkId::Invalid.GetId());
	}
	NAVIGATIONSYSTEM_API FNavLinkId GetNavLinkUserId(NavNodeRef LinkPolyID) const;
	/** Retrieves start and end point of offmesh link */
	NAVIGATIONSYSTEM_API bool GetLinkEndPoints(NavNodeRef LinkPolyID, FVector& PointA, FVector& PointB) const;
	/** Check if poly is a custom link */
	NAVIGATIONSYSTEM_API bool IsCustomLink(NavNodeRef PolyRef) const;

#if	WITH_NAVMESH_CLUSTER_LINKS
	/** Retrieves bounds of cluster. Returns false on error. */
	NAVIGATIONSYSTEM_API bool GetClusterBounds(NavNodeRef ClusterRef, FBox& OutBounds) const;
	NAVIGATIONSYSTEM_API NavNodeRef GetClusterRefFromPolyRef(const NavNodeRef PolyRef) const;
#endif // WITH_NAVMESH_CLUSTER_LINKS

	uint32 GetTileIndexFromPolyRef(const NavNodeRef PolyRef) const { return DetourNavMesh != NULL ? DetourNavMesh->decodePolyIdTile(PolyRef) : uint32(INDEX_NONE); }

	static NAVIGATIONSYSTEM_API uint16 GetFilterForbiddenFlags(const FRecastQueryFilter* Filter);
	static NAVIGATIONSYSTEM_API void SetFilterForbiddenFlags(FRecastQueryFilter* Filter, uint16 ForbiddenFlags);

	NAVIGATIONSYSTEM_API void OnAreaCostChanged();

public:
	dtNavMesh const* GetRecastMesh() const { return DetourNavMesh; };
	dtNavMesh* GetRecastMesh() { return DetourNavMesh; };
	NAVIGATIONSYSTEM_API void ReleaseDetourNavMesh();

	NAVIGATIONSYSTEM_API void RemoveTileCacheLayers(int32 TileX, int32 TileY);
	NAVIGATIONSYSTEM_API void RemoveTileCacheLayer(int32 TileX, int32 TileY, int32 LayerIdx);
	NAVIGATIONSYSTEM_API void AddTileCacheLayers(int32 TileX, int32 TileY, const TArray<FNavMeshTileData>& Layers);
	NAVIGATIONSYSTEM_API void AddTileCacheLayer(int32 TileX, int32 TileY, int32 LayerIdx, const FNavMeshTileData& LayerData);
	NAVIGATIONSYSTEM_API void MarkEmptyTileCacheLayers(int32 TileX, int32 TileY);
	NAVIGATIONSYSTEM_API FNavMeshTileData GetTileCacheLayer(int32 TileX, int32 TileY, int32 LayerIdx) const;
	NAVIGATIONSYSTEM_API TArray<FNavMeshTileData> GetTileCacheLayers(int32 TileX, int32 TileY) const;
	NAVIGATIONSYSTEM_API bool HasTileCacheLayers(int32 TileX, int32 TileY) const;

	/** Assigns recast generated navmesh to this instance.
	 *	@param bOwnData if true from now on this FPImplRecastNavMesh instance will be responsible for this piece 
	 *		of memory
	 */
	NAVIGATIONSYSTEM_API void SetRecastMesh(dtNavMesh* NavMesh);

	NAVIGATIONSYSTEM_API SIZE_T GetTotalDataSize() const;

	/** Gets the size of the compressed tile cache, this is slow */
#if !UE_BUILD_SHIPPING
	NAVIGATIONSYSTEM_API int32 GetCompressedTileCacheSize();
#endif

	/** Called on world origin changes */
	NAVIGATIONSYSTEM_API void ApplyWorldOffset(const FVector& InOffset, bool bWorldShift);

	/** calculated cost of given segment if traversed on specified poly. Function measures distance between specified points
	 *	and returns cost of traversing this distance on given poly.
	 *	@note no check if segment is on poly is performed. */
	NAVIGATIONSYSTEM_API FVector::FReal CalcSegmentCostOnPoly(NavNodeRef PolyID, const dtQueryFilter* Filter, const FVector& StartLoc, const FVector& EndLoc) const;

	ARecastNavMesh* NavMeshOwner;
	
	/** Recast's runtime navmesh data that we can query against */
	dtNavMesh* DetourNavMesh;

	/** Compressed layers data, can be reused for tiles generation */
	TMap<FIntPoint, TArray<FNavMeshTileData> > CompressedTileCacheLayers;

#if RECAST_INTERNAL_DEBUG_DATA
	TMap<FIntPoint, FRecastInternalDebugData> DebugDataMap;
#endif

	/** query used for searching data on game thread */
	mutable dtNavMeshQuery SharedNavQuery;

	/** Helper function to serialize a single Recast tile. */
	static NAVIGATIONSYSTEM_API void SerializeRecastMeshTile(FArchive& Ar, int32 NavMeshVersion, unsigned char*& TileData, int32& TileDataSize);

	/** Helper function to serialize a Recast tile compressed data. */
	static NAVIGATIONSYSTEM_API void SerializeCompressedTileCacheData(FArchive& Ar, int32 NavMeshVersion, unsigned char*& CompressedData, int32& CompressedDataSize);

	/** Initialize data for pathfinding
	* @param UnrealStart Start location for the desired path
	* @param UnrealEnd End location for the desired path
	* @param Query Query to use to project start and end position to the navmesh
	* @param Filter Filters which polygons are acceptable to project start and end locations
	* @param RecastStart Outputs the start location snapped to the navmesh
	* @param StartPoly Outputs the navmesh poly at the start location
	* @param RecastEnd Outputs the end location snapped to the navmesh. Equal to UnrealEnd when there's no navmesh at the end location.
	* @param EndPoly Outputs the navmesh poly at the end location. Invalid when there's no navmesh at the end location
	* @param OutPath [optional] When provided, Path will be used to store relevant error codes that this method can produce
	*/
	NAVIGATIONSYSTEM_API bool InitPathfinding(const FVector& UnrealStart, const FVector& UnrealEnd, 
		const dtNavMeshQuery& Query, const dtQueryFilter* Filter,
		FVector& RecastStart, dtPolyRef& StartPoly,
		FVector& RecastEnd, dtPolyRef& EndPoly,
		FNavMeshPath* OutPath = nullptr) const;

	/** Marks path flags, perform string pulling if needed */
	NAVIGATIONSYSTEM_API void PostProcessPath(dtStatus PathfindResult, FNavMeshPath& Path,
		const dtNavMeshQuery& Query, const dtQueryFilter* Filter,
		NavNodeRef StartNode, NavNodeRef EndNode,
		FVector UnrealStart, FVector UnrealEnd,
		FVector RecastStart, FVector RecastEnd,
		dtQueryResult& PathResult) const;

	UE_DEPRECATED(5.5, "Use GetTilePolyEdges instead.")
	NAVIGATIONSYSTEM_API void GetDebugPolyEdges(const dtMeshTile& Tile, bool bInternalEdges, bool bNavMeshEdges, TArray<FVector>& InternalEdgeVerts, TArray<FVector>& NavMeshEdgeVerts) const;

	/**
	 * Traverses given tile's edges and detects the ones that are either internal poly (i.e. not triangle, but whole navmesh polygon)
	 * or external navmesh edge. Returns a pair of verts for each edge found.
	 * @param Tile The tile whose edges to traverse.
	 * @param bGatherInteriorPolyEdges If true, populates OutPolyEdgeVerts with the tile's internal poly edges.
	 * @param bGatherExternalNavMeshEdges If true, populates OutNavMeshEdgeVerts with the tile's external navmesh edges.
	 * @param OutInteriorPolyEdgeVerts Output poly edge vertex array. Contains a pair of verts for each edge.
	 * @param OutExteriorNavMeshEdgeVerts Output navmesh edge vertex array. Contains a pair of verts for each edge.
	 * @note This is really slow.
	 */
	NAVIGATIONSYSTEM_API void GetTilePolyEdges(const dtMeshTile& Tile, bool bGatherInteriorPolyEdges, bool bGatherExteriorNavMeshEdges, TArray<FVector>& OutInteriorPolyEdgeVerts, TArray<FVector>& OutExteriorNavMeshEdgeVerts) const;

	/** workhorse function finding portal edges between corridor polys */
	NAVIGATIONSYSTEM_API void GetEdgesForPathCorridorImpl(const TArray<NavNodeRef>* PathCorridor, TArray<FNavigationPortalEdge>* PathCorridorEdges, const dtNavMeshQuery& NavQuery) const;

	/** 
	 * Traverse a tile and returns a list of all the edges around the passed in filter.
	 * @param Tile The tile whose edges to traverse.
	 * @param InQueryFilter the filter used to determine if an edge should be considered a border
	 * @param OutEdges Output edges array
	*/
	NAVIGATIONSYSTEM_API void GetTilePolyEdgesForFilter(const dtMeshTile& Tile, const FNavigationQueryFilter& InQueryFilter, TArray<FNavigationWallEdge>& OutEdges) const;

protected:
	/** 
	 *	@param ForbiddenFlags polys with flags matching the filter will get added to 
	 */
	NAVIGATIONSYSTEM_API int32 GetTilesDebugGeometry(const FRecastNavMeshGenerator* Generator, const dtMeshTile& Tile, int32 VertBase, FRecastDebugGeometry& OutGeometry, int32 TileIdx = INDEX_NONE, uint16 ForbiddenFlags = 0) const;

	NAVIGATIONSYSTEM_API ENavigationQueryResult::Type PostProcessPathInternal(dtStatus FindPathStatus, FNavMeshPath& Path, 
		const dtNavMeshQuery& NavQuery, const dtQueryFilter* QueryFilter, 
		NavNodeRef StartPolyID, NavNodeRef EndPolyID, 
		const FVector& RecastStartPos, const FVector& RecastEndPos, 
		dtQueryResult& PathResult) const;
};

template< typename TRecastAStar, typename TRecastAStartGraph, typename TRecastGraphAStarFilter, typename TRecastAStarResult >
ENavigationQueryResult::Type FPImplRecastNavMesh::FindPathCustomAStar(TRecastAStartGraph& RecastGraphWrapper, TRecastAStar& AStarAlgo, const FVector& StartLoc, const FVector& EndLoc, const FVector::FReal CostLimit, FNavMeshPath& Path, const FNavigationQueryFilter& InQueryFilter, const UObject* Owner) const
{
	const FRecastQueryFilter* FilterImplementation = (const FRecastQueryFilter*)(InQueryFilter.GetImplementation());
	if (FilterImplementation == nullptr)
	{
		UE_VLOG(NavMeshOwner, LogNavigation, Error, TEXT("FPImplRecastNavMesh::FindPath failed due to passed filter having NULL implementation!"));
		return ENavigationQueryResult::Error;
	}

	const dtQueryFilter* QueryFilter = FilterImplementation->GetAsDetourQueryFilter();
	if (QueryFilter == nullptr)
	{
		UE_VLOG(NavMeshOwner, LogNavigation, Warning, TEXT("FPImplRecastNavMesh::FindPath failing due to QueryFilter == NULL"));
		return ENavigationQueryResult::Error;
	}

	// initialize recast wrapper with the NavMeshOwner, not multithread safe!
	RecastGraphWrapper.Initialize(NavMeshOwner);
	TRecastGraphAStarFilter AStarFilter(RecastGraphWrapper, *FilterImplementation, InQueryFilter.GetMaxSearchNodes(), CostLimit, Owner);

	FVector RecastStartPos, RecastEndPos;
	NavNodeRef StartPolyID, EndPolyID;
	const bool bCanSearch = InitPathfinding(StartLoc, EndLoc, RecastGraphWrapper.GetRecastQuery(), QueryFilter, RecastStartPos, StartPolyID, RecastEndPos, EndPolyID, &Path);
	if (!bCanSearch)
	{
		return ENavigationQueryResult::Error;
	}

	typename TRecastAStar::FSearchNode StartNode(StartPolyID, RecastStartPos);
	typename TRecastAStar::FSearchNode EndNode(EndPolyID, RecastEndPos);
	StartNode.Initialize(RecastGraphWrapper);
	EndNode.Initialize(RecastGraphWrapper);
	TRecastAStarResult PathResult;
	auto AStarResult = AStarAlgo.FindPath(StartNode, EndNode, AStarFilter, PathResult);

	dtStatus FindPathStatus = RecastGraphWrapper.ConvertToRecastStatus(AStarAlgo, AStarFilter, AStarResult);
	return PostProcessPathInternal(FindPathStatus, Path, RecastGraphWrapper.GetRecastQuery(), FilterImplementation, StartNode.NodeRef, EndNode.NodeRef, 
		FVector(StartNode.Position[0], StartNode.Position[1], StartNode.Position[2]), 
		FVector(EndNode.Position[0], EndNode.Position[1], EndNode.Position[2]),
		PathResult);
}

#endif	// WITH_RECAST
