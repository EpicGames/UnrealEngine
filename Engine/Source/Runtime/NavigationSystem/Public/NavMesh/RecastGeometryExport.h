// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AI/NavigationSystemHelpers.h"
#include "Engine/EngineTypes.h"

struct FNavigationRelevantData;
struct FNavigationElement;

#if WITH_RECAST

/**
 * Class that handles geometry exporting for Recast navmesh generation.
 */
struct FRecastGeometryExport : public FNavigableGeometryExport
{
	NAVIGATIONSYSTEM_API FRecastGeometryExport(FNavigationRelevantData& InData);

	FNavigationRelevantData* Data;
	TNavStatArray<FVector::FReal> VertexBuffer;
	TNavStatArray<int32> IndexBuffer;
	FWalkableSlopeOverride SlopeOverride;

	/** Export the collision of a Chaos triangle mesh into the Vertex and Index buffer. */
	NAVIGATIONSYSTEM_API virtual void ExportChaosTriMesh(const Chaos::FTriangleMeshImplicitObject* const TriMesh, const FTransform& LocalToWorld) override;
	/** Export the collision of a Chaos convex mesh into the Vertex and Index buffer. */
	NAVIGATIONSYSTEM_API virtual void ExportChaosConvexMesh(const FKConvexElem* const Convex, const FTransform& LocalToWorld) override;
	/** Export the collision of a Chaos height field into the Vertex and Index buffer. */
	NAVIGATIONSYSTEM_API virtual void ExportChaosHeightField(const Chaos::FHeightField* const Heightfield, const FTransform& LocalToWorld) override;
	/** Export a slice of the collision of a Chaos height field into the Vertex and Index buffer.
	 * @param SliceBox Box that defines the slice we want to extract from the height field */
	NAVIGATIONSYSTEM_API virtual void ExportChaosHeightFieldSlice(const FNavHeightfieldSamples& PrefetchedHeightfieldSamples, const int32 NumRows, const int32 NumCols, const FTransform& LocalToWorld, const FBox& SliceBox) override;
	/** Export a custom mesh into the Vertex and Index buffer.*/
	NAVIGATIONSYSTEM_API virtual void ExportCustomMesh(const FVector* InVertices, int32 NumVerts, const int32* InIndices, int32 NumIndices, const FTransform& LocalToWorld) override;
	/** Export a rigid body into the Vertex and Index buffer.*/
	NAVIGATIONSYSTEM_API virtual void ExportRigidBodySetup(UBodySetup& BodySetup, const FTransform& LocalToWorld) override;
	/** Add Nav Modifiers to the owned NavigationRelevantData. */
	NAVIGATIONSYSTEM_API virtual void AddNavModifiers(const FCompositeNavModifier& Modifiers) override;
	/** Optional delegate for geometry per instance transforms. */
	NAVIGATIONSYSTEM_API virtual void SetNavDataPerInstanceTransformDelegate(const FNavDataPerInstanceTransformDelegate& InDelegate) override;

	/** Convert the vertices in VertexBuffer from Unreal to Recast coordinates. */
	NAVIGATIONSYSTEM_API void ConvertVertexBufferToRecast();
	/** Store Vertex and Index buffer data in associated FNavigationRelevantData. */
	NAVIGATIONSYSTEM_API void StoreCollisionCache();

	/** Collects the collision information from a navigation element and stores it into the FNavigationRelevantData's CollisionData. */
	static NAVIGATIONSYSTEM_API void ExportElementGeometry(const FNavigationElement& InElement, FNavigationRelevantData& OutData);

	/** Convert a list of vertices into the navigation format and store it into the FNavigationRelevantData's CollisionData.
	 * @param InVerts Array of triangles vertices position. Each triangle will be created from 3 consecutive vertices in the array. Its size must be a multiple a 3.*/
	static NAVIGATIONSYSTEM_API void ExportVertexSoupGeometry(const TArray<FVector>& InVerts, FNavigationRelevantData& OutData);

	/** Collect the collision information of a BodySetup as a triangle mesh. */
	static NAVIGATIONSYSTEM_API void ExportRigidBodyGeometry(UBodySetup& InOutBodySetup,
		TNavStatArray<FVector>& OutVertexBuffer,
		TNavStatArray<int32>& OutIndexBuffer,
		FBox& OutBounds,
		const FTransform& LocalToWorld = FTransform::Identity);

	/** Collect the collision information of a BodySetup as a triangle mesh and a series of convex shapes. */
	static NAVIGATIONSYSTEM_API void ExportRigidBodyGeometry(
		UBodySetup& InOutBodySetup,
		TNavStatArray<FVector>& OutTriMeshVertexBuffer,
		TNavStatArray<int32>& OutTriMeshIndexBuffer,
		TNavStatArray<FVector>& OutConvexVertexBuffer,
		TNavStatArray<int32>& OutConvexIndexBuffer,
		TNavStatArray<int32>& OutShapeBuffer,
		FBox& OutBounds,
		const FTransform& LocalToWorld = FTransform::Identity);

	/** Collect the collision information of an AggregateGeometry as a series of convex shapes. */
	static NAVIGATIONSYSTEM_API void ExportAggregatedGeometry(
		const FKAggregateGeom& AggGeom,
		TNavStatArray<FVector>& OutConvexVertexBuffer,
		TNavStatArray<int32>& OutConvexIndexBuffer,
		TNavStatArray<int32>& OutShapeBuffer,
		FBox& OutBounds,
		const FTransform& LocalToWorld = FTransform::Identity);

	/** Transform a list of vertex triplets from Unreal to Recast coordinate and generate an Index buffer. */
	static NAVIGATIONSYSTEM_API void TransformVertexSoupToRecast(const TArray<FVector>& VertexSoup, TNavStatArray<FVector>& Verts, TNavStatArray<int32>& Faces);

private:
	FString GetDataOwnerName() const;
};

#endif // WITH_RECAST
