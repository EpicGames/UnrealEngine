// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Coord system utilities
 *
 * Translates between Unreal and Recast coords.
 * Unreal: x, y, z
 * Recast: -x, z, -y
 */

#pragma once

#include "Math/Box.h"
#include "Math/Color.h"
#include "Math/Matrix.h"
#include "Math/Vector.h"


extern NAVIGATIONSYSTEM_API FVector Unreal2RecastPoint(const FVector::FReal* UnrealPoint);
extern NAVIGATIONSYSTEM_API FVector Unreal2RecastPoint(const FVector& UnrealPoint);
extern NAVIGATIONSYSTEM_API FBox Unreal2RecastBox(const FBox& UnrealBox);
extern NAVIGATIONSYSTEM_API FMatrix Unreal2RecastMatrix();

extern NAVIGATIONSYSTEM_API FVector Recast2UnrealPoint(const FVector::FReal* RecastPoint);
extern NAVIGATIONSYSTEM_API FVector Recast2UnrealPoint(const FVector& RecastPoint);

extern NAVIGATIONSYSTEM_API FBox Recast2UnrealBox(const FVector::FReal* RecastMin, const FVector::FReal* RecastMax);
extern NAVIGATIONSYSTEM_API FBox Recast2UnrealBox(const FBox& RecastBox);
extern NAVIGATIONSYSTEM_API FMatrix Recast2UnrealMatrix();
extern NAVIGATIONSYSTEM_API FColor Recast2UnrealColor(const unsigned int RecastColor);
