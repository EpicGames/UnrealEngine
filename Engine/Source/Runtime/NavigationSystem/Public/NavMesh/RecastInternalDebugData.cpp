// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavMesh/RecastInternalDebugData.h"

#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "NavMesh/RecastHelpers.h"

#if WITH_RECAST

#include "DebugUtils/DebugDraw.h"

void FRecastInternalDebugData::vertex(const FVector::FReal x, const FVector::FReal y, const FVector::FReal z, unsigned int color, const FVector::FReal u, const FVector::FReal v)
{
	const FVector::FReal RecastPos[3] = { x,y,z };
	const FVector Pos = Recast2UnrealPoint(RecastPos);
	const FColor Color = Recast2UnrealColor(color);
	switch(CurrentPrim)
	{
	case DU_DRAW_POINTS:
		PointVertices.Push(Pos);
		PointColors.Push(Color);
		break;
	case DU_DRAW_LINES:
		LineVertices.Push(Pos);
		LineColors.Push(Color);
		break;
	case DU_DRAW_TRIS:
		// Fallthrough
	case DU_DRAW_QUADS:
		TriangleVertices.Push(Pos);
		TriangleColors.Push(Color);
		break;
	}
}

void FRecastInternalDebugData::text(const FVector::FReal x, const FVector::FReal y, const FVector::FReal z, const char* text)
{
	const FVector::FReal RecastPos[3] = { x,y,z };
	const FVector Pos = Recast2UnrealPoint(RecastPos);
	LabelVertices.Push(Pos);
	Labels.Push(FString(text));
}

void FRecastInternalDebugData::end()
{
	if (CurrentPrim == DU_DRAW_QUADS)
	{
		// Turns quads to triangles
		for (int32 i = FirstVertexIndex; i < TriangleVertices.Num(); i += 4)
		{
			ensure(i + 3 < TriangleVertices.Num());
			TriangleIndices.Push(i + 0);
			TriangleIndices.Push(i + 1);
			TriangleIndices.Push(i + 3);

			TriangleIndices.Push(i + 3);
			TriangleIndices.Push(i + 1);
			TriangleIndices.Push(i + 2);
		}
	}
	else if (CurrentPrim == DU_DRAW_TRIS)
	{
		// Add indices for triangles.
		for (int32 i = FirstVertexIndex; i < TriangleVertices.Num(); i++)
		{
			TriangleIndices.Push(i);
		}
	}
}
#endif // WITH_RECAST
