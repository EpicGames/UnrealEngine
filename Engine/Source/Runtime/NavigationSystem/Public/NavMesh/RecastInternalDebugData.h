// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "NavMesh/RecastHelpers.h"

#if WITH_RECAST

#include "DebugUtils/DebugDraw.h"

struct FRecastInternalDebugData : public duDebugDraw
{
	duDebugDrawPrimitives CurrentPrim = DU_DRAW_POINTS;
	int32 FirstVertexIndex = 0;

	TArray<uint32> TriangleIndices;
	TArray<FVector> TriangleVertices;
	TArray<FColor> TriangleColors;

	TArray<FVector> LineVertices;
	TArray<FColor>  LineColors;

	TArray<FVector> PointVertices;
	TArray<FColor>  PointColors;

	TArray<FVector> LabelVertices;
	TArray<FString> Labels;

	double BuildTime = 0.;
	double BuildCompressedLayerTime = 0.;
	double BuildNavigationDataTime = 0.;
	double BuildLinkTime = 0;

	uint32 TriangleCount = 0;
	unsigned char Resolution = 0;
	
	FRecastInternalDebugData() {}
	virtual ~FRecastInternalDebugData() override = default;

	virtual void depthMask(bool state) override { /*unused*/ };
	virtual void texture(bool state) override { /*unused*/ };

	virtual void begin(duDebugDrawPrimitives prim, float size = 1.0f) override
	{
		CurrentPrim = prim;
		FirstVertexIndex = TriangleVertices.Num();
	}

	virtual void vertex(const FVector::FReal* pos, unsigned int color) override
	{
		vertex(pos[0], pos[1], pos[2], color, 0.0f, 0.0f);
	}

	virtual void vertex(const FVector::FReal x, const FVector::FReal y, const FVector::FReal z, unsigned int color) override
	{
		vertex(x, y, z, color, 0.0f, 0.0f);
	}

	virtual void vertex(const FVector::FReal* pos, unsigned int color, const FVector::FReal* uv) override
	{
		vertex(pos[0], pos[1], pos[2], color, uv[0], uv[1]);
	}

	NAVIGATIONSYSTEM_API virtual void vertex(const FVector::FReal x, const FVector::FReal y, const FVector::FReal z, unsigned int color, const FVector::FReal u, const FVector::FReal v) override;

	NAVIGATIONSYSTEM_API virtual void text(const FVector::FReal x, const FVector::FReal y, const FVector::FReal z, const char* text) override;

	NAVIGATIONSYSTEM_API virtual void end() override;
};
#endif // WITH_RECAST
