// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "EngineDefines.h"
#include "LinkGenerationConfig.h"
#include "AI/Navigation/NavigationTypes.h"
#include "AI/Navigation/NavigationDataResolution.h"
#include "NavigationSystemTypes.h"
#include "NavigationData.h"
#include "NavMesh/NavMeshPath.h"
#include "RecastNavMesh.generated.h"

#define RECAST_MAX_SEARCH_NODES		2048

#define RECAST_MIN_TILE_SIZE		300.f

#define RECAST_MAX_AREAS			64
#define RECAST_DEFAULT_AREA			(RECAST_MAX_AREAS - 1)
#define RECAST_LOW_AREA				(RECAST_MAX_AREAS - 2)
#define RECAST_NULL_AREA			0
// Note poly costs are still floats in UE so we are using FLT_MAX as unwalkable still. 
// Path CostLimit and Distance are based on FVector::FReal.
#define RECAST_UNWALKABLE_POLY_COST	FLT_MAX 

// If set, recast will use async workers for rebuilding tiles in runtime
// All access to tile data must be guarded with critical sections
#ifndef RECAST_ASYNC_REBUILDING
#define RECAST_ASYNC_REBUILDING	1
#endif

//If set we will time slice the nav regen if RECAST_ASYNC_REBUILDING is 0
#ifndef ALLOW_TIME_SLICE_NAV_REGEN
#define ALLOW_TIME_SLICE_NAV_REGEN 0
#endif

//TIME_SLICE_NAV_REGEN must be 0 if we are async rebuilding recast
#define TIME_SLICE_NAV_REGEN (ALLOW_TIME_SLICE_NAV_REGEN && !RECAST_ASYNC_REBUILDING)

// LWC_TODO_AI Note we are using int32 here for X and Y which does mean that we could overflow the limit of an int for LWCoords
// unless fairly large tile sizes are used.As WORLD_MAX is currently in flux until we have a better idea of what we are
// going to be able to support its probably not worth investing time in this potential issue right now.

class FPImplRecastNavMesh;
class FRecastQueryFilter;
class INavLinkCustomInterface;
class UCanvas;
class UNavArea;
class UNavigationDataChunk;
class UPrimitiveComponent;
class URecastNavMeshDataChunk;
class ARecastNavMesh;
struct FRecastAreaNavModifierElement;
class dtNavMesh;
class dtQueryFilter;
class FRecastNavMeshGenerator;
struct dtMeshTile;
class UNavigationSystemV1;
class UNavigationSystemBase;

UENUM()
namespace ERecastPartitioning
{
	// keep in sync with rcRegionPartitioning enum!

	enum Type : int
	{
		Monotone,
		Watershed,
		ChunkyMonotone,
	};
}

UENUM()
enum class ENavigationLedgeSlopeFilterMode : uint8
{
	Recast,							// Use walkableClimb value to filter
	None,							// Skip slope filtering
	UseStepHeightFromAgentMaxSlope	// Use maximum step height computed from AgentMaxSlope
};

struct FDetourTileSizeInfo
{
	unsigned short VertCount = 0;
	unsigned short PolyCount = 0;
	unsigned short MaxLinkCount = 0;
	unsigned short DetailMeshCount = 0;
	unsigned short DetailVertCount = 0;
	unsigned short DetailTriCount = 0;
	unsigned short BvNodeCount = 0;
	unsigned short OffMeshConCount = 0;
	unsigned short OffMeshSegConCount = 0;
	unsigned short ClusterCount = 0;
	unsigned short OffMeshBase = 0;
};

struct FDetourTileLayout
{
	NAVIGATIONSYSTEM_API FDetourTileLayout(const dtMeshTile& tile);
	NAVIGATIONSYSTEM_API FDetourTileLayout(const FDetourTileSizeInfo& SizeInfo);

private:
	void InitFromSizeInfo(const FDetourTileSizeInfo& SizeInfo);

public:
	int32 HeaderSize = 0;
	int32 VertsSize = 0;
	int32 PolysSize = 0;
	int32 LinksSize = 0;
	int32 DetailMeshesSize = 0;
	int32 DetailVertsSize = 0;
	int32 DetailTrisSize = 0;
	int32 BvTreeSize = 0;
	int32 OffMeshConsSize = 0;
	int32 OffMeshSegsSize = 0;
	int32 ClustersSize = 0;
	int32 PolyClustersSize = 0;
	int32 TileSize = 0;
};

namespace ERecastPathFlags
{
	/** If set, path won't be post processed. */
	inline const int32 SkipStringPulling = (1 << 0);

	/** If set, path will contain navigation corridor. */
	inline const int32 GenerateCorridor = (1 << 1);

	/** Make your game-specific flags start at this index */
	inline const uint8 FirstAvailableFlag = 2;
}

#if WITH_RECAST
struct FRecastDebugPathfindingNode
{
	NavNodeRef PolyRef;
	NavNodeRef ParentRef;
	FVector::FReal Cost = 0.;
	FVector::FReal TotalCost = 0.;
	FVector::FReal Length = 0.;

	FVector NodePos;
	TArray<FVector3f, TInlineAllocator<6> > Verts; // LWC_TODO: Precision loss. Issue here is regarding debug rendering needing to work with FVector3f.
	uint8 NumVerts;

	uint8 bOpenSet : 1;
	uint8 bOffMeshLink : 1;
	uint8 bModified : 1;

	FRecastDebugPathfindingNode() : PolyRef(0), ParentRef(0), NumVerts(0), bOpenSet(0), bOffMeshLink(0), bModified(0) {}
	FRecastDebugPathfindingNode(NavNodeRef InPolyRef) : PolyRef(InPolyRef), ParentRef(0), NumVerts(0), bOpenSet(0), bOffMeshLink(0), bModified(0) {}

	inline bool operator==(const NavNodeRef& OtherPolyRef) const { return PolyRef == OtherPolyRef; }
	inline bool operator==(const FRecastDebugPathfindingNode& Other) const { return PolyRef == Other.PolyRef; }
	inline friend uint32 GetTypeHash(const FRecastDebugPathfindingNode& Other) { return GetTypeHash(Other.PolyRef); }

	inline FVector::FReal GetHeuristicCost() const { return TotalCost - Cost; }
};

namespace ERecastDebugPathfindingFlags
{
	enum Type : uint8
	{
		Basic = 0x0,
		BestNode = 0x1,
		Vertices = 0x2,
		PathLength = 0x4
	};
}

struct FRecastDebugPathfindingData
{
	TSet<FRecastDebugPathfindingNode> Nodes;
	FSetElementId BestNode;
	uint8 Flags;

	FRecastDebugPathfindingData() : Flags(ERecastDebugPathfindingFlags::Basic) {}
	FRecastDebugPathfindingData(ERecastDebugPathfindingFlags::Type InFlags) : Flags(InFlags) {}
};

struct FRecastDebugGeometry
{
	enum EOffMeshLinkEnd
	{
		OMLE_None = 0x0,
		OMLE_Left = 0x1,
		OMLE_Right = 0x2,
		OMLE_Both = OMLE_Left | OMLE_Right
	};

	struct FOffMeshLink
	{
		FVector Left;
		FVector Right;
		uint8	AreaID;
		uint8	Direction;
		uint8	ValidEnds;
		bool	bIsGenerated;
		float	Radius;
		float	Height;
		FColor	Color;
	};

#if WITH_NAVMESH_CLUSTER_LINKS
	struct FCluster
	{
		TArray<int32> MeshIndices;
	};

	struct FClusterLink
	{
		FVector FromCluster;
		FVector ToCluster;
	};
#endif // WITH_NAVMESH_CLUSTER_LINKS

// This is an unsupported feature and has not been finished to production quality.
#if WITH_NAVMESH_SEGMENT_LINKS
	struct FOffMeshSegment
	{
		FVector LeftStart, LeftEnd;
		FVector RightStart, RightEnd;
		uint8	AreaID;
		uint8	Direction;
		uint8	ValidEnds;
	};
#endif // WITH_NAVMESH_SEGMENT_LINKS

	static constexpr int32 BuildTimeBucketsCount = 5;
	
	TArray<FVector> MeshVerts;
	
	TArray<int32> AreaIndices[RECAST_MAX_AREAS];
	TArray<int32> ForbiddenIndices;
	TArray<int32> BuiltMeshIndices;
	TArray<int32> TileBuildTimesIndices[BuildTimeBucketsCount];
	
	TArray<FVector> PolyEdges;
	TArray<FVector> NavMeshEdges;
	TArray<FOffMeshLink> OffMeshLinks;
	TArray<FOffMeshLink> ForbiddenLinks;

#if WITH_NAVMESH_CLUSTER_LINKS
	TArray<FCluster> Clusters;
	TArray<FClusterLink> ClusterLinks;
#endif // WITH_NAVMESH_CLUSTER_LINKS

#if WITH_NAVMESH_SEGMENT_LINKS
	TArray<FOffMeshSegment> OffMeshSegments;
	TArray<int32> OffMeshSegmentAreas[RECAST_MAX_AREAS];
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if RECAST_INTERNAL_DEBUG_DATA
	TArray<FIntPoint> TilesToDisplayInternalData;
#endif

	uint32 bGatherPolyEdges : 1;
	uint32 bGatherNavMeshEdges : 1;
	uint32 bMarkForbiddenPolys : 1;
	uint32 bGatherTileBuildTimesHeatMap : 1;

	double MinTileBuildTime = DBL_MAX;
	double MaxTileBuildTime = 0.;
	
	FRecastDebugGeometry() : bGatherPolyEdges(false), bGatherNavMeshEdges(false), bMarkForbiddenPolys(false), bGatherTileBuildTimesHeatMap(false)
	{}

	uint32 NAVIGATIONSYSTEM_API GetAllocatedSize() const;
};

struct FNavTileRef
{
	FNavTileRef() {}
	explicit FNavTileRef(const uint64 InTileRef) : TileRef(InTileRef) {}

	explicit operator uint64() const { return TileRef; }

	bool operator==(const FNavTileRef InRef) const { return TileRef == (uint64)InRef; }
	bool operator!=(const FNavTileRef InRef) const { return TileRef != (uint64)InRef; }

	bool IsValid() const { return TileRef != (uint64)FNavTileRef(); }

	inline friend uint32 GetTypeHash(const FNavTileRef& NavTileRef) { return GetTypeHash(NavTileRef.TileRef); }

	/** Those 2 functions are used for backward compatibility of the following deprecated functions in FRecastNavMeshGenerator and ARecastNavMesh:
	*	  RemoveTileLayers
	*     AddGeneratedTilesTimeSliced
	*     AddGeneratedTiles
	*     RemoveLayers
	*     ProcessTileTasksAsync
	*     ProcessTileTasks
	*     AttachTiles
	*     DetachTiles
	*     OnNavMeshTilesUpdated
	*     InvalidateAffectedPaths
	*   They will be removed with the deprecated methods */
	static void NAVIGATIONSYSTEM_API DeprecatedGetTileIdsFromNavTileRefs(const FPImplRecastNavMesh* RecastNavMeshImpl, const TArray<FNavTileRef>& InTileRefs, TArray<uint32>& OutTileIds);
	static void NAVIGATIONSYSTEM_API DeprecatedMakeTileRefsFromTileIds(const FPImplRecastNavMesh* RecastNavMeshImpl, const TArray<uint32>& InTileIds, TArray<FNavTileRef>& OutTileRefs);
	
private:	
	uint64 TileRef = 0;
};

struct FNavPoly
{
	NavNodeRef Ref;
	FVector Center;
};

namespace ERecastNamedFilter
{
	enum Type 
	{
		FilterOutNavLinks = 0,		// filters out all off-mesh connections
		FilterOutAreas,				// filters out all navigation areas except the default one (RECAST_DEFAULT_AREA)
		FilterOutNavLinksAndAreas,	// combines FilterOutNavLinks and FilterOutAreas

		NamedFiltersCount,
	};
}

struct FNavigationWallEdge
{
	FNavigationWallEdge() = default;
	FNavigationWallEdge(const FVector& InStart, const FVector& InEnd)
		: Start(InStart)
		, End(InEnd)
	{
	}
	FVector Start = FVector::ZeroVector;
	FVector End = FVector::ZeroVector;
};
#endif //WITH_RECAST

UENUM()
enum class EHeightFieldRenderMode : uint8
{
	Solid = 0,
	Walkable
};

USTRUCT()
struct FRecastNavMeshTileGenerationDebug
{
	GENERATED_BODY()
	
	NAVIGATIONSYSTEM_API FRecastNavMeshTileGenerationDebug();

	/** If set, the selected internal debug data will be kept during tile generation to be displayed with the navmesh. */
	UPROPERTY(Transient, EditAnywhere, Category = Debug)
	uint32 bEnabled : 1;

	/** Selected tile coordinate, only this tile will have it's internal data kept. 
	 *  When MaxTileCoordinate is enabled, this defines the lower bound of a rectangle.
	 *  Tip: displaying the navmesh using 'Draw Tile Labels' show tile coordinates. */
	UPROPERTY(EditAnywhere, Category = Debug)
	FIntVector TileCoordinate = FIntVector::ZeroValue;

	/** Optional: Highest bound included tile to define a rectangle with TileCoordinate for which the internal data is kept.
	 *  Tip: displaying the navmesh using 'Draw Tile Labels' show tile coordinates. */
	UPROPERTY(EditAnywhere, Category = Debug, meta=(EditCondition="bUseMaxTileCoordinate"))
	FIntVector MaxTileCoordinate = FIntVector::ZeroValue;

	/** If set, the generator will only generate the tile selected to debug (set in TileCoordinate).*/
	UPROPERTY(EditAnywhere, Category = Debug)
	uint32 bGenerateDebugTileOnly : 1;

	/** Display the collision used for the navmesh rasterization.
	 * Note: The visualization is affected by the DrawOffset of the RecastNavmesh owner*/
	UPROPERTY(EditAnywhere, Category = Debug)
	uint32 bCollisionGeometry : 1;

	/** Display the nav area modifier shapes.
	 * Note: The visualization is affected by the DrawOffset of the RecastNavmesh owner*/
	UPROPERTY(EditAnywhere, Category = Debug)
	uint32 bDynamicAreas : 1;

	UPROPERTY(EditAnywhere, Category = Debug)
	EHeightFieldRenderMode HeightFieldRenderMode;

	UPROPERTY(EditAnywhere, Category = Debug)
	uint32 bHeightfieldFromRasterization : 1;

	UPROPERTY(EditAnywhere, Category = Debug)
	uint32 bHeightfieldPostInclusionBoundsFiltering : 1;

	UPROPERTY(EditAnywhere, Category = Debug)
	uint32 bHeightfieldPostHeightFiltering : 1;

	UPROPERTY(EditAnywhere, Category = Debug)
	uint32 bHeightfieldBounds : 1;

	UPROPERTY(EditAnywhere, Category = Debug)
	uint32 bCompactHeightfield : 1;

	UPROPERTY(EditAnywhere, Category = Debug)
	uint32 bCompactHeightfieldEroded : 1;

	UPROPERTY(EditAnywhere, Category = Debug)
	uint32 bHeightFieldLayers : 1;
	
	UPROPERTY(EditAnywhere, Category = Debug)
	uint32 bCompactHeightfieldRegions : 1;

	UPROPERTY(EditAnywhere, Category = Debug)
	uint32 bCompactHeightfieldDistances : 1;

	UPROPERTY(EditAnywhere, Category = Debug)
	uint32 bTileCacheLayerAreas : 1;

	UPROPERTY(EditAnywhere, Category = Debug)
	uint32 bTileCacheLayerRegions : 1;

	/** If set, the contour simplification step will be skipped. Beware that enabling this changes the way navmesh will generate when Tile Generation Debug is enabled. */
	UPROPERTY(EditAnywhere, Category = Debug)
	uint32 bSkipContourSimplification : 1;
	
	UPROPERTY(EditAnywhere, Category = Debug)
	uint32 bTileCacheContours : 1;

	UPROPERTY(EditAnywhere, Category = Debug)
	uint32 bTileCachePolyMesh : 1;

	UPROPERTY(EditAnywhere, Category = Debug)
	uint32 bTileCacheDetailMesh : 1;

	UPROPERTY(EditAnywhere, Category = Debug, meta=(InlineEditConditionToggle))
	uint32 bUseMaxTileCoordinate : 1;

	UPROPERTY(EditAnywhere, Category = Debug, meta = (Bitmask, BitmaskEnum = "/Script/NavigationSystem.ELinkGenerationDebugFlags"))
	uint16 LinkGenerationDebugFlags;

	/** Using -1 as 'no edge selected'. */
	UPROPERTY(EditAnywhere, Category = Debug, meta=(UIMin=-1, ClampMin=-1))
	int32 LinkGenerationSelectedEdge;

	/** Using -1 as 'no config selected'. */
	UPROPERTY(EditAnywhere, Category = Debug, meta=(UIMin=-1, ClampMin=-1))
	int32 LinkGenerationSelectedConfig;
	
	/** Indicates if the tile coordinate in parameter is within the tile coordinate selected to debug.
	 *  Either equal to TileCoordinate or inside the rectangle formed by TileCoordinate and MaxTileCoordinate. */
	NAVIGATIONSYSTEM_API bool IsWithinTileCoordinates(const int32 TileX, const int32 TileY) const;
};

/**
 *	Used to list tiles that needs rebuilding.
 */
struct FNavMeshDirtyTileElement
{
	FIntPoint Coordinates;
	FVector::FReal InvokerDistanceSquared;
	ENavigationInvokerPriority InvokerPriority;
};

/**
 *	Structure to handle nav mesh tile's raw data persistence and releasing
 */
struct FNavMeshTileData
{
	// helper function so that we release NavData via dtFree not regular delete (for navigation mem stats)
	struct FNavData
	{
		// Temporary test to help reproduce a crash.
		void TestPtr() const;
		
		FNavData(uint8* InNavData, const int32 InDataSize) : RawNavData(InNavData)
		{
			if (RawNavData != nullptr)
			{
				AllocatedSize = FMemory::GetAllocSize((void*)RawNavData);
				check(AllocatedSize == 0 || AllocatedSize >= InDataSize);
			}
			else
			{
				AllocatedSize = 0;
			}
		}
		~FNavData();

		const uint8* GetRawNavData() const { return RawNavData; }
		uint8* GetMutableRawNavData() { return RawNavData; }

		void Reset()
		{
			RawNavData = nullptr;
			AllocatedSize = 0;
		}
				
	protected:
		uint8* RawNavData;
		SIZE_T AllocatedSize; // != DataSize
	};
	
	// layer index
	int32	LayerIndex;
	FBox	LayerBBox;
	// size of allocated data
	int32	DataSize;
	// actual tile data
	TSharedPtr<FNavData, ESPMode::ThreadSafe> NavData;
	
	FNavMeshTileData() : LayerIndex(0), DataSize(0) { }
	~FNavMeshTileData();
	
	explicit FNavMeshTileData(uint8* RawData, int32 RawDataSize, int32 LayerIdx = 0, FBox LayerBounds = FBox(ForceInit));
		
	inline uint8* GetData()
	{
		check(NavData.IsValid());
		return NavData->GetMutableRawNavData();
	}

	inline const uint8* GetData() const
	{
		check(NavData.IsValid());
		return NavData->GetRawNavData();
	}

	inline uint8* GetDataSafe()
	{
		return NavData.IsValid() ? NavData->GetMutableRawNavData() : NULL;
	}

	inline bool operator==(const uint8* RawData) const
	{
		return GetData() == RawData;
	}

	inline bool IsValid() const { return NavData.IsValid() && GetData() != nullptr && DataSize > 0; }

	uint8* Release();

	// Duplicate shared state so we will have own copy of the data
	void MakeUnique();
};

DECLARE_MULTICAST_DELEGATE(FOnNavMeshUpdate);

namespace FNavMeshConfig
{
	struct FRecastNamedFiltersCreator
	{
		FRecastNamedFiltersCreator(bool bVirtualFilters);
	};
}

USTRUCT()
struct FNavMeshResolutionParam
{
	GENERATED_BODY()

	bool IsValid() const { return CellSize > 0.f && CellHeight > 0.f && AgentMaxStepHeight > 0.f; }
	
	/** Horizontal size of voxelization cell */
	UPROPERTY(EditAnywhere, Category = Generation, config, meta = (ClampMin = "1.0", ClampMax = "1024.0"))
	float CellSize = 25.f;

	/** Vertical size of voxelization cell */
	UPROPERTY(EditAnywhere, Category = Generation, config, meta = (ClampMin = "1.0", ClampMax = "1024.0"))
	float CellHeight = 10.f;

	/** Largest vertical step the agent can perform */
	UPROPERTY(EditAnywhere, Category = Generation, config, meta = (ClampMin = "0.0"))
	float AgentMaxStepHeight = 35.f;
};

UCLASS(config=Engine, defaultconfig, hidecategories=(Input,Rendering,Tags,Transformation,Actor,Layers,Replication), notplaceable, MinimalAPI)
class ARecastNavMesh : public ANavigationData
{
	GENERATED_UCLASS_BODY()

	typedef uint16 FNavPolyFlags;

	NAVIGATIONSYSTEM_API virtual void Serialize( FArchive& Ar ) override;

#if WITH_EDITOR
	NAVIGATIONSYSTEM_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif

	/** Draw edges of every navmesh's triangle */
	UPROPERTY(EditAnywhere, Category=Display)
	uint32 bDrawTriangleEdges:1;

	/** Draw edges of every poly (i.e. not only border-edges)  */
	UPROPERTY(EditAnywhere, Category=Display, config)
	uint32 bDrawPolyEdges:1;

	/** if disabled skips filling drawn navmesh polygons */
	UPROPERTY(EditAnywhere, Category = Display)
	uint32 bDrawFilledPolys:1;

	/** Draw border-edges */
	UPROPERTY(EditAnywhere, Category=Display)
	uint32 bDrawNavMeshEdges:1;

	/** Draw the tile boundaries */
	UPROPERTY(EditAnywhere, Category=Display)
	uint32 bDrawTileBounds:1;

	/** Draw the tile resolutions */
	UPROPERTY(EditAnywhere, Category=Display)
	uint32 bDrawTileResolutions:1;
	
	/** Draw input geometry passed to the navmesh generator.  Recommend disabling other geometry rendering via viewport showflags in editor. */
	UPROPERTY(EditAnywhere, Category=Display)
	uint32 bDrawPathCollidingGeometry:1;

	UPROPERTY(EditAnywhere, Category=Display)
	uint32 bDrawTileLabels:1;

	UPROPERTY(EditAnywhere, Category=Display)
	uint32 bDrawTileBuildTimes:1;

	UPROPERTY(EditAnywhere, Category=Display)
	uint32 bDrawTileBuildTimesHeatMap:1;
	
	/** Draw a label for every poly that indicates its poly and tile indices */
	UPROPERTY(EditAnywhere, Category=Display, meta = (DisplayName = "Draw Polygon Indices"))
	uint32 bDrawPolygonLabels:1;

	/** Draw a label for every poly that indicates its default and fixed costs */
	UPROPERTY(EditAnywhere, Category=Display, meta=(DisplayName="Draw Polygon Costs"))
	uint32 bDrawDefaultPolygonCost:1;

	/** Draw a label for every poly that indicates its poly and area flags */
	UPROPERTY(EditAnywhere, Category=Display)
	uint32 bDrawPolygonFlags:1;

	/** Draw a label for every poly that indicates its area id and the list of all NavAreaClass used in the displayed tiles. */
	UPROPERTY(EditAnywhere, Category=Display)
	uint32 bDrawPolygonAreaIDs:1;

	UPROPERTY(EditAnywhere, Category=Display)
	uint32 bDrawLabelsOnPathNodes:1;

	/** Draw valid links (both ends are valid). */
	UPROPERTY(EditAnywhere, Category=Display)
	uint32 bDrawNavLinks:1;

	/** Draw failed links and valid links. */
	UPROPERTY(EditAnywhere, Category=Display, Meta = (DisplayName = "Draw Failed and Valid NavLinks"))
	uint32 bDrawFailedNavLinks:1;
	
	/** Draw navmesh's clusters and cluster links. (Requires WITH_NAVMESH_CLUSTER_LINKS=1) */
	UPROPERTY(EditAnywhere, Category=Display)
	uint32 bDrawClusters:1;

	/** Draw octree used to store navigation relevant actors */
	UPROPERTY(EditAnywhere, Category = Display)
	uint32 bDrawOctree : 1;

	/** Draw octree used to store navigation relevant actors with the elements bounds */
	UPROPERTY(EditAnywhere, Category = Display, meta=(editcondition = "bDrawOctree"))
	uint32 bDrawOctreeDetails : 1;

	UPROPERTY(EditAnywhere, Category = Display)
	uint32 bDrawMarkedForbiddenPolys : 1;

	/** Draw detailed navmesh information as 2D text. */
	UPROPERTY(EditAnywhere, Category = Display)
	uint32 bDrawOnScreenInformation : 1;

	/** if true, show currently rebuilding tiles differently when visualizing */
	UPROPERTY(config)
	uint32 bDistinctlyDrawTilesBeingBuilt:1;

	/** vertical offset added to navmesh's debug representation for better readability */
	UPROPERTY(EditAnywhere, Category=Display, config)
	float DrawOffset;

	UPROPERTY(EditAnywhere, Category = Display)
	FRecastNavMeshTileGenerationDebug TileGenerationDebug;
	
	//----------------------------------------------------------------------//
	// NavMesh generation parameters
	//----------------------------------------------------------------------//

	/** if true, the NavMesh will allocate fixed size pool for tiles, should be enabled to support streaming */
	UPROPERTY(EditAnywhere, Category=Generation, config)
	uint32 bFixedTilePoolSize:1;

	/** maximum number of tiles NavMesh can hold */
	UPROPERTY(EditAnywhere, Category=Generation, config, meta=(editcondition = "bFixedTilePoolSize"))
	int32 TilePoolSize;

	/** size of single tile, expressed in uu */
	UPROPERTY(EditAnywhere, Category=Generation, config, meta=(ClampMin = "300.0"))
	float TileSizeUU;

	/** Expected average number of layers per tile. 
	 *  This is a factor in the number of tiles the navmesh can hold when not using bFixedTilePoolSize.
	 *  Log warnings are issued if this is too low. */
	UPROPERTY(EditAnywhere, Category=Generation, config, meta=(ClampMin = "1.0"))
	float AverageLayersPerTile;

	/** If true (default), minimize base link pool size using optimized allocation, else use previous larger base link pool.
	 *  Changing this property requires a navmesh rebuild to fully benefit from the change. */
	UPROPERTY(EditAnywhere, Category=Generation, config, AdvancedDisplay)
	uint32 bMinimizeLinkPoolSize:1;

	/**
	 * Note that we are not using _DEPRECATED on the following deprecated properties
	 * since it prevents the property from being serialized back which can break the
	 * process of duplicating the navmesh for PIE
	 */

	UE_DEPRECATED(all, "Use NavMeshResolutionParams to set CellSize for the different resolutions instead")
	UPROPERTY(config, meta = (DeprecatedProperty, DeprecationMessage = "Use NavMeshResolutionParams to set CellSize for the different resolutions instead"))
	float CellSize;

	UE_DEPRECATED(all, "Use NavMeshResolutionParams to set CellHeight for the different resolutions instead")
	UPROPERTY(config, meta = (DeprecatedProperty, DeprecationMessage = "Use NavMeshResolutionParams to set CellHeight for the different resolutions instead"))
	float CellHeight;

	UE_DEPRECATED(all, "Use NavMeshResolutionParams to set AgentMaxStepHeight for the different resolutions instead")
	UPROPERTY(config, meta = (DeprecatedProperty, DeprecationMessage = "Use NavMeshResolutionParams to set AgentMaxStepHeight for the different resolutions instead"))
	float AgentMaxStepHeight;

	/** Resolution params 
	 * If using multiple resolutions, it's recommended to choose the highest resolution first and 
	 * set it according to the highest desired precision and then the other resolutions. */
	UPROPERTY(EditAnywhere, Category = Generation, config)
	FNavMeshResolutionParam NavMeshResolutionParams[(uint8)ENavigationDataResolution::MAX];

	/** Radius of smallest agent to traverse this navmesh */
	UPROPERTY(EditAnywhere, Category = Generation, config, meta = (ClampMin = "0.0", ClampMax = "100000.0", UIMin = "0.0", UIMax = "100000.0"))
	float AgentRadius;

	/** Size of the tallest agent that will path with this navmesh. */
	UPROPERTY(EditAnywhere, Category = Generation, config, meta = (ClampMin = "0.0", ClampMax = "100000.0", UIMin = "0.0", UIMax = "100000.0"))
	float AgentHeight;

	/* The maximum slope (angle) that the agent can move on. */ 
	UPROPERTY(EditAnywhere, Category=Generation, config, meta=(ClampMin = "0.0", ClampMax = "89.0", UIMin = "0.0", UIMax = "89.0" ))
	float AgentMaxSlope;

	/* The minimum dimension of area. Areas smaller than this will be discarded */
	UPROPERTY(EditAnywhere, Category=Generation, config, meta=(ClampMin = "0.0"))
	float MinRegionArea;

	/* The size limit of regions to be merged with bigger regions (watershed partitioning only) */
	UPROPERTY(EditAnywhere, Category=Generation, config, meta=(ClampMin = "0.0"))
	float MergeRegionSize;

	/** Maximum vertical deviation between raw contour points to allowing merging (in voxel).
	 * Use a low value (2-5) depending on CellHeight, AgentMaxStepHeight and AgentMaxSlope, to allow more precise contours (also see SimplificationElevationRatio).
	 * Use very high value to deactivate (Recast behavior). */
	UE_DEPRECATED(5.5, "Not used anymore, the behavior is now binded to SimplificationElevationRatio.")
	UPROPERTY(config)
	int MaxVerticalMergeError;
	
	/** How much navigable shapes can get simplified - the higher the value the more freedom */
	UPROPERTY(EditAnywhere, Category = Generation, config, meta = (ClampMin = "0.0"))
	float MaxSimplificationError;

	/** When simplifying contours, how much is the vertical error taken into account when comparing with MaxSimplificationError.
	 * Use 0 to deactivate (Recast behavior), use 1 as a typical value. */
	UPROPERTY(EditAnywhere, Category = Generation, config, meta = (ClampMin = "0.0"))
	float SimplificationElevationRatio;

	/** Sets the limit for number of asynchronous tile generators running at one time, also used for some synchronous tasks */
	UPROPERTY(EditAnywhere, Category = Generation, config, meta = (ClampMin = "0", UIMin = "0"), AdvancedDisplay)
	int32 MaxSimultaneousTileGenerationJobsCount;

	/** Absolute hard limit to number of navmesh tiles. Be very, very careful while modifying it while
	 *	having big maps with navmesh. A single, empty tile takes 176 bytes and empty tiles are
	 *	allocated up front (subject to change, but that's where it's at now)
	 *	@note TileNumberHardLimit is always rounded up to the closest power of 2 */
	UPROPERTY(EditAnywhere, Category = Generation, config, meta = (ClampMin = "1", UIMin = "1"), AdvancedDisplay)
	int32 TileNumberHardLimit;

	/** Used when connecting segment links across layers to determine how much memory to allocate to hold skipped layers */
	UPROPERTY(EditAnywhere, Category = Generation, config, meta = (ClampMin = "1", UIMin = "1"), AdvancedDisplay)
	int32 ExpectedMaxLayersPerTile;

	UPROPERTY(VisibleAnywhere, Category = Generation, AdvancedDisplay)
	int32 PolyRefTileBits;

	UPROPERTY(VisibleAnywhere, Category = Generation, AdvancedDisplay)
	int32 PolyRefNavPolyBits;

	UPROPERTY(VisibleAnywhere, Category = Generation, AdvancedDisplay)
	int32 PolyRefSaltBits;

	/** Use this if you don't want your tiles to start at (0,0,0) */
	UPROPERTY(EditAnywhere, Category = Generation, AdvancedDisplay)
	FVector NavMeshOriginOffset;

	/** navmesh draw distance in game (always visible in editor) */
	UPROPERTY(config)
	float DefaultDrawDistance;

	/** specifies default limit to A* nodes used when performing navigation queries. 
	 *	Can be overridden by passing custom FNavigationQueryFilter */
	UPROPERTY(config)
	float DefaultMaxSearchNodes;

	/** specifies default limit to A* nodes used when performing hierarchical navigation queries. */
	UPROPERTY(config)
	float DefaultMaxHierarchicalSearchNodes;

	/** filtering methode used for filtering ledge slopes */
	UPROPERTY(EditAnywhere, Category=Generation, config, AdvancedDisplay)
	ENavigationLedgeSlopeFilterMode LedgeSlopeFilterMode;
	
	/** partitioning method for creating navmesh polys */
	UPROPERTY(EditAnywhere, Category=Generation, config, AdvancedDisplay)
	TEnumAsByte<ERecastPartitioning::Type> RegionPartitioning;

	/** partitioning method for creating tile layers */
	UPROPERTY(EditAnywhere, Category=Generation, config, AdvancedDisplay)
	TEnumAsByte<ERecastPartitioning::Type> LayerPartitioning;

	/** number of chunk splits (along single axis) used for region's partitioning: ChunkyMonotone */
	UPROPERTY(EditAnywhere, Category=Generation, config, AdvancedDisplay)
	int32 RegionChunkSplits;

	/** number of chunk splits (along single axis) used for layer's partitioning: ChunkyMonotone */
	UPROPERTY(EditAnywhere, Category=Generation, config, AdvancedDisplay)
	int32 LayerChunkSplits;

	/** Controls whether Navigation Areas will be sorted by cost before application 
	 *	to navmesh during navmesh generation. This is relevant when there are
	 *	areas overlapping and we want to have area cost express area relevancy
	 *	as well. Setting it to true will result in having area sorted by cost,
	 *	but it will also increase navmesh generation cost a bit */
	UPROPERTY(EditAnywhere, Category=Generation, config)
	uint32 bSortNavigationAreasByCost:1;

	/* In a world partitioned map, is this navmesh using world partitioning */
	UPROPERTY(EditAnywhere, Category=Generation, config, meta = (EditCondition = "bAllowWorldPartitionedNavMesh", HideEditConditionToggle, DisplayName = "IsWorldPartitionedNavMesh"))
	uint32 bIsWorldPartitioned : 1;

	/** Experimental: if set, navlinks will be automatically generated.
	 * @see FNavLinkGenerationJumpConfig */ 
	UPROPERTY(EditAnywhere, Category=Generation, config)
	uint32 bGenerateNavLinks : 1;
	
	/** controls whether voxel filtering will be applied (via FRecastTileGenerator::ApplyVoxelFilter). 
	 *	Results in generated navmesh better fitting navigation bounds, but hits (a bit) generation performance */
	UPROPERTY(EditAnywhere, Category=Generation, config, AdvancedDisplay)
	uint32 bPerformVoxelFiltering:1;

	/** mark areas with insufficient free height above instead of cutting them out (accessible only for area modifiers using replace mode) */
	UPROPERTY(EditAnywhere, Category = Generation, config, AdvancedDisplay)
	uint32 bMarkLowHeightAreas : 1;

	/** Expand the top of the area nav modifier's bounds by one cell height when applying to the navmesh.
		If unset, navmesh on top of surfaces might not be marked by marking bounds flush with top surfaces (since navmesh is generated slightly above collision, depending on cell height). */
	UPROPERTY(EditAnywhere, Category = Generation, config, AdvancedDisplay)
	uint32 bUseExtraTopCellWhenMarkingAreas : 1;

	/** if set, only single low height span will be allowed under valid one */
	UPROPERTY(EditAnywhere, Category = Generation, config, AdvancedDisplay)
	uint32 bFilterLowSpanSequences : 1;

	/** if set, only low height spans with corresponding area modifier will be stored in tile cache (reduces memory, can't modify without full tile rebuild) */
	UPROPERTY(EditAnywhere, Category = Generation, config, AdvancedDisplay)
	uint32 bFilterLowSpanFromTileCache : 1;

	/** if set, navmesh data gathering will never happen on the game thread and will only be done on background threads */
	UPROPERTY(EditAnywhere, Category = Generation, config, AdvancedDisplay)
	uint32 bDoFullyAsyncNavDataGathering : 1;
	
	/** TODO: switch to disable new code from OffsetFromCorners if necessary - remove it later */
	UPROPERTY(config)
	uint32 bUseBetterOffsetsFromCorners : 1;

	/** If set, tiles generated without any navmesh data will be marked to distinguish them from not generated / streamed out ones. Defaults to false. */
	UPROPERTY(config)
	uint32 bStoreEmptyTileLayers : 1;

	/** Indicates whether default navigation filters will use virtual functions. Defaults to true. */
	UPROPERTY(config)
	uint32 bUseVirtualFilters : 1;

	/** Indicates whether to use the virtual methods to check if an object should generate geometry or
	 *  if we should call the normal method directly (i.e. FNavigationOctreeElement::ShouldUseGeometry).
	 *  If enabled, will also check if an object requesting an update on the navmesh is excluded to avoid dirtying the areas unnecessarily.
	 *  Defaults to false. */
	UPROPERTY(config)
	uint32 bUseVirtualGeometryFilteringAndDirtying : 1;

	/** If set, paths can end at navlink poly (not the ground one!) */
	UPROPERTY(config)
	uint32 bAllowNavLinkAsPathEnd : 1;

	/** The maximum number of y coords to process when time slicing filter ledge spans during navmesh regeneration. */
	UPROPERTY(EditAnywhere, Category = TimeSlicing, config, AdvancedDisplay, meta = (ClampMin = "1", UIMin = "1"))
	int32 TimeSliceFilterLedgeSpansMaxYProcess = 13;

	/** If a single time sliced section of navmesh regen code exceeds this duration then it will trigger debug logging */
	UPROPERTY(EditAnywhere, Category = TimeSlicing, config, AdvancedDisplay)
	double TimeSliceLongDurationDebug = 0.002;

	/** If >= 1, when sorting pending tiles by priority, tiles near invokers (within the distance threshold) will have their priority increased. */
	UPROPERTY(EditAnywhere, Category = Generation, config, AdvancedDisplay)
	uint32 InvokerTilePriorityBumpDistanceThresholdInTileUnits = 1;

	/** Priority increase steps for tiles that are withing near distance. */
	UPROPERTY(EditAnywhere, Category = Generation, config, AdvancedDisplay)
	uint8 InvokerTilePriorityBumpIncrease = 1;

protected:
#if WITH_EDITORONLY_DATA
	/** World partitioned navmesh are only allowed in partitioned worlds. */
	UPROPERTY() 
	uint32 bAllowWorldPartitionedNavMesh : 1;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Experimental configuration to generate vertical links. */
	UE_DEPRECATED(5.7, "Use the NavLinkJumpConfigs array instead.")
	UPROPERTY(config)
	FNavLinkGenerationJumpDownConfig NavLinkJumpDownConfig;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

	/** Experimental configurations to generate jump links. */
	UPROPERTY(EditAnywhere, Category=Generation, config)
	TArray<FNavLinkGenerationJumpConfig> NavLinkJumpConfigs;

private:
	/** @returns true if there were no tiles when the navmesh was loaded. */
	bool bHasNoTileData : 1 = false;
	
	/** Cache rasterized voxels instead of just collision vertices/indices in navigation octree */
	UPROPERTY(config)
	uint32 bUseVoxelCache : 1;

	/** indicates how often we will sort navigation tiles to mach players position */
	UPROPERTY(config)
	float TileSetUpdateInterval;
	
	/** contains last available dtPoly's flag bit set (8th bit at the moment of writing) */
	static NAVIGATIONSYSTEM_API FNavPolyFlags NavLinkFlag;

	/** Squared draw distance */
	static NAVIGATIONSYSTEM_API FVector::FReal DrawDistanceSq;

	/** MinimumSizeForChaosNavMeshInfluence*/
	static NAVIGATIONSYSTEM_API float MinimumSizeForChaosNavMeshInfluenceSq;

public:

	struct FRaycastResult
	{
		enum 
		{
			MAX_PATH_CORRIDOR_POLYS = 128
		};

		NavNodeRef CorridorPolys[MAX_PATH_CORRIDOR_POLYS];
		float CorridorCost[MAX_PATH_CORRIDOR_POLYS];
		int32 CorridorPolysCount;
		FVector::FReal HitTime;
		FVector HitNormal;
		uint32 bIsRaycastEndInCorridor : 1;

		FRaycastResult()
			: CorridorPolysCount(0)
			, HitTime(TNumericLimits<FVector::FReal>::Max())
			, HitNormal(0.f)
			, bIsRaycastEndInCorridor(false)
		{
			FMemory::Memzero(CorridorPolys);
			FMemory::Memzero(CorridorCost);
		}

		inline int32 GetMaxCorridorSize() const { return MAX_PATH_CORRIDOR_POLYS; }
		inline bool HasHit() const { return HitTime != TNumericLimits<FVector::FReal>::Max(); }
		inline NavNodeRef GetLastNodeRef() const { return CorridorPolysCount > 0 ? CorridorPolys[CorridorPolysCount - 1] : INVALID_NAVNODEREF; }
	};

	//----------------------------------------------------------------------//
	// Recast runtime params
	//----------------------------------------------------------------------//
	/** Euclidean distance heuristic scale used while pathfinding */
	UPROPERTY(EditAnywhere, Category = Query, config, meta = (ClampMin = "0.1"))
	float HeuristicScale;

	/** Value added to each search height to compensate for error between navmesh polys and walkable geometry  */
	UPROPERTY(EditAnywhere, Category = Query, config, meta = (ClampMin = "0.0"))
	float VerticalDeviationFromGroundCompensation;

	/** broadcast for navmesh updates */
	FOnNavMeshUpdate OnNavMeshUpdate;

	inline static void SetDrawDistance(FVector::FReal NewDistance) { DrawDistanceSq = NewDistance * NewDistance; }
	inline static FVector::FReal GetDrawDistanceSq() { return DrawDistanceSq; }

	inline static void SetMinimumSizeForChaosNavMeshInfluence(float NewSize) { MinimumSizeForChaosNavMeshInfluenceSq = NewSize * NewSize; }
	inline static float GetMinimumSizeForChaosNavMeshInfluenceSq() { return MinimumSizeForChaosNavMeshInfluenceSq; }
	

	//////////////////////////////////////////////////////////////////////////

	NAVIGATIONSYSTEM_API bool HasValidNavmesh() const;

	/** Dtor */
	NAVIGATIONSYSTEM_API virtual ~ARecastNavMesh();

#if WITH_RECAST
	//----------------------------------------------------------------------//
	// Life cycle & serialization
	//----------------------------------------------------------------------//
public:

	//~ Begin UObject Interface
	NAVIGATIONSYSTEM_API virtual void PostInitProperties() override;
	NAVIGATIONSYSTEM_API virtual void PostLoad() override;
	NAVIGATIONSYSTEM_API virtual void PostRegisterAllComponents() override;
	NAVIGATIONSYSTEM_API virtual void PostUnregisterAllComponents() override;
	NAVIGATIONSYSTEM_API virtual void BeginDestroy() override;

#if WITH_EDITOR

private:
	bool CanUpdateNavMeshInEditor() const;

public:
	NAVIGATIONSYSTEM_API virtual void PostEditChangeChainProperty( struct FPropertyChangedChainEvent& PropertyChangedChainEvent) override;
	NAVIGATIONSYSTEM_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent) override;
	NAVIGATIONSYSTEM_API virtual void PostEditUndo() override;
	NAVIGATIONSYSTEM_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

#if WITH_EDITOR
	/** RecastNavMesh instances are dynamically spawned and should not be copied */
	virtual bool ShouldExport() override { return false; }
#endif

	NAVIGATIONSYSTEM_API virtual void LoadBeforeGeneratorRebuild() override;
	
	NAVIGATIONSYSTEM_API virtual void CleanUp() override;

	//~ Begin ANavigationData Interface
	NAVIGATIONSYSTEM_API virtual FNavLocation GetRandomPoint(FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override;
	/** finds a random location in Radius, reachable from Origin.
	 *  @param Radius needs to be non-negative. The function fails for Radius < 0. Radius being 0 is still rasults in a valid request. */
	NAVIGATIONSYSTEM_API virtual bool GetRandomReachablePointInRadius(const FVector& Origin, float Radius, FNavLocation& OutResult, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override;
	NAVIGATIONSYSTEM_API virtual bool GetRandomPointInNavigableRadius(const FVector& Origin, float Radius, FNavLocation& OutResult, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override;

	NAVIGATIONSYSTEM_API virtual bool FindMoveAlongSurface(const FNavLocation& StartLocation, const FVector& TargetPosition, FNavLocation& OutLocation, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override;
	NAVIGATIONSYSTEM_API virtual void BatchFindMoveAlongSurface(TArray<FNavigationFindMoveAlongSurfaceWork>& Workload, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override;
	NAVIGATIONSYSTEM_API virtual bool FindOverlappingEdges(const FNavLocation& StartLocation, TConstArrayView<FVector> ConvexPolygon, TArray<FVector>& OutEdges, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override;
	NAVIGATIONSYSTEM_API virtual bool GetPathSegmentBoundaryEdges(const FNavigationPath& Path, const FNavPathPoint& StartPoint, const FNavPathPoint& EndPoint, const TConstArrayView<FVector> SearchArea, TArray<FVector>& OutEdges, const float MaxAreaEnterCost, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override;
	NAVIGATIONSYSTEM_API virtual bool ProjectPoint(const FVector& Point, FNavLocation& OutLocation, const FVector& Extent, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override;
	NAVIGATIONSYSTEM_API virtual bool IsNodeRefValid(NavNodeRef NodeRef) const override;

	/** Project batch of points using shared search extent and filter */
	NAVIGATIONSYSTEM_API virtual void BatchProjectPoints(TArray<FNavigationProjectionWork>& Workload, const FVector& Extent, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override;

	/** Project batch of points using shared search filter. This version is not requiring user to pass in Extent, 
	 *	and is instead relying on FNavigationProjectionWork.ProjectionLimit.
	 *	@note function will assert if item's FNavigationProjectionWork.ProjectionLimit is invalid */
	NAVIGATIONSYSTEM_API virtual void BatchProjectPoints(TArray<FNavigationProjectionWork>& Workload, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override;
	
	NAVIGATIONSYSTEM_API virtual ENavigationQueryResult::Type CalcPathCost(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& OutPathCost, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override;
	NAVIGATIONSYSTEM_API virtual ENavigationQueryResult::Type CalcPathLength(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& OutPathLength, FSharedConstNavQueryFilter QueryFilter = NULL, const UObject* Querier = NULL) const override;
	NAVIGATIONSYSTEM_API virtual ENavigationQueryResult::Type CalcPathLengthAndCost(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& OutPathLength, FVector::FReal& OutPathCost, FSharedConstNavQueryFilter QueryFilter = NULL, const UObject* Querier = NULL) const override;
	NAVIGATIONSYSTEM_API virtual bool DoesNodeContainLocation(NavNodeRef NodeRef, const FVector& WorldSpaceLocation) const override;

	NAVIGATIONSYSTEM_API virtual UPrimitiveComponent* ConstructRenderingComponent() override;
	/** Returns bounding box for the navmesh. */
	virtual FBox GetBounds() const override { return GetNavMeshBounds(); }
	/** Called on world origin changes **/
	NAVIGATIONSYSTEM_API virtual void ApplyWorldOffset(const FVector& InOffset, bool bWorldShift) override;

	NAVIGATIONSYSTEM_API virtual void FillNavigationDataChunkActor(const FBox& InQueryBounds, class ANavigationDataChunkActor& DataChunkActor, FBox& OutTilesBounds) const override;

	NAVIGATIONSYSTEM_API virtual void OnStreamingNavDataAdded(class ANavigationDataChunkActor& InActor) override;
	NAVIGATIONSYSTEM_API virtual void OnStreamingNavDataRemoved(class ANavigationDataChunkActor& InActor) override;
	
	NAVIGATIONSYSTEM_API virtual void OnStreamingLevelAdded(ULevel* InLevel, UWorld* InWorld) override;
	NAVIGATIONSYSTEM_API virtual void OnStreamingLevelRemoved(ULevel* InLevel, UWorld* InWorld) override;

#if WITH_EDITOR
	NAVIGATIONSYSTEM_API virtual double GetWorldPartitionNavigationDataBuilderOverlap() const override;
#endif
	//~ End ANavigationData Interface

	NAVIGATIONSYSTEM_API virtual void AttachNavMeshDataChunk(URecastNavMeshDataChunk& NavDataChunk);
	NAVIGATIONSYSTEM_API virtual void DetachNavMeshDataChunk(URecastNavMeshDataChunk& NavDataChunk);

	NAVIGATIONSYSTEM_API const TSet<FIntPoint>& GetActiveTileSet() const;
	NAVIGATIONSYSTEM_API TSet<FIntPoint>& GetActiveTileSet(); 
	
	NAVIGATIONSYSTEM_API void LogRecastTile(const TCHAR* Caller, const FName& Prefix, const FName& OperationName, const dtNavMesh& DetourMesh, const int32 TileX, const int32 TileY, const int32 LayerIndex, const uint64 TileRef) const;
	
protected:
	/** Serialization helper. */
	NAVIGATIONSYSTEM_API void SerializeRecastNavMesh(FArchive& Ar, FPImplRecastNavMesh*& NavMesh, int32 NavMeshVersion);

	NAVIGATIONSYSTEM_API virtual void RestrictBuildingToActiveTiles(bool InRestrictBuildingToActiveTiles) override;

	NAVIGATIONSYSTEM_API virtual void OnRegistered() override;
	NAVIGATIONSYSTEM_API virtual void OnUnregistered() override;

public:
	/** Get the CellSize for the given resolution. */
	float GetCellSize(const ENavigationDataResolution Resolution) const { return NavMeshResolutionParams[(uint8)Resolution].CellSize; }

	/** Set the CellSize for the given resolution. */
	void SetCellSize(const ENavigationDataResolution Resolution, const float Size) { NavMeshResolutionParams[(uint8)Resolution].CellSize = Size; }

	/** Get the CellHeight for the given resolution. */
	float GetCellHeight(const ENavigationDataResolution Resolution) const { return NavMeshResolutionParams[(uint8)Resolution].CellHeight; }

	/** Set the CellHeight for the given resolution. */
	void SetCellHeight(const ENavigationDataResolution Resolution, const float Height) { NavMeshResolutionParams[(uint8)Resolution].CellHeight = Height; }

	/** Get the AgentMaxStepHeight for the given resolution. */
	float GetAgentMaxStepHeight(const ENavigationDataResolution Resolution) const { return NavMeshResolutionParams[(uint8)Resolution].AgentMaxStepHeight; }

	/** Set the AgentMaxStepHeight for the given resolution. */
	void SetAgentMaxStepHeight(const ENavigationDataResolution Resolution, const float MaxStepHeight) { NavMeshResolutionParams[(uint8)Resolution].AgentMaxStepHeight = MaxStepHeight; }
	
	/** Returns the tile size in world units. */
	NAVIGATIONSYSTEM_API float GetTileSizeUU() const;
	
	/** Whether NavMesh should adjust its tile pool size when NavBounds are changed */
	NAVIGATIONSYSTEM_API bool IsResizable() const;

	/** Returns bounding box for the whole navmesh. */
	NAVIGATIONSYSTEM_API FBox GetNavMeshBounds() const;

	/** Returns bounding box for a given navmesh tile. */
	UE_DEPRECATED(5.5, "Use the version of this function that takes a FNavTileRef instead.")
	NAVIGATIONSYSTEM_API FBox GetNavMeshTileBounds(int32 TileIndex) const;

	/** Returns bounding box for a given navmesh tile. */
	NAVIGATIONSYSTEM_API FBox GetNavMeshTileBounds(FNavTileRef TileRef) const;

	/** Retrieves XY coordinates of tile specified by index */
	UE_DEPRECATED(5.5, "Use the version of this function that takes a FNavTileRef instead.")
	NAVIGATIONSYSTEM_API bool GetNavMeshTileXY(int32 TileIndex, int32& OutX, int32& OutY, int32& Layer) const;

	/** Retrieves XY coordinates of tile */
	NAVIGATIONSYSTEM_API bool GetNavMeshTileXY(FNavTileRef TileRef, int32& OutX, int32& OutY, int32& Layer) const;

	/** Retrieves XY coordinates of tile specified by position */
	NAVIGATIONSYSTEM_API bool GetNavMeshTileXY(const FVector& Point, int32& OutX, int32& OutY) const;

	/** Retrieves the tile resolution */
	UE_DEPRECATED(5.5, "Use the version of this function that takes a FNavTileRef instead.")
	NAVIGATIONSYSTEM_API bool GetNavmeshTileResolution(int32 TileIndex, ENavigationDataResolution& OutResolution) const;

	/** Retrieves the tile resolution */
	NAVIGATIONSYSTEM_API bool GetNavmeshTileResolution(FNavTileRef TileRef, ENavigationDataResolution& OutResolution) const;

	/** Checks the supplied Points tile indicies can fit in the range of an int32 */
	NAVIGATIONSYSTEM_API bool CheckTileIndicesInValidRange(const FVector& Point, bool& bOutInRange) const;

	/** Retrieves all tile indices at matching XY coordinates */
	UE_DEPRECATED(5.5, "Use the version of this function that takes an array of FNavTileRefs instead.")
	NAVIGATIONSYSTEM_API void GetNavMeshTilesAt(int32 TileX, int32 TileY, TArray<int32>& Indices) const;

	/** Retrieves all tiles at matching XY coordinates */
	NAVIGATIONSYSTEM_API void GetNavMeshTilesAt(int32 TileX, int32 TileY, TArray<FNavTileRef>& OutRefs) const;

	/** Retrieves number of tiles in this navmesh */
	NAVIGATIONSYSTEM_API int32 GetNavMeshTilesCount() const;
	/** Returns the number of tiles that have active data (header and dataSize > 0). Unlike GetNavMeshTilesCount, this counts actual populated tiles rather than pool capacity. */
	NAVIGATIONSYSTEM_API int32 GetNumActiveTiles() const;

	/** Retrieves all tiles in this navmesh */
	NAVIGATIONSYSTEM_API void GetAllNavMeshTiles(TArray<FNavTileRef>& OutRefs) const;

	/** Removes compressed tile data at given tile coord */
	NAVIGATIONSYSTEM_API void RemoveTileCacheLayers(int32 TileX, int32 TileY);
	
	/** Stores compressed tile data for given tile coord */
	NAVIGATIONSYSTEM_API void AddTileCacheLayers(int32 TileX, int32 TileY, const TArray<FNavMeshTileData>& InLayers);

#if RECAST_INTERNAL_DEBUG_DATA
	NAVIGATIONSYSTEM_API void RemoveTileDebugData(int32 TileX, int32 TileY);
	NAVIGATIONSYSTEM_API void AddTileDebugData(int32 TileX, int32 TileY, const struct FRecastInternalDebugData& InTileDebugData);
#endif

	/** Marks tile coord as rebuild and empty */
	NAVIGATIONSYSTEM_API void MarkEmptyTileCacheLayers(int32 TileX, int32 TileY);
	
	/** Returns compressed tile data at given tile coord */
	NAVIGATIONSYSTEM_API TArray<FNavMeshTileData> GetTileCacheLayers(int32 TileX, int32 TileY) const;

	/** Gets the size of the compressed tile cache, this is slow */
#if !UE_BUILD_SHIPPING
	NAVIGATIONSYSTEM_API int32 GetCompressedTileCacheSize();
#endif

	NAVIGATIONSYSTEM_API void GetEdgesForPathCorridor(const TArray<NavNodeRef>* PathCorridor, TArray<struct FNavigationPortalEdge>* PathCorridorEdges) const;

	NAVIGATIONSYSTEM_API void UpdateDrawing();

	/** Creates a task to be executed on GameThread calling UpdateDrawing */
	NAVIGATIONSYSTEM_API void RequestDrawingUpdate(bool bForce = false);

	/** called after regenerating tiles */
	NAVIGATIONSYSTEM_API virtual void OnNavMeshTilesUpdated(const TArray<FNavTileRef>& ChangedTiles);

#if WITH_NAVMESH_SEGMENT_LINKS
	/** Creates segment link connections for the given tiles. Called after navigation tiles are finished generating. */
	NAVIGATIONSYSTEM_API virtual void CreateSegmentLinkConnections(TSet<FNavTileRef>& TilesPendingSegmentLinks);
#endif

	/** Event from generator that navmesh build has finished */
	NAVIGATIONSYSTEM_API virtual void OnNavMeshGenerationFinished();

	NAVIGATIONSYSTEM_API virtual void EnsureBuildCompletion() override;

	NAVIGATIONSYSTEM_API virtual void SetConfig(const FNavDataConfig& Src) override;
protected:
	NAVIGATIONSYSTEM_API virtual void FillConfig(FNavDataConfig& Dest) override;

	inline const FNavigationQueryFilter& GetRightFilterRef(FSharedConstNavQueryFilter Filter) const 
	{
		return *(Filter.IsValid() ? Filter.Get() : GetDefaultQueryFilter().Get());
	}

public:

	static NAVIGATIONSYSTEM_API bool IsVoxelCacheEnabled();

	//----------------------------------------------------------------------//
	// Debug                                                                
	//----------------------------------------------------------------------//

	/* Gather debug geometry.
	 * @params OutGeometry Output geometry.
	 * @params TileIndex Used to collect geometry for a specific tile, INDEX_NONE will gather all tiles
	 * @return True if done collecting.
	 */
	UE_DEPRECATED(5.5, "Use the version of the function that takes a FNavTileRef instead.")
	NAVIGATIONSYSTEM_API bool GetDebugGeometryForTile(FRecastDebugGeometry& OutGeometry, int32 TileIndex) const;

	/* Gather debug geometry.
	 * @params OutGeometry Output geometry.
	 * @params TileRef Used to collect geometry for a specific tile, an invalid FNavTileRef will gather all tiles
	 * @return True if done collecting.
	 */
	NAVIGATIONSYSTEM_API bool GetDebugGeometryForTile(FRecastDebugGeometry& OutGeometry, FNavTileRef TileRef) const;

	// @todo docuement
	NAVIGATIONSYSTEM_API void DrawDebugPathCorridor(NavNodeRef const* PathPolys, int32 NumPathPolys, bool bPersistent=true) const;

#if !UE_BUILD_SHIPPING
	NAVIGATIONSYSTEM_API virtual uint32 LogMemUsed() const override;

	/** Dumps detailed per-section and per-tile memory statistics to the log. */
	NAVIGATIONSYSTEM_API void DumpDetailedMemoryStats() const;	
#endif // !UE_BUILD_SHIPPING

	NAVIGATIONSYSTEM_API void UpdateNavMeshDrawing();

	//----------------------------------------------------------------------//
	// Utilities
	//----------------------------------------------------------------------//
	NAVIGATIONSYSTEM_API virtual void OnNavAreaChanged() override;
	NAVIGATIONSYSTEM_API virtual void OnNavAreaAdded(const UClass* NavAreaClass, int32 AgentIndex) override;
	NAVIGATIONSYSTEM_API virtual void OnNavAreaRemoved(const UClass* NavAreaClass) override;
	NAVIGATIONSYSTEM_API virtual int32 GetNewAreaID(const UClass* AreaClass) const override;
	virtual int32 GetMaxSupportedAreas() const override { return RECAST_MAX_AREAS; }

	/** Get forbidden area flags from default query filter */
	NAVIGATIONSYSTEM_API uint16 GetDefaultForbiddenFlags() const;
	/** Change forbidden area flags in default query filter */
	NAVIGATIONSYSTEM_API void SetDefaultForbiddenFlags(uint16 ForbiddenAreaFlags);

	/** Area sort function */
	NAVIGATIONSYSTEM_API virtual void SortAreasForGenerator(TArray<FRecastAreaNavModifierElement>& Areas) const;

	NAVIGATIONSYSTEM_API virtual void RecreateDefaultFilter();

	int32 GetMaxSimultaneousTileGenerationJobsCount() const { return MaxSimultaneousTileGenerationJobsCount; }
	NAVIGATIONSYSTEM_API void SetMaxSimultaneousTileGenerationJobsCount(int32 NewJobsCountLimit);

	/** Returns query extent including adjustments for voxelization error compensation */
	FVector GetModifiedQueryExtent(const FVector& QueryExtent) const
	{
		// Using HALF_WORLD_MAX instead of BIG_NUMBER, else using the extent for a box will result in NaN.
		return FVector(QueryExtent.X, QueryExtent.Y, QueryExtent.Z >= HALF_WORLD_MAX ? HALF_WORLD_MAX : (QueryExtent.Z + FMath::Max(0., VerticalDeviationFromGroundCompensation)));
	}

	//----------------------------------------------------------------------//
	// Custom navigation links
	//----------------------------------------------------------------------//

	NAVIGATIONSYSTEM_API virtual void UpdateCustomLink(const INavLinkCustomInterface* CustomLink) override;

	UE_DEPRECATED(5.3, "Use version of this function that takes a FNavLinkId. This function now has no effect.")
	void UpdateNavigationLinkArea(int32 UserId, TSubclassOf<UNavArea> AreaClass) const {}

	/** update area class and poly flags for all offmesh links with given UserId */
	NAVIGATIONSYSTEM_API void UpdateNavigationLinkArea(FNavLinkId UserId, TSubclassOf<UNavArea> AreaClass) const;

#if WITH_NAVMESH_SEGMENT_LINKS
	/** update area class and poly flags for all offmesh segment links with given UserId */
	NAVIGATIONSYSTEM_API void UpdateSegmentLinkArea(int32 UserId, TSubclassOf<UNavArea> AreaClass) const;
#endif // WITH_NAVMESH_SEGMENT_LINKS

	//----------------------------------------------------------------------//
	// Batch processing (important with async rebuilding)
	//----------------------------------------------------------------------//

	/** Starts batch processing and locks access to navmesh from other threads */
	NAVIGATIONSYSTEM_API virtual void BeginBatchQuery() const override;

	/** Finishes batch processing and release locks */
	NAVIGATIONSYSTEM_API virtual void FinishBatchQuery() const override;

	//----------------------------------------------------------------------//
	// Querying                                                                
	//----------------------------------------------------------------------//
	
	/** dtNavMesh getter */
	NAVIGATIONSYSTEM_API dtNavMesh* GetRecastMesh();

	/** dtNavMesh getter */
	NAVIGATIONSYSTEM_API const dtNavMesh* GetRecastMesh() const;

	UE_DEPRECATED(5.3, "Please use GetNavLinkUserId() instead. This function only returns Invalid.")
	int32 GetLinkUserId(NavNodeRef LinkPolyID) const
	{
		return static_cast<int32>(FNavLinkId::Invalid.GetId());
	}

	/** Retrieves LinkUserID associated with indicated PolyID */
	NAVIGATIONSYSTEM_API FNavLinkId GetNavLinkUserId(NavNodeRef LinkPolyID) const;

	NAVIGATIONSYSTEM_API FColor GetAreaIDColor(uint8 AreaID) const;

	/** Finds the polygons along the navigation graph that touch the specified circle. */
	NAVIGATIONSYSTEM_API bool FindPolysAroundCircle(const FVector& CenterPos, const NavNodeRef CenterNodeRef, const FVector::FReal Radius, const FSharedConstNavQueryFilter& Filter, const UObject* QueryOwner, TArray<NavNodeRef>* OutPolys = nullptr, TArray<NavNodeRef>* OutPolysParent = nullptr, TArray<float>* OutPolysCost = nullptr, int32* OutPolysCount = nullptr) const;
	
	/** Returns nearest navmesh polygon to Loc, or INVALID_NAVNODEREF if Loc is not on the navmesh. */
	NAVIGATIONSYSTEM_API NavNodeRef FindNearestPoly(FVector const& Loc, FVector const& Extent, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const;

	/** Finds the distance to the closest wall, limited to MaxDistance
	 *	[out] OutClosestPointOnWall, if supplied, will be set to closest point on closest wall. Will not be set if no wall in the area (return value 0.f) */
	NAVIGATIONSYSTEM_API FVector::FReal FindDistanceToWall(const FVector& StartLoc, FSharedConstNavQueryFilter Filter = nullptr, FVector::FReal MaxDistance = TNumericLimits<FVector::FReal>::Max(), FVector* OutClosestPointOnWall = nullptr) const;

	/** Retrieves center of the specified polygon. Returns false on error. */
	NAVIGATIONSYSTEM_API bool GetPolyCenter(NavNodeRef PolyID, FVector& OutCenter) const;

	/** Retrieves the vertices for the specified polygon. Returns false on error. */
	NAVIGATIONSYSTEM_API bool GetPolyVerts(NavNodeRef PolyID, TArray<FVector>& OutVerts) const;

	/** Retrieves a random point inside the specified polygon. Returns false on error. */
	NAVIGATIONSYSTEM_API bool GetRandomPointInPoly(NavNodeRef PolyID, FVector& OutPoint) const;

	/** Retrieves the surface area of the specified polygon. Returns 0 on error. */
	NAVIGATIONSYSTEM_API FVector::FReal GetPolySurfaceArea(NavNodeRef PolyID) const;

	/** Retrieves area ID for the specified polygon. */
	NAVIGATIONSYSTEM_API uint32 GetPolyAreaID(NavNodeRef PolyID) const;

	/** Sets area ID for the specified polygon. */
	NAVIGATIONSYSTEM_API bool SetPolyArea(NavNodeRef PolyID, TSubclassOf<UNavArea> AreaClass);

	/** Sets area ID for the specified polygons */
	NAVIGATIONSYSTEM_API void SetPolyArrayArea(const TArray<FNavPoly>& Polys, TSubclassOf<UNavArea> AreaClass);

	/** In given Bounds find all areas of class OldArea and replace them with NewArea
	 *	@return number of polys touched */
	NAVIGATIONSYSTEM_API int32 ReplaceAreaInTileBounds(const FBox& Bounds, TSubclassOf<UNavArea> OldArea, TSubclassOf<UNavArea> NewArea, bool ReplaceLinks = true, TArray<NavNodeRef>* OutTouchedNodes = nullptr);
	
	/** Retrieves poly and area flags for specified polygon */
	NAVIGATIONSYSTEM_API bool GetPolyFlags(NavNodeRef PolyID, uint16& PolyFlags, uint16& AreaFlags) const;
	NAVIGATIONSYSTEM_API bool GetPolyFlags(NavNodeRef PolyID, FNavMeshNodeFlags& Flags) const;

	/** Finds all polys connected with specified one */
	NAVIGATIONSYSTEM_API bool GetPolyNeighbors(NavNodeRef PolyID, TArray<FNavigationPortalEdge>& Neighbors) const;

	/** Finds all polys connected with specified one, results expressed as array of NavNodeRefs */
	NAVIGATIONSYSTEM_API bool GetPolyNeighbors(NavNodeRef PolyID, TArray<NavNodeRef>& Neighbors) const;

	/** Finds edges of specified poly */
	NAVIGATIONSYSTEM_API bool GetPolyEdges(NavNodeRef PolyID, TArray<FNavigationPortalEdge>& Neighbors) const;

	/** Finds all wall segments for the specified polygon (walls or area borders) */
	NAVIGATIONSYSTEM_API bool GetPolyWallSegments(NavNodeRef PolyID, FSharedConstNavQueryFilter Filter, const UObject* QueryOwner, TArray<FNavigationPortalEdge>& OutNeighbors) const;

	/** Finds closest point constrained to given poly */
	NAVIGATIONSYSTEM_API bool GetClosestPointOnPoly(NavNodeRef PolyID, const FVector& TestPt, FVector& PointOnPoly) const;

	/** Decode poly ID into tile index and poly index */
	NAVIGATIONSYSTEM_API bool GetPolyTileIndex(NavNodeRef PolyID, uint32& PolyIndex, uint32& TileIndex) const;

	/** Retrieves start and end point of offmesh link */
	NAVIGATIONSYSTEM_API bool GetLinkEndPoints(NavNodeRef LinkPolyID, FVector& PointA, FVector& PointB) const;

	/** Retrieves bounds of cluster. Returns false on error. */
	NAVIGATIONSYSTEM_API bool GetClusterBounds(NavNodeRef ClusterRef, FBox& OutBounds) const;

	/** Get random point in given cluster */
	NAVIGATIONSYSTEM_API bool GetRandomPointInCluster(NavNodeRef ClusterRef, FNavLocation& OutLocation) const;

	/** Get cluster ref containing given poly ref */
	NAVIGATIONSYSTEM_API NavNodeRef GetClusterRef(NavNodeRef PolyRef) const;

	/** Retrieves all polys within given pathing distance from StartLocation.
	 *	@NOTE query is not using string-pulled path distance (for performance reasons),
	 *		it measured distance between middles of portal edges, do you might want to 
	 *		add an extra margin to PathingDistance */
	NAVIGATIONSYSTEM_API bool GetPolysWithinPathingDistance(FVector const& StartLoc, const FVector::FReal PathingDistance, TArray<NavNodeRef>& FoundPolys,
		FSharedConstNavQueryFilter Filter = nullptr, const UObject* Querier = nullptr, FRecastDebugPathfindingData* DebugData = nullptr) const;

	/** Filters nav polys in PolyRefs with Filter */
	NAVIGATIONSYSTEM_API bool FilterPolys(TArray<NavNodeRef>& PolyRefs, const FRecastQueryFilter* Filter, const UObject* Querier = NULL) const;

	/** Get all polys from tile */
	UE_DEPRECATED(5.5, "Use the version of this function that takes a FNavTileRef instead.")
	NAVIGATIONSYSTEM_API bool GetPolysInTile(int32 TileIndex, TArray<FNavPoly>& Polys) const;

	/** Get all polys from tile */
	NAVIGATIONSYSTEM_API bool GetPolysInTile(FNavTileRef TileRef, TArray<FNavPoly>& Polys) const;

	/** Get up to 256 polys that overlap the specified box */
	NAVIGATIONSYSTEM_API bool GetPolysInBox(const FBox& Box, TArray<FNavPoly>& Polys, FSharedConstNavQueryFilter Filter = nullptr, const UObject* Owner = nullptr) const;

	/** Find up to 64 navmesh eges in up to 64 polys around the center */
	NAVIGATIONSYSTEM_API bool FindEdges(const NavNodeRef CenterNodeRef, const FVector Center, const FVector::FReal Radius, const FSharedConstNavQueryFilter Filter, TArray<FNavigationWallEdge>& OutEdges) const;

	/** Get all  navmesh edges from tile that form the border of the passed in Filter */
	NAVIGATIONSYSTEM_API void GetTilePolyEdgesForFilter(FNavTileRef TileRef, FSharedConstNavQueryFilter Filter, TArray<FNavigationWallEdge>& OutEdges) const;

	/** Get all exterior nav mesh edges from tile */
	NAVIGATIONSYSTEM_API bool GetEdgesInTile(FNavTileRef TileRef, TArray<FNavigationWallEdge>& OutEdges) const;

	/** Get all polys from tile */
	NAVIGATIONSYSTEM_API bool GetNavLinksInTile(const int32 TileIndex, TArray<FNavPoly>& Polys, const bool bIncludeLinksFromNeighborTiles) const;

	/** Projects point on navmesh, returning all hits along vertical line defined by min-max Z params */
	NAVIGATIONSYSTEM_API bool ProjectPointMulti(const FVector& Point, TArray<FNavLocation>& OutLocations, const FVector& Extent,
		FVector::FReal MinZ, FVector::FReal MaxZ, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const;
	
	// @todo document
	static NAVIGATIONSYSTEM_API FPathFindingResult FindPath(const FNavAgentProperties& AgentProperties, const FPathFindingQuery& Query);
	static NAVIGATIONSYSTEM_API bool TestPath(const FNavAgentProperties& AgentProperties, const FPathFindingQuery& Query, int32* NumVisitedNodes);
	static NAVIGATIONSYSTEM_API bool TestHierarchicalPath(const FNavAgentProperties& AgentProperties, const FPathFindingQuery& Query, int32* NumVisitedNodes);
	static NAVIGATIONSYSTEM_API bool NavMeshRaycast(const ANavigationData* Self, const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, FSharedConstNavQueryFilter QueryFilter, const UObject* Querier, FRaycastResult& Result);
	static bool NavMeshRaycast(const ANavigationData* Self, const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, FSharedConstNavQueryFilter QueryFilter, const UObject* Querier = NULL);
	static bool NavMeshRaycast(const ANavigationData* Self, const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, FNavigationRaycastAdditionalResults* AdditionalResults, FSharedConstNavQueryFilter QueryFilter, const UObject* Querier = NULL);
	static NAVIGATIONSYSTEM_API bool NavMeshRaycast(const ANavigationData* Self, NavNodeRef RayStartNode, const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, FSharedConstNavQueryFilter QueryFilter, const UObject* Querier = NULL);
	static NAVIGATIONSYSTEM_API bool NavMeshRaycast(const ANavigationData* Self, NavNodeRef RayStartNode, const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, FNavigationRaycastAdditionalResults* AdditionalResults, FSharedConstNavQueryFilter QueryFilter, const UObject* Querier = NULL);

	NAVIGATIONSYSTEM_API virtual void BatchRaycast(TArray<FNavigationRaycastWork>& Workload, FSharedConstNavQueryFilter QueryFilter, const UObject* Querier = NULL) const override;

	/** finds a Filter-passing navmesh location closest to specified StartLoc
	 *	@return true if adjusting was required, false otherwise */
	NAVIGATIONSYSTEM_API bool AdjustLocationWithFilter(const FVector& StartLoc, FVector& OutAdjustedLocation, const FNavigationQueryFilter& Filter, const UObject* Querier = NULL) const;
	
	/** Check if navmesh is defined (either built/streamed or recognized as empty tile by generator) in given radius.
	  * @returns true if ALL tiles inside are ready
	  */
	NAVIGATIONSYSTEM_API bool HasCompleteDataInRadius(const FVector& TestLocation, FVector::FReal TestRadius) const;
	
	/** Check if navmesh is defined (either built/streamed or recognized as empty tile by generator) within given radius around the given segment.
	* @returns true if ALL tiles inside are ready
	*/
	NAVIGATIONSYSTEM_API bool HasCompleteDataAroundSegment(const FVector& StartLocation, const FVector& EndLocation, FVector::FReal TestRadius) const;

	/** @return true is specified segment is fully on navmesh (respecting the optional filter) */
	NAVIGATIONSYSTEM_API bool IsSegmentOnNavmesh(const FVector& SegmentStart, const FVector& SegmentEnd, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const;

	/** Check if poly is a custom link */
	NAVIGATIONSYSTEM_API bool IsCustomLink(NavNodeRef PolyRef) const;

	/** finds stringpulled path from given corridor */
	NAVIGATIONSYSTEM_API bool FindStraightPath(const FVector& StartLoc, const FVector& EndLoc, const TArray<NavNodeRef>& PathCorridor, TArray<FNavPathPoint>& PathPoints, TArray<FNavLinkId>* CustomLinks = NULL) const;

	/** Runs A* pathfinding on navmesh and collect data for every step */
	NAVIGATIONSYSTEM_API int32 DebugPathfinding(const FPathFindingQuery& Query, TArray<FRecastDebugPathfindingData>& Steps);

	static NAVIGATIONSYSTEM_API const FRecastQueryFilter* GetNamedFilter(ERecastNamedFilter::Type FilterType);
	inline static FNavPolyFlags GetNavLinkFlag() { return NavLinkFlag; }
	
	NAVIGATIONSYSTEM_API virtual bool NeedsRebuild() const override;
	NAVIGATIONSYSTEM_API virtual bool SupportsRuntimeGeneration() const override;
	NAVIGATIONSYSTEM_API virtual bool SupportsStreaming() const override;

	NAVIGATIONSYSTEM_API bool IsWorldPartitionedDynamicNavmesh() const;
	
	/** When using active tiles generation, navigation is only allowed to be runtime generated on a subset of tiles.
	 *  The subset is be defined by navinvokers or loaded world partitioned cells. */
	NAVIGATIONSYSTEM_API bool IsUsingActiveTilesGeneration(const UNavigationSystemV1& NavSys) const;

	/** Runs after LoadBeforeGeneratorRebuild but before the rebuild. */
	NAVIGATIONSYSTEM_API virtual void PostLoadPreRebuild() override;
	
	NAVIGATIONSYSTEM_API virtual void ConditionalConstructGenerator() override;
	
	bool ShouldGatherDataOnGameThread() const { return bDoFullyAsyncNavDataGathering == false; }
	int32 GetTileNumberHardLimit() const { return TileNumberHardLimit; }

	NAVIGATIONSYSTEM_API virtual void UpdateActiveTiles(const TArray<FNavigationInvokerRaw>& InvokerLocations);
	NAVIGATIONSYSTEM_API virtual void RemoveTiles(const TArray<FIntPoint>& Tiles);

	NAVIGATIONSYSTEM_API void RebuildTile(const TArray<FNavMeshDirtyTileElement>& Tiles);
	
	NAVIGATIONSYSTEM_API void DirtyTilesInBounds(const FBox& Bounds);

#if RECAST_INTERNAL_DEBUG_DATA
	NAVIGATIONSYSTEM_API const TMap<FIntPoint, struct FRecastInternalDebugData>* GetDebugDataMap() const;
#endif

#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "Use GetNavLinkJumpConfigs instead.")
	const FNavLinkGenerationJumpDownConfig& GetNavLinkJumpDownConfig() const { return NavLinkJumpDownConfig; }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

	const TArray<FNavLinkGenerationJumpConfig>& GetNavLinkJumpConfigs() const { return NavLinkJumpConfigs; }

protected:

	NAVIGATIONSYSTEM_API void UpdatePolyRefBitsPreview();

	/** Invalidates active paths that go through changed tiles  */
	NAVIGATIONSYSTEM_API void InvalidateAffectedPaths(const TArray<FNavTileRef>& ChangedTiles);

	/** created a new FRecastNavMeshGenerator instance. Overrider to supply your
	 *	own extentions. Note: needs to derive from FRecastNavMeshGenerator */
	NAVIGATIONSYSTEM_API virtual FRecastNavMeshGenerator* CreateGeneratorInstance();

	NAVIGATIONSYSTEM_API void CheckToDiscardSubLevelNavData(const UNavigationSystemBase& NavSys);
	
	bool IsGeneratingLinks() const;
	void RegisterGeneratedLinksProxy(FNavLinkGenerationJumpConfig& InOutNavLinkJumpConfig);
	void UnregisterGeneratedLinksProxy(FNavLinkGenerationJumpConfig& InOutNavLinkJumpConfig);

	UE_DEPRECATED(5.8, "Use the override using an FNavLinkGenerationJumpConfig instead.")
	void RegisterGeneratedLinksProxy();

	UE_DEPRECATED(5.8, "Use the override using an FNavLinkGenerationJumpConfig instead.")
	void UnregisterGeneratedLinksProxy();

	/* Create and register links proxy. It's expected to be called on load or when the navmesh is rebuilt. */
	UE_DEPRECATED(5.7, "Use the override using an FNavLinkGenerationJumpConfig instead.")
	void CreateAndRegisterJumpDownLinksProxy(const FNavLinkId LinkProxyId = FNavLinkId::GenerateUniqueId());

	/* Create and register links proxy. It's expected to be called on load or when the navmesh is rebuilt. */
	void CreateAndRegisterJumpLinksProxy(FNavLinkGenerationJumpConfig& InOutNavLinkJumpConfig, const FNavLinkId LinkProxyId = FNavLinkId::GenerateUniqueId());
	
private:
	friend struct FRecastGraphWrapper;
	friend FRecastNavMeshGenerator;
	friend class FPImplRecastNavMesh;
	friend class URecastNavMeshDataChunk;
	// destroys FPImplRecastNavMesh instance if it has been created 
	NAVIGATIONSYSTEM_API void DestroyRecastPImpl();
	// @todo docuement
	NAVIGATIONSYSTEM_API void UpdateNavVersion();
	NAVIGATIONSYSTEM_API void UpdateNavObject();

	/** @return Navmesh data chunk that belongs to this actor */
	NAVIGATIONSYSTEM_API URecastNavMeshDataChunk* GetNavigationDataChunk(ULevel* InLevel) const;

	/** @return Navmesh data chunk that belongs to this actor */
	NAVIGATIONSYSTEM_API URecastNavMeshDataChunk* GetNavigationDataChunk(const ANavigationDataChunkActor& InActor) const;

	/** Check if navmesh is defined (either built/streamed or recognized as empty tile by generator) in given tile */
	bool HasCompleteDataInTile(const int32 TileX, const int32 TileY) const;

protected:
	// retrieves RecastNavMeshImpl
	FPImplRecastNavMesh* GetRecastNavMeshImpl() { return RecastNavMeshImpl; }
	const FPImplRecastNavMesh* GetRecastNavMeshImpl() const { return RecastNavMeshImpl; }

	struct FUpdateActiveTilesWorkingMem
	{
		TSet<FIntPoint> OldActiveSet;

#if WITH_EDITORONLY_DATA
		UE_DEPRECATED(5.5, "TilesInMinDistance not used anymore, use TilesInMinDistanceMap instead.")
		TArray<FNavMeshDirtyTileElement> TilesInMinDistance;
#endif
		TMap<FIntPoint, FNavMeshDirtyTileElement> TilesInMinDistanceMap;
		
		TSet<FIntPoint> TilesInMaxDistance;
		TArray<FIntPoint> TileToAppend;
	};
	FUpdateActiveTilesWorkingMem UpdateActiveTilesWorkingMem;
	
private:
	/** @return Navmesh data chunk that belongs to this actor */
	NAVIGATIONSYSTEM_API URecastNavMeshDataChunk* GetNavigationDataChunk(const TArray<UNavigationDataChunk*>& InChunks) const;

	/** NavMesh versioning. */
	uint32 NavMeshVersion;
	
	/** 
	 * This is a pimpl-style arrangement used to tightly hide the Recast internals from the rest of the engine.
	 * Using this class should *not* require the inclusion of the private RecastNavMesh.h
	 *	@NOTE: if we switch over to C++11 this should be unique_ptr
	 *	@TODO since it's no secret we're using recast there's no point in having separate implementation class. FPImplRecastNavMesh should be merged into ARecastNavMesh
	 */
	FPImplRecastNavMesh* RecastNavMeshImpl;
	
#if RECAST_ASYNC_REBUILDING
	/** batch query counter */
	mutable int32 BatchQueryCounter;

#endif // RECAST_ASYNC_REBUILDING

private:
	static NAVIGATIONSYSTEM_API const FRecastQueryFilter* NamedFilters[ERecastNamedFilter::NamedFiltersCount];
#else
	virtual bool IsNodeRefValid(NavNodeRef NodeRef) const override { return true; }
	virtual FBox GetBounds() const override { return FBox(); }
	virtual void BatchRaycast(TArray<FNavigationRaycastWork>& Workload, FSharedConstNavQueryFilter QueryFilter, const UObject* Querier = nullptr) const override {}
	virtual bool FindMoveAlongSurface(const FNavLocation& StartLocation, const FVector& TargetPosition, FNavLocation& OutLocation, FSharedConstNavQueryFilter Filter = nullptr, const UObject* Querier = nullptr) const override { return false; }
	virtual void BatchFindMoveAlongSurface(TArray<FNavigationFindMoveAlongSurfaceWork>& Workload, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const override {}
	virtual bool FindOverlappingEdges(const FNavLocation& StartLocation, TConstArrayView<FVector> ConvexPolygon, TArray<FVector>& OutEdges, FSharedConstNavQueryFilter Filter = nullptr, const UObject* Querier = nullptr) const override { return false; }
	virtual bool GetPathSegmentBoundaryEdges(const FNavigationPath& Path, const FNavPathPoint& StartPoint, const FNavPathPoint& EndPoint, const TConstArrayView<FVector> SearchArea, TArray<FVector>& OutEdges, const float MaxAreaEnterCost, FSharedConstNavQueryFilter Filter = nullptr, const UObject* Querier = nullptr) const override { return false; }
	virtual FNavLocation GetRandomPoint(FSharedConstNavQueryFilter Filter = nullptr, const UObject* Querier = nullptr) const override { return FNavLocation(); }
	virtual bool GetRandomReachablePointInRadius(const FVector& Origin, float Radius, FNavLocation& OutResult, FSharedConstNavQueryFilter Filter = nullptr, const UObject* Querier = nullptr) const override { return false; }
	virtual bool GetRandomPointInNavigableRadius(const FVector& Origin, float Radius, FNavLocation& OutResult, FSharedConstNavQueryFilter Filter = nullptr, const UObject* Querier = nullptr) const override { return false; }
	virtual bool ProjectPoint(const FVector& Point, FNavLocation& OutLocation, const FVector& Extent, FSharedConstNavQueryFilter Filter = nullptr, const UObject* Querier = nullptr) const override { return false; }
	virtual void BatchProjectPoints(TArray<FNavigationProjectionWork>& Workload, const FVector& Extent, FSharedConstNavQueryFilter Filter = nullptr, const UObject* Querier = nullptr) const override {}
	virtual void BatchProjectPoints(TArray<FNavigationProjectionWork>& Workload, FSharedConstNavQueryFilter Filter = nullptr, const UObject* Querier = nullptr) const override {}
	virtual ENavigationQueryResult::Type CalcPathCost(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& OutPathCost, FSharedConstNavQueryFilter QueryFilter = nullptr, const UObject* Querier = nullptr) const override { return ENavigationQueryResult::Invalid; }
	virtual ENavigationQueryResult::Type CalcPathLength(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& OutPathLength, FSharedConstNavQueryFilter QueryFilter = nullptr, const UObject* Querier = nullptr) const override { return ENavigationQueryResult::Invalid; }
	virtual ENavigationQueryResult::Type CalcPathLengthAndCost(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& OutPathLength, FVector::FReal& OutPathCost, FSharedConstNavQueryFilter QueryFilter = nullptr, const UObject* Querier = nullptr) const override { return ENavigationQueryResult::Invalid; }
	virtual bool DoesNodeContainLocation(NavNodeRef NodeRef, const FVector& WorldSpaceLocation) const override { return false; }
#endif // WITH_RECAST

public:
	//----------------------------------------------------------------------//
	// Blueprint functions
	//----------------------------------------------------------------------//

	/** @return true if any polygon/link has been touched */
	UFUNCTION(BlueprintCallable, Category = NavMesh, meta = (DisplayName = "ReplaceAreaInTileBounds"))
	NAVIGATIONSYSTEM_API bool K2_ReplaceAreaInTileBounds(FBox Bounds, TSubclassOf<UNavArea> OldArea, TSubclassOf<UNavArea> NewArea, bool ReplaceLinks = true);
};

#if WITH_RECAST
inline
bool ARecastNavMesh::NavMeshRaycast(const ANavigationData* Self, const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, FSharedConstNavQueryFilter QueryFilter, const UObject* Querier)
{
	return NavMeshRaycast(Self, RayStart, RayEnd, HitLocation, nullptr, QueryFilter, Querier);
}

inline
bool ARecastNavMesh::NavMeshRaycast(const ANavigationData* Self, const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, FNavigationRaycastAdditionalResults* AdditionalResults, FSharedConstNavQueryFilter QueryFilter, const UObject* Querier)
{
	FRaycastResult Result;
	const bool bDidHit = NavMeshRaycast(Self, RayStart, RayEnd, HitLocation, QueryFilter, Querier, Result);
	if (AdditionalResults)
	{
		AdditionalResults->bIsRayEndInCorridor = Result.bIsRaycastEndInCorridor;
	}
	return bDidHit;
}


/** structure to cache owning RecastNavMesh data so that it doesn't have to be polled
 *	directly from RecastNavMesh while asyncronously generating navmesh */
struct FRecastNavMeshCachedData
{
	ARecastNavMesh::FNavPolyFlags FlagsPerArea[RECAST_MAX_AREAS];
	ARecastNavMesh::FNavPolyFlags FlagsPerOffMeshLinkArea[RECAST_MAX_AREAS];
	TMap<const UClass*, int32> AreaClassToIdMap;
	TWeakObjectPtr<const ARecastNavMesh> ActorOwner;
	uint32 bUseSortFunction : 1;

	static FRecastNavMeshCachedData Construct(const ARecastNavMesh* RecastNavMeshActor);
	void OnAreaAdded(const UClass* AreaClass, int32 AreaID);
	void OnAreaRemoved(const UClass* AreaClass);
};

#endif // WITH_RECAST
