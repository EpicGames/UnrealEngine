// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "AI/Navigation/NavigationDataChunk.h"
#include "NavMesh/RecastNavMesh.h"
#include "RecastNavMeshDataChunk.generated.h"

class FPImplRecastNavMesh;

struct FRecastTileData
{
	struct FRawData
	{
		FRawData(uint8* InData);
		~FRawData();

		uint8* RawData;
	};

	FRecastTileData();
	FRecastTileData(int32 TileDataSize, uint8* TileRawData, int32 TileCacheDataSize, uint8* TileCacheRawData);
	
	// Location of attached tile
	int32					OriginalX;	// Tile X coordinates when gathered
	int32					OriginalY;	// Tile Y coordinates when gathered
	int32					X;					
	int32					Y;					
	int32					Layer;
		
	// Tile data
	int32					TileDataSize;
	TSharedPtr<FRawData>	TileRawData;

	// Compressed tile cache layer 
	int32					TileCacheDataSize;
	TSharedPtr<FRawData>	TileCacheRawData;

	// Whether this tile is attached to NavMesh
	bool					bAttached;	
};

class dtNavMesh;
class FPImplRecastNavMesh;

enum EGatherTilesCopyMode
{
	NoCopy = 0,
	CopyData = 1 << 0,
	CopyCacheData = 1 << 1,
	CopyDataAndCacheData = (CopyData | CopyCacheData)
};

/** 
 * 
 */
UCLASS(MinimalAPI)
class URecastNavMeshDataChunk : public UNavigationDataChunk
{
	GENERATED_UCLASS_BODY()

	//~ Begin UObject Interface
	NAVIGATIONSYSTEM_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

#if WITH_RECAST
	/** Attaches tiles to specified navmesh, transferring tile ownership to navmesh */
	NAVIGATIONSYSTEM_API TArray<FNavTileRef> AttachTiles(ARecastNavMesh& NavMesh);

	/** Attaches tiles to specified navmesh */
	NAVIGATIONSYSTEM_API TArray<FNavTileRef> AttachTiles(ARecastNavMesh& NavMesh, const bool bKeepCopyOfData, const bool bKeepCopyOfCacheData);

	/** Detaches tiles from specified navmesh, taking tile ownership */
	NAVIGATIONSYSTEM_API TArray<FNavTileRef> DetachTiles(ARecastNavMesh& NavMesh);

	/** Detaches tiles from specified navmesh */
	NAVIGATIONSYSTEM_API TArray<FNavTileRef> DetachTiles(ARecastNavMesh& NavMesh, const bool bTakeDataOwnership, const bool bTakeCacheDataOwnership);
#endif // WITH_RECAST

	/** 
	 * Experimental: Moves tiles data on the xy plane by the offset (in tile coordinates) and rotation (in degree).
	 * @param NavMeshImpl		Recast navmesh implementation.
	 * @param Offset			Offset in tile coordinates.
	 * @param RotationDeg		Rotation in degrees.
	 * @param RotationCenter	World position
	 */
	NAVIGATIONSYSTEM_API void MoveTiles(FPImplRecastNavMesh& NavMeshImpl, const FIntPoint& Offset, const FVector::FReal RotationDeg, const FVector2D& RotationCenter);
	
	/** Number of tiles in this chunk */
	NAVIGATIONSYSTEM_API int32 GetNumTiles() const;

	/** Const accessor to the list of tiles in the data chunk. */
	const TArray<FRecastTileData>& GetTiles() const { return Tiles; }

#if WITH_RECAST
	/** Returns the AABB for the given tiles. */
	NAVIGATIONSYSTEM_API void GetTilesBounds(const FPImplRecastNavMesh& NavMeshImpl, const TArray<FNavTileRef>& TileRefs, FBox& OutBounds) const;
#endif // WITH_RECAST

	/** Mutable accessor to the list of tiles in the data chunk. */
	TArray<FRecastTileData>& GetMutableTiles() { return Tiles; }

	/** Releases all tiles that this chunk holds */
	NAVIGATIONSYSTEM_API void ReleaseTiles();

	/** Collect tiles with data and/or cache data from the provided TileIndices. */
	UE_DEPRECATED(5.5, "Use the version of this function that takes an array of FNavTileRefs instead.")
	NAVIGATIONSYSTEM_API void GetTiles(const FPImplRecastNavMesh* NavMeshImpl, const TArray<int32>& TileIndices, const EGatherTilesCopyMode CopyMode, const bool bMarkAsAttached = true);

#if WITH_RECAST
	/** Collect tiles with data and/or cache data from the provided tile references. */
	NAVIGATIONSYSTEM_API void GetTiles(const FPImplRecastNavMesh* NavMeshImpl, const TArray<FNavTileRef>& TileRefs, const EGatherTilesCopyMode CopyMode, const bool bMarkAsAttached = true);
#endif // WITH_RECAST

private:
#if WITH_RECAST
	NAVIGATIONSYSTEM_API void SerializeRecastData(FArchive& Ar, int32 NavMeshVersion);
#endif//WITH_RECAST

private:
	TArray<FRecastTileData> Tiles;
};
