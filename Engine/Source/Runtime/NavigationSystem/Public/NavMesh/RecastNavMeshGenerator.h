// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AI/NavDataGenerator.h"
#include "AI/Navigation/NavigationDirtyArea.h"
#include "AI/Navigation/NavigationInvokerPriority.h"
#include "AI/Navigation/NavigationRelevantData.h"
#include "AI/Navigation/NavigationTypes.h"
#include "AI/NavigationModifier.h"
#include "AI/NavigationSystemBase.h"
#include "Async/AsyncWork.h"
#include "EngineDefines.h"
#include "NavDebugTypes.h"
#include "NavigationOctree.h"
#include "NavMesh/RecastHelpers.h"
#include "NavMesh/RecastNavMesh.h"
#include "UObject/GCObject.h"

#if WITH_RECAST

#include "Recast/Recast.h"
#include "Detour/DetourNavMesh.h"
#include "Detour/DetourNavLinkBuilderConfig.h"

#if RECAST_INTERNAL_DEBUG_DATA
#include "NavMesh/RecastInternalDebugData.h"
#endif

class UBodySetup;
class ARecastNavMesh;
class FNavigationOctree;
class FNavMeshBuildContext;
class FRecastNavMeshGenerator;
struct FTileRasterizationContext;
struct BuildContext;
struct dtTileCacheLayer;
struct FKAggregateGeom;
struct FTileCacheCompressor;
struct FTileCacheAllocator;
struct FTileGenerationContext;
struct dtLinkBuilderData;
struct FNavigationElement;
class dtNavMesh;
class FNavRegenTimeSliceManager;
class UNavigationSystemV1;

#define MAX_VERTS_PER_POLY	6

struct FRecastBuildConfig : public rcConfig
{
	/** controls whether voxel filtering will be applied (via FRecastTileGenerator::ApplyVoxelFilter) */
	uint32 bPerformVoxelFiltering:1;
	/** generate detailed mesh (additional tessellation to match heights of geometry) */
	uint32 bGenerateDetailedMesh:1;
	/** generate BV tree (space partitioning for queries) */
	uint32 bGenerateBVTree:1;
	/** if set, mark areas with insufficient free height instead of cutting them out  */
	uint32 bMarkLowHeightAreas : 1;
	/** Expand the top of the area nav modifier's bounds by one cell height when applying to the navmesh.
		If unset, navmesh on top of surfaces might not be marked by marking bounds flush with top surfaces (since navmesh is generated slightly above collision, depending on cell height). */
	uint32 bUseExtraTopCellWhenMarkingAreas : 1;
	/** if set, only single low height span will be allowed under valid one */
	uint32 bFilterLowSpanSequences : 1;
	/** if set, only low height spans with corresponding area modifier will be stored in tile cache (reduces memory, can't modify without full tile rebuild) */
	uint32 bFilterLowSpanFromTileCache : 1;
	/** if set, navigation links will be automatically generated */
	uint32 bGenerateLinks : 1;
	/** if set, minimize base link pool size, else use previous larger base link pool. */
	uint32 bMinimizeLinkPoolSize : 1;

	/** region partitioning method used by tile cache */
	int32 TileCachePartitionType;
	/** chunk size for ChunkyMonotone partitioning */
	int32 TileCacheChunkSize;

	/** indicates what's the limit of navmesh polygons per tile. This value is calculated from other
	 *	factors - DO NOT SET IT TO ARBITRARY VALUE */
	int32 MaxPolysPerTile;

	/** NavLink building configuration */
	TArray<dtNavLinkBuilderJumpConfig> JumpConfigs;
	
	/** Actual agent height (in uu)*/
	float AgentHeight;
	/** Actual agent climb (in uu)*/
	float AgentMaxClimb;
	/** Actual agent radius (in uu)*/
	float AgentRadius;
	/** Agent index for filtering links */
	int32 AgentIndex;
	/** Resolution level */ 
	ENavigationDataResolution TileResolution;
	/** Average layers per tile */
	float AverageLayersPerTile;
	/** Ledge filtering mode */
	ENavigationLedgeSlopeFilterMode LedgeSlopeFilterMode;
	/** Is the config completely setup */
	bool bIsTileSetupConfigCompleted;
	/** Used when generating links automatically. Distance representing how far generated links can go outside a tile (in uu). */ 
	float LinkSpillDistance;
	
	FRecastBuildConfig()
	{
		Reset();
	}

	void Reset()
	{
		FMemory::Memzero(*this);
		bPerformVoxelFiltering = true;
		bGenerateDetailedMesh = true;
		bGenerateBVTree = true;
		bMarkLowHeightAreas = false;
		bUseExtraTopCellWhenMarkingAreas = true;
		bFilterLowSpanSequences = false;
		bFilterLowSpanFromTileCache = false;
		bGenerateLinks = false;
		bMinimizeLinkPoolSize = true;
		MaxPolysPerTile = -1;
		JumpConfigs.Empty();
		AgentIndex = 0;
		TileResolution = ENavigationDataResolution::Default;
		AverageLayersPerTile = 3;
		LedgeSlopeFilterMode = ENavigationLedgeSlopeFilterMode::Recast;
		bIsTileSetupConfigCompleted = false;
		LinkSpillDistance = 0.f;
	}

	rcReal GetTileSizeUU() const { return tileSize * cs; }

	/** Detects if we are using big cell size values in relation to the walkableClimb and walkableSlopeAngle. */
	bool IsUsingCoarseCellSize() const;
};

struct FRecastVoxelCache
{
	struct FTileInfo
	{
		int32 TileX;
		int32 TileY;
		int32 NumSpans;
		FTileInfo* NextTile;
		rcSpanCache* SpanData;
	};

	int32 NumTiles;

	/** tile info */
	FTileInfo* Tiles;

	FRecastVoxelCache() {}
	FRecastVoxelCache(const uint8* Memory);
};

struct FRecastGeometryCache
{
	struct FHeader
	{
		FNavigationRelevantData::FCollisionDataHeader Validation;
		
		int32 NumVerts;
		int32 NumFaces;
		struct FWalkableSlopeOverride SlopeOverride;

		static uint32 StaticMagicNumber;
	};

	FHeader Header;

	/** recast coords of vertices (size: NumVerts * 3) */
	FVector::FReal* Verts;

	/** vert indices for triangles (size: NumFaces * 3) */
	int32* Indices;

	FRecastGeometryCache() {}
	NAVIGATIONSYSTEM_API FRecastGeometryCache(const uint8* Memory);
};

struct FRecastRawGeometryElement
{
	// Instance geometry
	TArray<FVector::FReal>		GeomCoords;
	TArray<int32>				GeomIndices;

	// Bounds of the geometry
	FBox GeomCoordsBounds;
	
	// Per instance transformations in unreal coords
	// When empty, geometry is in world space
	TArray<FTransform>	PerInstanceTransform;

	rcRasterizationFlags RasterizationFlags;

	// Nav area for collision triangles. If set, replaces RC_WALKABLE_AREA during rasterization.
	TSubclassOf<UNavAreaBase> AreaClass;

	NAVIGATIONSYSTEM_API static rcRasterizationFlags GetRasterizationFlags(const FCompositeNavModifier& InModifier);

	// Extracts AreaClass from nav modifier (UNavCollision creates single area modifier).
	NAVIGATIONSYSTEM_API static TSubclassOf<UNavAreaBase> GetCollisionAreaClass(const FCompositeNavModifier& InModifier);
};

struct FRecastAreaNavModifierElement
{
	TArray<FAreaNavModifier> Areas;
	
	// Per instance transformations in unreal coords
	// When empty, areas are in world space
	TArray<FTransform>	PerInstanceTransform;

	ENavigationDataResolution NavMeshResolution = ENavigationDataResolution::Invalid;
	
	bool bMaskFillCollisionUnderneathForNavmesh = false;
};

struct FRcTileBox
{
	int32 XMin, XMax, YMin, YMax;

	FRcTileBox(const FBox& UnrealBounds, const FVector& RcNavMeshOrigin, const FVector::FReal TileSizeInWorldUnits)
	{
		check(TileSizeInWorldUnits > 0);
		checkf(!RcNavMeshOrigin.ContainsNaN(), TEXT("%hs: RcNavMeshOrigin ContainsNaN"), __FUNCTION__);
		checkf(!UnrealBounds.ContainsNaN(), TEXT("%hs: UnrealBounds ContainsNaN"), __FUNCTION__);
		
		if (!ensureMsgf(UnrealBounds.IsValid, TEXT("%hs: UnrealBounds !IsValid"), __FUNCTION__))
		{
			// This is a bug but we'll handle it as well as we can.

			// Invalid bounds, set to empty range.
			// Max is set to -1, because the range is inclusive, used like: for (int32 y = TileBox.YMin; y <= TileBox.YMax; ++y)
			XMin = 0;
			XMax = -1;
			YMin = 0;
			YMax = -1;
			return;
		}

		auto CalcMaxCoordExclusive = [](const FVector::FReal MaxAsFloat, const int32 MinCoord) -> int32
		{
			FVector::FReal UnusedIntPart;
			// If MaxCoord falls exactly on the boundary of a tile
			if (FMath::Modf(MaxAsFloat, &UnusedIntPart) == 0)
			{
				// Return the lower tile
				return FMath::Max(ClampToInt32(FMath::FloorToInt(MaxAsFloat) - 1), MinCoord);
			}
			// Otherwise use default behaviour
			return ClampToInt32(FMath::FloorToInt(MaxAsFloat));
		};

		const FBox RcAreaBounds = Unreal2RecastBox(UnrealBounds);
		XMin = ClampToInt32(FMath::FloorToInt((RcAreaBounds.Min.X - RcNavMeshOrigin.X) / TileSizeInWorldUnits));
		XMax = CalcMaxCoordExclusive((RcAreaBounds.Max.X - RcNavMeshOrigin.X) / TileSizeInWorldUnits, XMin);
		YMin = ClampToInt32(FMath::FloorToInt((RcAreaBounds.Min.Z - RcNavMeshOrigin.Z) / TileSizeInWorldUnits));
		YMax = CalcMaxCoordExclusive((RcAreaBounds.Max.Z - RcNavMeshOrigin.Z) / TileSizeInWorldUnits, YMin);
	}

	inline bool Contains(const FIntPoint& Point) const
	{
		return Point.X >= XMin && Point.X <= XMax
			&& Point.Y >= YMin && Point.Y <= YMax;
	}

	static inline int32 ClampToInt32(const int64 Value)
	{
#if !NO_LOGGING
		UE_CLOG(!IntFitsIn<int32>(Value), LogNavigation, Warning,
			TEXT("FRcTileBox clamped a NavMesh transform value to fit in int32. Old value: %" UINT64_FMT), Value);
#endif // !NO_LOGGING

		return static_cast<int32>(FMath::Clamp(Value, MIN_int32, MAX_int32));
	}
};

/**
 * TIME SLICING 
 * The general idea is that any function that handles time slicing internally will be named XXXXTimeSliced
 * and returns a ETimeSliceWorkResult. These functions also call TestTimeSliceFinished() internally when required,
 * IsTimeSliceFinishedCached() can be called externally after they have finished. Non time sliced functions are 
 * managed externally and the calling function should call TestTimeSliceFinished() when necessary.
 */

/** Return state of calling time sliced functions */
enum class ETimeSliceWorkResult : uint8
{
	Failed,
	Succeeded,
	CallAgainNextTimeSlice, /** time slice is finished this frame but we need to call this functionality again next frame */
};

/** State representing which area of GenerateCompressedLayersTimeSliced() we are processing */
enum class EGenerateCompressedLayersTimeSliced : uint8
{
	Invalid,
	Init,
	CreateHeightField,
	RasterizeTriangles,
	EmptyLayers,
	VoxelFilter,
	RecastFilter,
	CompactHeightField,
	ErodeWalkable,
	BuildLayers,
	BuildTileCache,
};

enum class ERasterizeGeomRecastTimeSlicedState : uint8 
{
	MarkWalkableTriangles,
	RasterizeTriangles,
};

enum class ERasterizeGeomTimeSlicedState : uint8
{
	RasterizeGeometryTransformCoordsAndFlipIndices,
	RasterizeGeometryTransformCoords UE_DEPRECATED(5.5, "The state doesn't handle indices flipping. Use RasterizeGeometryTransformCoordsAndIndices instead.") = RasterizeGeometryTransformCoordsAndFlipIndices,
	RasterizeGeometryRecast,
};

enum class EGenerateRecastFilterTimeSlicedState : uint8
{
	FilterLowHangingWalkableObstacles,
	FilterLedgeSpans,
	FilterWalkableLowHeightSpans,
};

enum class EDoWorkTimeSlicedState : uint8
{
	Invalid,
	GatherGeometryFromSources,
	GenerateTile,
};

enum class EGenerateTileTimeSlicedState : uint8
{
	Invalid,
	GenerateCompressedLayers,
	GenerateNavigationData,
};

enum class EGenerateNavDataTimeSlicedState : uint8
{
	Invalid,
	Init,
	GenerateLayers,
};

struct FRecastTileTimeSliceSettings
{
	int32 FilterLedgeSpansMaxYProcess = 13;
};

struct FGeneratedNavigationLink : public FNavigationLink
{
	/// User defined flags assigned to the polys of off-mesh connections
	unsigned short generatedLinkPolyFlag = 0;

	/// User defined area ids assigned to the off-mesh connections
	unsigned char generatedLinkArea = 0;
};
typedef TArray<FGeneratedNavigationLink, TInlineAllocator<64>> TInlineGeneratedNavLinkArray;


/**
 * Class handling generation of a single tile, caching data that can speed up subsequent tile generations
 */
class FRecastTileGenerator : public FNoncopyable, public FGCObject
{
	friend FRecastNavMeshGenerator;

public:
	NAVIGATIONSYSTEM_API FRecastTileGenerator(FRecastNavMeshGenerator& ParentGenerator, const FIntPoint& Location, const double PendingTileCreationTime);
	NAVIGATIONSYSTEM_API virtual ~FRecastTileGenerator();
		
	/** Does the work involved with regenerating this tile using time slicing.
	 *  The return value determines the result of the time slicing
	 */
	NAVIGATIONSYSTEM_API ETimeSliceWorkResult DoWorkTimeSliced();
	/** Does the work involved with regenerating this tile */
	NAVIGATIONSYSTEM_API bool DoWork();

	inline int32 GetTileX() const { return TileX; }
	inline int32 GetTileY() const { return TileY; }
	inline const FBox& GetTileBB() const { return TileBB; }
	/** Whether specified layer was updated */
	inline bool IsLayerChanged(int32 LayerIdx) const { return DirtyLayers[LayerIdx]; }
	inline const TBitArray<>& GetDirtyLayersMask() const { return DirtyLayers; }
	/** Whether tile data was fully regenerated */
	inline bool IsFullyRegenerated() const { return bRegenerateCompressedLayers; }
	/** Whether tile task has anything to build */
	NAVIGATIONSYSTEM_API bool HasDataToBuild() const;

	const TArray<FNavMeshTileData>& GetCompressedLayers() const { return CompressedLayers; }

	static NAVIGATIONSYSTEM_API FBox CalculateTileBounds(int32 X, int32 Y, const FVector& RcNavMeshOrigin, const FBox& TotalNavBounds, FVector::FReal TileSizeInWorldUnits);

protected:
	// to be used solely by FRecastNavMeshGenerator
	TArray<FNavMeshTileData>& GetNavigationData() { return NavigationData; }
	
public:
	NAVIGATIONSYSTEM_API uint32 GetUsedMemCount() const;

	// Memory amount used to construct generator 
	uint32 UsedMemoryOnStartup;

	// FGCObject begin
	NAVIGATIONSYSTEM_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	NAVIGATIONSYSTEM_API virtual FString GetReferencerName() const override;
	// FGCObject end

#if RECAST_INTERNAL_DEBUG_DATA
	const FRecastInternalDebugData& GetDebugData() const { return DebugData; }
	FRecastInternalDebugData& GetMutableDebugData() { return DebugData; }
#endif
		
	typedef TArray<int32, TInlineAllocator<4096>> TInlineMaskArray;

protected:
	/** Does the actual TimeSliced tile generation. 
	 *	@note always trigger tile generation only via DoWorkTimeSliced(). This is a worker function
	 *  The return value determines the result of the time slicing
	 *	@return Suceeded if new tile navigation data has been generated and is ready to be added to navmesh instance,
	 *	@return Failed if failed or no need to generate (still valid).
	 *  @return CallAgainNextTimeSlice, time slice is finished this frame but we need to call this function again next frame
	 */
	NAVIGATIONSYSTEM_API ETimeSliceWorkResult GenerateTileTimeSliced();
	/** Does the actual tile generation.
	 *	@note always trigger tile generation only via DoWorkTime(). This is a worker function
	 *  The return value determines the result of the time slicing
	 *	@return true if new tile navigation data has been generated and is ready to be added to navmesh instance,
	 *	@return false if failed or no need to generate (still valid).
	 */
	NAVIGATIONSYSTEM_API bool GenerateTile();

	NAVIGATIONSYSTEM_API void Setup(const FRecastNavMeshGenerator& ParentGenerator, const TArray<FBox>& DirtyAreas);

	/** Find highest navmesh resolution from Modifiers and use it to update the tile configuration. */
	void SetupTileConfigFromHighestResolution(const FRecastNavMeshGenerator& ParentGenerator);
	
	/** Gather geometry */
	NAVIGATIONSYSTEM_API virtual void GatherGeometry(const FRecastNavMeshGenerator& ParentGenerator, bool bGeometryChanged);
	/** Gather geometry sources to be processed later by the GatherGeometryFromSources */
	NAVIGATIONSYSTEM_API virtual void PrepareGeometrySources(const FRecastNavMeshGenerator& ParentGenerator, bool bGeometryChanged);
	/** Gather geometry from the prefetched sources */
	NAVIGATIONSYSTEM_API void GatherGeometryFromSources();
	/** Gather geometry from the prefetched sources time sliced version */
	NAVIGATIONSYSTEM_API ETimeSliceWorkResult GatherGeometryFromSourcesTimeSliced();
	/** Gather geometry from a specified Navigation Data */
	NAVIGATIONSYSTEM_API void GatherNavigationDataGeometry(const TSharedRef<FNavigationRelevantData, ESPMode::ThreadSafe>& ElementData, UNavigationSystemV1& NavSys, const FNavDataConfig& OwnerNavDataConfig, bool bGeometryChanged);

	/** Start functions used by GenerateCompressedLayersTimeSliced / GenerateCompressedLayers */
	NAVIGATIONSYSTEM_API bool CreateHeightField(FNavMeshBuildContext& BuildContext);
	NAVIGATIONSYSTEM_API ETimeSliceWorkResult RasterizeTrianglesTimeSliced(FNavMeshBuildContext& BuildContext, FTileRasterizationContext& RasterContext);
	NAVIGATIONSYSTEM_API void RasterizeTriangles(FNavMeshBuildContext& BuildContext, FTileRasterizationContext& RasterContext);
	NAVIGATIONSYSTEM_API ETimeSliceWorkResult RasterizeGeometryRecastTimeSliced(FNavMeshBuildContext& BuildContext, const TArray<FVector::FReal>& Coords, const TArray<int32>& Indices, const FBox& CoordsBounds, const rcRasterizationFlags RasterizationFlags, FTileRasterizationContext& RasterContext, TSubclassOf<UNavAreaBase> AreaClass = nullptr);
	NAVIGATIONSYSTEM_API void RasterizeGeometryRecast(FNavMeshBuildContext& BuildContext, const TArray<FVector::FReal>& Coords, const TArray<int32>& Indices, const FBox& CoordsBounds, const rcRasterizationFlags RasterizationFlags, FTileRasterizationContext& RasterContext, TSubclassOf<UNavAreaBase> AreaClass = nullptr);
	NAVIGATIONSYSTEM_API void RasterizeGeometryTransformCoordsAndFlipIndices(const TArray<FVector::FReal>& Coords, const TArray<int32>& Indices, const FBox& CoordsBounds, const FTransform& LocalToWorld);
	NAVIGATIONSYSTEM_API ETimeSliceWorkResult RasterizeGeometryTimeSliced(FNavMeshBuildContext& BuildContext, const TArray<FVector::FReal>& Coords, const TArray<int32>& Indices, const FBox& CoordsBounds, const FTransform& LocalToWorld, const rcRasterizationFlags RasterizationFlags, FTileRasterizationContext& RasterContext, TSubclassOf<UNavAreaBase> AreaClass = nullptr);
	NAVIGATIONSYSTEM_API void RasterizeGeometry(FNavMeshBuildContext& BuildContext, const TArray<FVector::FReal>& Coords, const TArray<int32>& Indices, const FBox& CoordsBounds, const FTransform& LocalToWorld, const rcRasterizationFlags RasterizationFlags, FTileRasterizationContext& RasterContext, TSubclassOf<UNavAreaBase> AreaClass = nullptr);
	NAVIGATIONSYSTEM_API void GenerateRecastFilter(FNavMeshBuildContext& BuildContext);
	NAVIGATIONSYSTEM_API ETimeSliceWorkResult GenerateRecastFilterTimeSliced(FNavMeshBuildContext& BuildContext);
	NAVIGATIONSYSTEM_API bool BuildCompactHeightField(FNavMeshBuildContext& BuildContext);
	NAVIGATIONSYSTEM_API bool RecastErodeWalkable(FNavMeshBuildContext& BuildContext);
	NAVIGATIONSYSTEM_API bool RecastBuildLayers(FNavMeshBuildContext& BuildContext, FTileRasterizationContext& RasterContext);
	NAVIGATIONSYSTEM_API bool RecastBuildTileCache(FNavMeshBuildContext& BuildContext, FTileRasterizationContext& RasterContext);
	/** End functions used by GenerateCompressedLayersTimeSliced / GenerateCompressedLayers */

	/** Builds CompressedLayers array (geometry + modifiers) time sliced*/
	NAVIGATIONSYSTEM_API virtual ETimeSliceWorkResult GenerateCompressedLayersTimeSliced(FNavMeshBuildContext& BuildContext);

	/** Builds CompressedLayers array (geometry + modifiers) */
	NAVIGATIONSYSTEM_API virtual bool GenerateCompressedLayers(FNavMeshBuildContext& BuildContext, const dtLinkBuilderData& InLinkBuilderData);

	/** Builds a navigation data layer */
	NAVIGATIONSYSTEM_API bool GenerateNavigationDataLayer(FNavMeshBuildContext& BuildContext, FTileCacheCompressor& TileCompressor, FTileCacheAllocator& GenNavAllocator, FTileGenerationContext& GenerationContext, const dtLinkBuilderData& InLinkBuilderData, int32 LayerIdx);

	/** Builds NavigationData array (layers + obstacles) time sliced */
	NAVIGATIONSYSTEM_API ETimeSliceWorkResult GenerateNavigationDataTimeSliced(FNavMeshBuildContext& BuildContext, const dtLinkBuilderData& InLinkBuilderData);

	/** Builds NavigationData array (layers + obstacles) */
	NAVIGATIONSYSTEM_API bool GenerateNavigationData(FNavMeshBuildContext& BuildContext, const dtLinkBuilderData& InLinkBuilderData);

	/** Builds navigation links */
	dtStatus BuildTileCacheLinks(FNavMeshBuildContext& BuildContext, struct dtTileCacheAlloc* alloc, const dtTileCacheLayer& layer,
		const struct dtTileCacheContourSet& lcset, const float DefaultSnapHeight, TInlineGeneratedNavLinkArray& OutGeneratedLinks) const;
	
	NAVIGATIONSYSTEM_API virtual void ApplyVoxelFilter(struct rcHeightfield* SolidHF, FVector::FReal WalkableRadius);

	/** Compute rasterization mask */
	NAVIGATIONSYSTEM_API void InitRasterizationMaskArray(const rcHeightfield* InSolidHF, TInlineMaskArray& OutRasterizationMasks);
	NAVIGATIONSYSTEM_API void ComputeRasterizationMasks(FNavMeshBuildContext& BuildContext, FTileRasterizationContext& RasterContext);
	NAVIGATIONSYSTEM_API void MarkRasterizationMask(rcContext* /*BuildContext*/, rcHeightfield* InSolidHF,
		const FAreaNavModifier& Modifier, const FTransform& LocalToWorld, const int32 Mask, TInlineMaskArray& OutMaskArray);

	/** apply areas from DynamicAreas to layer */
	NAVIGATIONSYSTEM_API void MarkDynamicAreas(FNavMeshBuildContext& BuildContext, dtTileCacheLayer& Layer);
	NAVIGATIONSYSTEM_API void MarkDynamicArea(FNavMeshBuildContext& BuildContext, const FAreaNavModifier& Modifier, const FTransform& LocalToWorld, dtTileCacheLayer& Layer);
	NAVIGATIONSYSTEM_API void MarkDynamicArea(FNavMeshBuildContext& BuildContext, const FAreaNavModifier& Modifier, const FTransform& LocalToWorld, dtTileCacheLayer& Layer, const int32 AreaId, const int32* ReplaceAreaIdPtr);

	NAVIGATIONSYSTEM_API void AppendModifier(const FCompositeNavModifier& Modifier, const FNavDataPerInstanceTransformDelegate& InTransformsDelegate);
	/** Appends specified geometry to tile's geometry */
	NAVIGATIONSYSTEM_API void ValidateAndAppendGeometry(const FNavigationRelevantData& ElementData, const FCompositeNavModifier& InModifier);

	NAVIGATIONSYSTEM_API void AppendGeometry(const FNavigationRelevantData& ElementData, const FCompositeNavModifier& InModifier, const FNavDataPerInstanceTransformDelegate& InTransformsDelegate);
	
	/** prepare voxel cache from collision data */
	NAVIGATIONSYSTEM_API void PrepareVoxelCache(const TNavStatArray<uint8>& RawCollisionCache, const FBox& RecastBounds, const FCompositeNavModifier& InModifier, TNavStatArray<rcSpanCache>& SpanData);
	NAVIGATIONSYSTEM_API bool HasVoxelCache(const TNavStatArray<uint8>& RawVoxelCache, rcSpanCache*& CachedVoxels, int32& NumCachedVoxels) const;
	NAVIGATIONSYSTEM_API void AddVoxelCache(TNavStatArray<uint8>& RawVoxelCache, const rcSpanCache* CachedVoxels, const int32 NumCachedVoxels) const;

	NAVIGATIONSYSTEM_API void DumpAsyncData();
	NAVIGATIONSYSTEM_API void DumpSyncData();

#if RECAST_INTERNAL_DEBUG_DATA
	NAVIGATIONSYSTEM_API bool IsTileDebugActive() const;
	NAVIGATIONSYSTEM_API bool IsTileDebugAllowingGeneration() const;
#endif

	/** Deprecated functions */
	UE_DEPRECATED(5.8, "Please use the overload with the added BuildContext.")
	NAVIGATIONSYSTEM_API void MarkDynamicAreas(dtTileCacheLayer& Layer);
	UE_DEPRECATED(5.8, "Please use the overload with the added BuildContext.")
	NAVIGATIONSYSTEM_API void MarkDynamicArea(const FAreaNavModifier& Modifier, const FTransform& LocalToWorld, dtTileCacheLayer& Layer);
	UE_DEPRECATED(5.8, "Please use the overload with the added BuildContext.")
	NAVIGATIONSYSTEM_API void MarkDynamicArea(const FAreaNavModifier& Modifier, const FTransform& LocalToWorld, dtTileCacheLayer& Layer, const int32 AreaId, const int32* ReplaceAreaIdPtr);


protected:
	uint32 bRegenerateCompressedLayers : 1;
	uint32 bFullyEncapsulatedByInclusionBounds : 1;
	uint32 bUpdateGeometry : 1;
	uint32 bHasLowAreaModifiers : 1;
	uint32 bHasGeometryToRasterizeAsFilledConvexVolume : 1; 

	/** Start time slicing variables */
	ERasterizeGeomRecastTimeSlicedState RasterizeGeomRecastState;
	ERasterizeGeomTimeSlicedState RasterizeGeomState;
	EGenerateRecastFilterTimeSlicedState GenerateRecastFilterState;
	int32 GenRecastFilterLedgeSpansYStart;
	EDoWorkTimeSlicedState DoWorkTimeSlicedState;
	EGenerateTileTimeSlicedState GenerateTileTimeSlicedState;

	EGenerateNavDataTimeSlicedState GenerateNavDataTimeSlicedState;
	int32 GenNavDataLayerTimeSlicedIdx;
	EGenerateCompressedLayersTimeSliced GenCompressedLayersTimeSlicedState;
	int32 RasterizeTrianglesTimeSlicedRawGeomIdx;
	int32 RasterizeTrianglesTimeSlicedInstTransformIdx;
	TNavStatArray<uint8> RasterizeGeomRecastTriAreas;
	const FNavRegenTimeSliceManager* TimeSliceManager;

	TUniquePtr<struct FTileCacheAllocator> GenNavDataTimeSlicedAllocator;
	TUniquePtr<struct FTileGenerationContext> GenNavDataTimeSlicedGenerationContext;
	TUniquePtr<struct FTileRasterizationContext> GenCompressedlayersTimeSlicedRasterContext;

	FRecastTileTimeSliceSettings TileTimeSliceSettings;
	/** End time slicing variables */

	int32 TileX;
	int32 TileY;
	uint32 Version;

	/** Time when the tile was requested */
	double TileCreationTime = 0.;
	
	/** Tile's bounding box, Unreal coords */
	FBox TileBB;

	/** Tile's bounding box expanded by Agent's radius horizontally, Unreal coords */
	FBox TileBBExpandedForAgent;
	
	/** Layers dirty flags */
	TBitArray<> DirtyLayers;
	
	/** Parameters defining navmesh tiles */
	FRecastBuildConfig TileConfig;

	/** Bounding geometry definition. */
	TNavStatArray<FBox> InclusionBounds;

	/** Additional config */
	FRecastNavMeshCachedData AdditionalCachedData;

	// generated tile data
	TArray<FNavMeshTileData> CompressedLayers;
	TArray<FNavMeshTileData> NavigationData;

	/** Result of calling RasterizeGeometry() */
	TArray<FVector::FReal> RasterizeGeometryWorldRecastCoords;
	TArray<int32> RasterizeGeometryFlippedIndices;
	FBox RasterizeGeometryWorldRecastCoordsBounds;
	uint8 bRasterizeGeometryUseFlippedIndices : 1;

	// tile's geometry: without voxel cache
	TArray<FRecastRawGeometryElement> RawGeometry;
	// areas used for creating navigation data: obstacles
	TArray<FRecastAreaNavModifierElement> Modifiers;
	// navigation links
	TArray<FSimpleLinkNavModifier> OffmeshLinks;

	TWeakPtr<FNavDataGenerator, ESPMode::ThreadSafe> ParentGeneratorWeakPtr;

	TNavStatArray<TSharedRef<FNavigationRelevantData, ESPMode::ThreadSafe> > NavigationRelevantData;
	TWeakObjectPtr<UNavigationSystemV1> NavSystem; 
	FNavDataConfig NavDataConfig;

	rcHeightfield* SolidHF;
	rcCompactHeightfield* CompactHF;

	FRecastNavMeshTileGenerationDebug TileDebugSettings;

#if RECAST_INTERNAL_DEBUG_DATA
	FRecastInternalDebugData DebugData;
#endif
};

struct FRecastTileGeneratorWrapper : public FNonAbandonableTask
{
	TSharedRef<FRecastTileGenerator> TileGenerator;

	FRecastTileGeneratorWrapper(TSharedRef<FRecastTileGenerator> InTileGenerator)
		: TileGenerator(InTileGenerator)
	{
	}
	
	void DoWork()
	{
		TileGenerator->DoWork();
	}

	inline TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FRecastTileGenerator, STATGROUP_ThreadPoolAsyncTasks);
	}
};

typedef FAsyncTask<FRecastTileGeneratorWrapper> FRecastTileGeneratorTask;
//typedef FAsyncTask<FRecastTileGenerator> FRecastTileGeneratorTask;

struct FPendingTileElement
{
	/** tile coordinates on a grid in recast space */
	FIntPoint	Coord;
	/** distance to seed, used for sorting pending tiles */
	FVector::FReal SeedDistance;
	/** time at which the element was first added to the queue */
	double		CreationTime;

#if !UE_BUILD_SHIPPING	
	/** Squared distance to invoker source */
	FVector::FReal DebugInvokerDistanceSquared = TNumericLimits<FVector::FReal>::Max();

	/** Priority from navigation invoker */
	ENavigationInvokerPriority DebugInvokerPriority = ENavigationInvokerPriority::Default;
#endif // !UE_BUILD_SHIPPING

	/** Priority used for sorting */
	ENavigationInvokerPriority SortingPriority = ENavigationInvokerPriority::Default;

	/** Whether we need a full rebuild for this tile grid cell */
	bool		bRebuildGeometry;
	/** We need to store dirty area bounds to check which cached layers needs to be regenerated
	 *  In case geometry is changed, cached layers data will be fully regenerated without using dirty areas list
	 */
	TArray<FBox> DirtyAreas;

	FPendingTileElement()
		: Coord(FIntPoint::NoneValue)
		, SeedDistance(TNumericLimits<FVector::FReal>::Max())
		, CreationTime(-1.)
		, bRebuildGeometry(false)
	{
	}

	bool operator == (const FIntPoint& Location) const
	{
		return Coord == Location;
	}

	bool operator == (const FPendingTileElement& Other) const
	{
		return Coord == Other.Coord;
	}

	bool operator < (const FPendingTileElement& Other) const
	{
		return Other.SeedDistance < SeedDistance;
	}

	double GetDurationSinceCreation(const double CurrenTimeSeconds) const
	{
		return (CreationTime > 0.) ? CurrenTimeSeconds - CreationTime : 0.;
	}

	friend uint32 GetTypeHash(const FPendingTileElement& Element)
	{
		return GetTypeHash(Element.Coord);
	}
};

template<typename TTileGeneratorTask>
struct TRunningTileElement
{
	TRunningTileElement()
		: Coord(FIntPoint::NoneValue)
		, bShouldDiscard(false)
		, AsyncTask(nullptr)
	{
	}
	
	TRunningTileElement(FIntPoint InCoord)
		: Coord(InCoord)
		, bShouldDiscard(false)
		, AsyncTask(nullptr)
	{
	}

	bool operator == (const TRunningTileElement& Other) const
	{
		return Coord == Other.Coord;
	}
	
	/** tile coordinates on a grid in recast space */
	FIntPoint					Coord;
	/** whether generated results should be discarded */
	bool						bShouldDiscard; 
	FAsyncTask<TTileGeneratorTask>* AsyncTask;
};

UE_DEPRECATED(5.5, "FRunningTileElement is deprecated. Please use TRunningTileElement<FRecastTileGeneratorWrapper> instead.")
typedef TRunningTileElement<FRecastTileGeneratorWrapper> FRunningTileElement;

struct FTileTimestamp
{
	FNavTileRef NavTileRef;
	double Timestamp;
	
	bool operator == (const FTileTimestamp& Other) const
	{
		return NavTileRef == Other.NavTileRef;
	}
};

enum class EProcessTileTasksSyncTimeSlicedState : uint8
{
	Init,
	DoWork,
	AddGeneratedTiles,
	StoreCompessedTileCacheLayers,
	AppendUpdateTiles,
	Finish,
};

enum class EAddGeneratedTilesTimeSlicedState : uint8
{
	Init,
	AddTiles,
};

/**
 * Class that handles generation of the whole Recast-based navmesh.
 */
class FRecastNavMeshGenerator : public FNavDataGenerator
{
public:
	NAVIGATIONSYSTEM_API FRecastNavMeshGenerator(ARecastNavMesh& InDestNavMesh);
	NAVIGATIONSYSTEM_API virtual ~FRecastNavMeshGenerator();

private:
	/** Prevent copying. */
	FRecastNavMeshGenerator(FRecastNavMeshGenerator const& NoCopy) { check(0); };
	FRecastNavMeshGenerator& operator=(FRecastNavMeshGenerator const& NoCopy) { check(0); return *this; }

public:
	NAVIGATIONSYSTEM_API virtual bool RebuildAll() override;
	NAVIGATIONSYSTEM_API virtual void EnsureBuildCompletion() override;
	NAVIGATIONSYSTEM_API virtual void CancelBuild() override;
	NAVIGATIONSYSTEM_API virtual void TickAsyncBuild(float DeltaSeconds) override;
	NAVIGATIONSYSTEM_API virtual void OnNavigationBoundsChanged() override;

	/** Asks generator to update navigation affected by DirtyAreas */
	NAVIGATIONSYSTEM_API virtual void RebuildDirtyAreas(const TArray<FNavigationDirtyArea>& DirtyAreas) override;

	/** determines whether this generator is performing navigation building actions at the moment, dirty areas are also checked */
	NAVIGATIONSYSTEM_API virtual bool IsBuildInProgressCheckDirty() const override;

#if !RECAST_ASYNC_REBUILDING
	/** returns true if we are time slicing and the data is valid to use false otherwise*/
	NAVIGATIONSYSTEM_API virtual bool GetTimeSliceData(int32& OutNumRemainingBuildTasks, double& OutCurrentBuildTaskDuration) const override;
#endif

	int32 GetNumRemaningBuildTasksHelper() const { return RunningDirtyTiles.Num() + PendingDirtyTiles.Num() + static_cast<int32>(SyncTimeSlicedData.TileGeneratorSync.IsValid()); }
	NAVIGATIONSYSTEM_API virtual int32 GetNumRemaningBuildTasks() const override;
	NAVIGATIONSYSTEM_API virtual int32 GetNumRunningBuildTasks() const override;

	/** Checks if a given tile is being build or has just finished building */
	NAVIGATIONSYSTEM_API bool IsTileChanged(const FNavTileRef InTileRef) const;
		
	inline uint32 GetVersion() const { return Version; }

	const ARecastNavMesh* GetOwner() const { return DestNavMesh; }

	/** update area data */
	NAVIGATIONSYSTEM_API void OnAreaAdded(const UClass* AreaClass, int32 AreaID);
	NAVIGATIONSYSTEM_API void OnAreaRemoved(const UClass* AreaClass);
		
	//--- accessors --- //
	inline class UWorld* GetWorld() const { return DestNavMesh->GetWorld(); }

	const FRecastBuildConfig& GetConfig() const { return Config; }

	const FRecastNavMeshTileGenerationDebug& GetTileDebugSettings() const { return DestNavMesh->TileGenerationDebug; } 

	const TNavStatArray<FBox>& GetInclusionBounds() const { return InclusionBounds; }
	
	FVector GetRcNavMeshOrigin() const { return RcNavMeshOrigin; }

	/** checks if any on InclusionBounds encapsulates given box.
	 *	@return index to first item in InclusionBounds that meets expectations */
	NAVIGATIONSYSTEM_API int32 FindInclusionBoundEncapsulatingBox(const FBox& Box) const;

	/** Total navigable area box, sum of all navigation volumes bounding boxes */
	FBox GetTotalBounds() const { return TotalNavBounds; }

	const FRecastNavMeshCachedData& GetAdditionalCachedData() const { return AdditionalCachedData; }

	NAVIGATIONSYSTEM_API bool HasDirtyTiles() const;
	NAVIGATIONSYSTEM_API bool HasDirtyTiles(const FBox& AreaBounds) const;
	NAVIGATIONSYSTEM_API int32 GetDirtyTilesCount(const FBox& AreaBounds) const;

	NAVIGATIONSYSTEM_API bool GatherGeometryOnGameThread() const;
	NAVIGATIONSYSTEM_API bool IsTimeSliceRegenActive() const;

	NAVIGATIONSYSTEM_API FBox GrowBoundingBox(const FBox& BBox, bool bIncludeAgentHeight) const;
	
	/** Returns if the provided Octree Element should generate geometry on the provided NavDataConfig. Can be used to extend the logic to decide what geometry is generated on what Navmesh */
	NAVIGATIONSYSTEM_API virtual bool ShouldGenerateGeometryForOctreeElement(const FNavigationOctreeElement& Element, const FNavDataConfig& NavDataConfig) const;
	
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && ENABLE_VISUAL_LOG
	NAVIGATIONSYSTEM_API virtual void ExportNavigationData(const FString& FileName) const override;
	NAVIGATIONSYSTEM_API virtual void GrabDebugSnapshot(struct FVisualLogEntry* Snapshot, const FBox& BoundingBox, const FName& CategoryName, ELogVerbosity::Type Verbosity) const override;
#endif

#if UE_ENABLE_DEBUG_DRAWING
	/** Converts data encoded in EncodedData.CollisionData to FNavDebugMeshData format */
	static NAVIGATIONSYSTEM_API void GetDebugGeometry(const FNavigationRelevantData& EncodedData, FNavDebugMeshData& DebugMeshData);
#endif  // UE_ENABLE_DEBUG_DRAWING

	const FNavRegenTimeSliceManager* GetTimeSliceManager() const { return SyncTimeSlicedData.TimeSliceManager; }
	
	void SetNextTimeSliceRegenActive(bool bRegenState) { SyncTimeSlicedData.bNextTimeSliceRegenActive = bRegenState; }

	/** Update the config according to the resolution */
	NAVIGATIONSYSTEM_API virtual void SetupTileConfig(const ENavigationDataResolution TileResolution, FRecastBuildConfig& OutConfig) const;

protected:
	// Performs initial setup of member variables so that generator is ready to
	// do its thing from this point on. Called just after construction by ARecastNavMesh
	NAVIGATIONSYSTEM_API virtual void Init();

	// Used to configure Config. Override to influence build properties
	NAVIGATIONSYSTEM_API virtual void ConfigureBuildProperties(FRecastBuildConfig& OutConfig);

	// Updates cached list of navigation bounds
	NAVIGATIONSYSTEM_API void UpdateNavigationBounds();
		
	// Sorts pending build tiles by proximity to player, so tiles closer to player will get generated first
	NAVIGATIONSYSTEM_API virtual void SortPendingBuildTiles();

	// Get seed locations used for sorting pending build tiles. Tiles closer to these locations will be prioritized first.
	NAVIGATIONSYSTEM_API virtual void GetSeedLocations(UWorld& World, TArray<FVector2D>& OutSeedLocations) const;

	/** Instantiates dtNavMesh and configures it for tiles generation. Returns false if failed */
	NAVIGATIONSYSTEM_API bool ConstructTiledNavMesh();

	/** Determine bit masks for poly address */
	NAVIGATIONSYSTEM_API void CalcNavMeshProperties(int32& MaxTiles, int32& MaxPolys);
	
	/** Marks grid tiles affected by specified areas as dirty */
	NAVIGATIONSYSTEM_API virtual void MarkDirtyTiles(const TArray<FNavigationDirtyArea>& DirtyAreas);

	UE_DEPRECATED(5.5, "Use ShouldDirtyTilesRequestedByElement with FNavigationElement instead.")
	NAVIGATIONSYSTEM_API virtual bool ShouldDirtyTilesRequestedByObject(
		const UNavigationSystemV1& NavSys,
		const FNavigationOctree& NavOctreeInstance,
		const UObject& SourceObject,
		const FNavDataConfig& NavDataConfig) const final;

	/** Returns if the provided FNavigationElement that requested a navmesh dirtying should dirty this Navmesh. Useful to avoid tiles regeneration from elements that are excluded from the provided NavDataConfig */
	NAVIGATIONSYSTEM_API virtual bool ShouldDirtyTilesRequestedByElement(
		const UNavigationSystemV1& NavSys,
		const FNavigationOctree& NavOctreeInstance,
		FNavigationElementHandle SourceElement,
		const FNavDataConfig& NavDataConfig) const;

	/** Marks all tiles overlapping with InclusionBounds dirty (via MarkDirtyTiles). */
	NAVIGATIONSYSTEM_API bool MarkNavBoundsDirty();

	/** Returns the number of tiles currently populated with navigation data. O(1) — reads the maintained counter directly. */
	int32 GetNumActiveTiles() const { return NumActiveTiles; }

	NAVIGATIONSYSTEM_API void RemoveLayers(const FIntPoint& Tile, TArray<FNavTileRef>& UpdatedTiles);
	NAVIGATIONSYSTEM_API void RemoveAllTiles(TArray<FNavTileRef>& UpdatedTiles);
	
	NAVIGATIONSYSTEM_API void StoreCompressedTileCacheLayers(const FRecastTileGenerator& TileGenerator, int32 TileX, int32 TileY);

#if RECAST_INTERNAL_DEBUG_DATA
	NAVIGATIONSYSTEM_API void StoreDebugData(const FRecastTileGenerator& TileGenerator, int32 TileX, int32 TileY);
#endif

#if RECAST_ASYNC_REBUILDING
	/** Processes pending tile generation tasks Async*/
	NAVIGATIONSYSTEM_API TArray<FNavTileRef> ProcessTileTasksAsyncAndGetUpdatedTiles(const int32 NumTasksToProcess);
#else
	NAVIGATIONSYSTEM_API TSharedRef<FRecastTileGenerator> CreateTileGeneratorFromPendingElement(FIntPoint &OutTileLocation, const int32 ForcedPendingTileIdx = INDEX_NONE);

	/** Processes pending tile generation tasks Sync with option for time slicing currently an experimental feature. */
	NAVIGATIONSYSTEM_API virtual TArray<FNavTileRef> ProcessTileTasksSyncTimeSlicedAndGetUpdatedTiles();
	NAVIGATIONSYSTEM_API TArray<FNavTileRef> ProcessTileTasksSyncAndGetUpdatedTiles(const int32 NumTasksToProcess);

	NAVIGATIONSYSTEM_API virtual int32 GetNextPendingDirtyTileToBuild() const;
#endif
	/** Processes pending tile generation tasks */
	NAVIGATIONSYSTEM_API TArray<FNavTileRef> ProcessTileTasksAndGetUpdatedTiles(const int32 NumTasksToProcess);

	NAVIGATIONSYSTEM_API void ResetTimeSlicedTileGeneratorSync();

public:
	/** Adds generated tiles to NavMesh, replacing old ones, uses time slicing returns Failed if any layer failed */
	NAVIGATIONSYSTEM_API ETimeSliceWorkResult AddGeneratedTilesTimeSliced(FRecastTileGenerator& TileGenerator, TArray<FNavTileRef>& OutResultTileRefs);

	/** Adds generated tiles to NavMesh, replacing old ones */
	NAVIGATIONSYSTEM_API TArray<FNavTileRef> AddGeneratedTilesAndGetUpdatedTiles(FRecastTileGenerator& TileGenerator);

public:
	/** Removes all tiles at specified grid location and returns the updated FNavTileRef */
	NAVIGATIONSYSTEM_API TArray<FNavTileRef> RemoveTileLayersAndGetUpdatedTiles(const int32 TileX, const int32 TileY, TMap<int32, dtPolyRef>* OldLayerTileIdMap = nullptr);

	/** Removes all tiles at specified grid location */
	NAVIGATIONSYSTEM_API void RemoveTileLayers(dtNavMesh* DetourMesh, const int32 TileX, const int32 TileY);

	NAVIGATIONSYSTEM_API void RemoveTiles(const TArray<FIntPoint>& Tiles);

	NAVIGATIONSYSTEM_API void ReAddTiles(const TArray<FNavMeshDirtyTileElement>& Tiles);

	bool IsBuildingRestrictedToActiveTiles() const { return bRestrictBuildingToActiveTiles; }
	NAVIGATIONSYSTEM_API bool IsInActiveSet(const FIntPoint& Tile) const;

	/** sets a limit to number of asynchronous tile generators running at one time
	 *	@note if used at runtime will not result in killing tasks above limit count
	 *	@mote function does not validate the input parameter - it's on caller */
	void SetMaxTileGeneratorTasks(int32 NewLimit) { MaxTileGeneratorTasks = NewLimit; }

	void SetSortPendingTileMethod(const ENavigationSortPendingTilesMethod InMethod) { SortPendingTilesMethod = InMethod; }

	static NAVIGATIONSYSTEM_API void CalcPolyRefBits(ARecastNavMesh* NavMeshOwner, int32& MaxTileBits, int32& MaxPolyBits);

	/** Returns true if bGenerateNavLinks is enabled and bAllowLinkGeneration is true. */ 
	bool IsGeneratingLinks() const;

protected:
	/** Resolve area class from link generation config into recast areaIds and flags.
	 * Must be called after AdditionalCachedData is constructed. */
	void ResolveGeneratedLinkAreas(FRecastBuildConfig& OutConfig);
	
	NAVIGATIONSYSTEM_API virtual void RestrictBuildingToActiveTiles(bool InRestrictBuildingToActiveTiles);
	
	/** Blocks until build for specified list of tiles is complete and discard results */
	NAVIGATIONSYSTEM_API void DiscardCurrentBuildingTasks();

	NAVIGATIONSYSTEM_API virtual TSharedRef<FRecastTileGenerator> CreateTileGenerator(const FIntPoint& Coord, const TArray<FBox>& DirtyAreas, const double PendingTileCreationTime = 0.);

	template <typename T>
	TSharedRef<T> ConstructTileGeneratorImpl(const FIntPoint& Coord, const TArray<FBox>& DirtyAreas, const double PendingTileCreationTime)
	{
		TSharedRef<T> TileGenerator = MakeShareable(new T(*this, Coord, PendingTileCreationTime));
		TileGenerator->Setup(*this, DirtyAreas);
		return TileGenerator;
	}

	FBox GrowDirtyBounds(const FBox& BBox, bool bIncludeAgentHeight) const;

	//----------------------------------------------------------------------//
	// debug
	//----------------------------------------------------------------------//
	NAVIGATIONSYSTEM_API virtual uint32 LogMemUsed() const override;

	NAVIGATIONSYSTEM_API bool IsAllowedToAddTileLayers(const FIntPoint Tile) const;
	NAVIGATIONSYSTEM_API void AddGeneratedTileLayer(int32 LayerIndex, FRecastTileGenerator& TileGenerator, const TMap<int32, dtPolyRef>& OldLayerTileIdMap, TArray<FNavTileRef>& OutResultTileRefs);

#if !UE_BUILD_SHIPPING
	/** Data struct used by 'LogDirtyAreas' that contains all the information regarding the areas that are being dirtied, per dirtied tile. */
	struct FNavigationDirtyAreaPerTileDebugInformation
	{
		FNavigationDirtyArea DirtyArea;
		bool bTileWasAlreadyAdded = false;
	};
	
	/** Used internally, when LogNavigationDirtyArea is VeryVerbose, to log the number of tiles a dirty area is requesting. */
	NAVIGATIONSYSTEM_API void LogDirtyAreas(const UObject& OwnerNav,
		const TMap<FPendingTileElement, TArray<FNavigationDirtyAreaPerTileDebugInformation>>& DirtyAreasDebuggingInformation) const;

#endif
	
protected:
	friend ARecastNavMesh;

	/** Parameters defining navmesh tiles */
	FRecastBuildConfig Config;

	/** Used to grow generic element bounds to match this generator's properties
	 *	(most notably Config.borderSize) */
	UE_DEPRECATED(5.5, "Use BBoxGrowthLow and BBoxGrowthHigh instead.")
	FVector BBoxGrowth;
	
	/** Growth in the negative axis direction. 
	 * Used to grow generic element bounds to match this generator's properties (most notably Config.borderSize) */
	FVector BBoxGrowthLow;

	/** Growth in the positive axis direction. 
	 * Used to grow generic element bounds to match this generator's properties (most notably Config.borderSize) */
	FVector BBoxGrowthHigh;
	
	int32 NumActiveTiles;
	/** the limit to number of asynchronous tile generators running at one time,
	 *  this is also used by the sync non time sliced regeneration ProcessTileTasksSync,
	 *  but not by ProcessTileTasksSyncTimeSliced.
	 */
	int32 MaxTileGeneratorTasks;

	/** Total bounding box that includes all volumes, in unreal units. */
	FBox TotalNavBounds;

	/** Bounding geometry definition. */
	TNavStatArray<FBox> InclusionBounds;

	/** Navigation mesh that owns this generator */
	ARecastNavMesh*	DestNavMesh;
	
	/** List of dirty tiles that needs to be regenerated */
	TNavStatArray<FPendingTileElement> PendingDirtyTiles;
	
	/** List of dirty tiles currently being regenerated */
	TNavStatArray<TRunningTileElement<FRecastTileGeneratorWrapper>> RunningDirtyTiles;

#if WITH_NAVMESH_SEGMENT_LINKS
	/** List of tiles that have been generated but have not completed segment link connections yet */
	TSet<FNavTileRef> TilesPendingSegmentLinkConnections;
#endif // WITH_NAVMESH_SEGMENT_LINKS

#if WITH_EDITOR
	/** List of tiles that were recently regenerated */
	TNavStatArray<FTileTimestamp> RecentlyBuiltTiles;
#endif// WITH_EDITOR
	
	TSet<FIntPoint> ActiveTileSet;

	/** */
	FRecastNavMeshCachedData AdditionalCachedData;

	/** Use this if you don't want your tiles to start at (0,0,0) */
	FVector RcNavMeshOrigin;

	double RebuildAllStartTime = 0;
	
	uint32 bInitialized:1;

	uint32 bRestrictBuildingToActiveTiles:1;

	/** Runtime generator's version, increased every time all tile generators get invalidated
	 *	like when navmesh size changes */
	uint32 Version;

	ENavigationSortPendingTilesMethod SortPendingTilesMethod = ENavigationSortPendingTilesMethod::SortWithSeedLocations;

	/** Grouping all the member variables used by the time slicing code together for neatness */
	struct FSyncTimeSlicedData
	{
		FSyncTimeSlicedData();

		/** Accumulated time spent processing the tile (in seconds). */
		double CurrentTileRegenDuration;

#if !UE_BUILD_SHIPPING		
		/** Frame when the tile start being processed. */
		int64 TileRegenStartFrame = 0;

		/** Frame when the tile is done being processed. */
		int64 TileRegenEndFrame = 0;
#endif // !UE_BUILD_SHIPPING		
		
		/** if we are currently using time sliced regen or not - currently an experimental feature.
		 *  do not manipulate this value directly instead call SetNextTimeSliceRegenActive()
		 */
		bool bTimeSliceRegenActive;
		bool bNextTimeSliceRegenActive;
		
		/** Used by ProcessTileTasksSyncTimeSliced */
		EProcessTileTasksSyncTimeSlicedState ProcessTileTasksSyncState;

		/** Used by ProcessTileTasksSyncTimeSliced */
		TArray<FNavTileRef> UpdatedTilesCache;

		/** Used by AddGeneratedTilesTimeSliced */
		TArray<FNavTileRef> ResultTileRefsCached;

		/** Used by AddGeneratedTilesTimeSliced */
		TMap<int32, dtPolyRef> OldLayerTileIdMapCached;

		/** Used by AddGeneratedTilesTimeSliced */
		EAddGeneratedTilesTimeSlicedState AddGeneratedTilesState;

		/** Used by AddGeneratedTilesTimeSliced */
		int32 AddGenTilesLayerIndex;

		/** Do not null or Reset this directly instead call ResetTimeSlicedTileGeneratorSync(). The only exception currently is in ProcessTileTasksSyncTimeSliced */
		TSharedPtr<FRecastTileGenerator> TileGeneratorSync;
		FNavRegenTimeSliceManager* TimeSliceManager;
	};

	FSyncTimeSlicedData SyncTimeSlicedData;
};

#endif // WITH_RECAST
