// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AI/Navigation/NavigationTypes.h"
#include "AI/Navigation/NavQueryFilter.h"

#if WITH_RECAST

#include "Detour/DetourNavMeshQuery.h"

#define RECAST_VERY_SMALL_AGENT_RADIUS 0.0f

class UNavigationSystemV1;

class FRecastQueryFilter : public INavigationQueryFilterInterface, public dtQueryFilter
{
public:
	NAVIGATIONSYSTEM_API FRecastQueryFilter(bool bIsVirtual = true);
	virtual ~FRecastQueryFilter()= default;

	NAVIGATIONSYSTEM_API virtual void Reset() override;

	NAVIGATIONSYSTEM_API virtual void SetAreaCost(uint8 AreaType, float Cost) override;
	NAVIGATIONSYSTEM_API virtual void SetFixedAreaEnteringCost(uint8 AreaType, float Cost) override;
	NAVIGATIONSYSTEM_API virtual void SetExcludedArea(uint8 AreaType) override;
	NAVIGATIONSYSTEM_API virtual void SetAllAreaCosts(const float* CostArray, const int32 Count) override;
	NAVIGATIONSYSTEM_API virtual void GetAllAreaCosts(float* CostArray, float* FixedCostArray, const int32 Count) const override;
	NAVIGATIONSYSTEM_API virtual void SetBacktrackingEnabled(const bool bBacktracking) override;
	NAVIGATIONSYSTEM_API virtual bool IsBacktrackingEnabled() const override;
	NAVIGATIONSYSTEM_API virtual float GetHeuristicScale() const override;
	NAVIGATIONSYSTEM_API virtual bool IsEqual(const INavigationQueryFilterInterface* Other) const override;
	NAVIGATIONSYSTEM_API virtual void SetIncludeFlags(uint16 Flags) override;
	NAVIGATIONSYSTEM_API virtual uint16 GetIncludeFlags() const override;
	NAVIGATIONSYSTEM_API virtual void SetExcludeFlags(uint16 Flags) override;
	NAVIGATIONSYSTEM_API virtual uint16 GetExcludeFlags() const override;
	virtual FVector GetAdjustedEndLocation(const FVector& EndLocation) const override { return EndLocation; }
	NAVIGATIONSYSTEM_API virtual INavigationQueryFilterInterface* CreateCopy() const override;

	const dtQueryFilter* GetAsDetourQueryFilter() const { return this; }

	/** Changes whether the filter will use virtual set of filtering functions (getVirtualCost and passVirtualFilter)
	 *	or the inlined ones (getInlineCost and passInlineFilter) */
	NAVIGATIONSYSTEM_API void SetIsVirtual(bool bIsVirtual);

	/** Instruct filter whether it can reopen nodes already on closed list */
	NAVIGATIONSYSTEM_API void SetShouldIgnoreClosedNodes(const bool bIgnoreClosed);

	//----------------------------------------------------------------------//
	// @note you might also want to override following functions from dtQueryFilter	
	// virtual bool passVirtualFilter(const dtPolyRef ref, const dtMeshTile* tile, const dtPoly* poly) const;
	// virtual FVector::FReal getVirtualCost(const FVector::FReal* pa, const FVector::FReal* pb, const dtPolyRef prevRef, const dtMeshTile* prevTile, const dtPoly* prevPoly, const dtPolyRef curRef, const dtMeshTile* curTile, const dtPoly* curPoly, const dtPolyRef nextRef, const dtMeshTile* nextTile, const dtPoly* nextPoly) const;
};

struct FRecastSpeciaLinkFilter : public dtQuerySpecialLinkFilter
{
	FRecastSpeciaLinkFilter(UNavigationSystemV1* NavSystem, const UObject* Owner) : NavSys(NavSystem), SearchOwner(Owner), CachedOwnerOb(nullptr) {}
	NAVIGATIONSYSTEM_API virtual bool isLinkAllowed(const uint64 UserId) const override;
	NAVIGATIONSYSTEM_API virtual void initialize() override;

	UNavigationSystemV1* NavSys;
	FWeakObjectPtr SearchOwner;
	UObject* CachedOwnerOb;
};

#endif	// WITH_RECAST
