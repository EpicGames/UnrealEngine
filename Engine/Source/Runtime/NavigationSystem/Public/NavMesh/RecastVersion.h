// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** 
 *  Versioning, Note the correct way of doing this is now to use the Custom Version see Ar.UsingCustomVersion(XXXBranchObjectVersion::GUID) this solves issues when versioning
 *  is changed in different branches at the same time.
 */

#define NAVMESHVER_INITIAL						1
#define NAVMESHVER_TILED_GENERATION				2
#define NAVMESHVER_SEAMLESS_REBUILDING_1		3
#define NAVMESHVER_AREA_CLASSES					4
#define NAVMESHVER_CLUSTER_PATH					5
#define NAVMESHVER_SEGMENT_LINKS				6
#define NAVMESHVER_DYNAMIC_LINKS				7
#define NAVMESHVER_64BIT						9
#define NAVMESHVER_CLUSTER_SIMPLIFIED			10
#define NAVMESHVER_OFFMESH_HEIGHT_BUG			11
#define NAVMESHVER_LANDSCAPE_HEIGHT				13
#define NAVMESHVER_LWCOORDS						14
#define NAVMESHVER_OODLE_COMPRESSION			15
#define NAVMESHVER_LWCOORDS_SEREALIZATION 		17 // Allows for nav meshes to be serialized agnostic of LWCoords being float or double.
#define NAVMESHVER_MAXTILES_COUNT_CHANGE 		19
#define NAVMESHVER_LWCOORDS_OPTIMIZATION		20
#define NAVMESHVER_OPTIM_FIX_SERIALIZE_PARAMS	21 // Fix, serialize params that used to be in the tile and are now in the navmesh.
#define NAVMESHVER_MAXTILES_COUNT_SKIP_INCLUSION 22
#define NAVMESHVER_TILE_RESOLUTIONS				23 // Addition of a tile resolution index to the tile header.
#define NAVMESHVER_TILE_RESOLUTIONS_CELLHEIGHT	24 // Addition of CellHeight in the resolution params, deprecating the original CellHeight.
#define NAVMESHVER_1_VOXEL_AGENT_STEEP_SLOPE_FILTER_FIX	25 // Fix, remove steep slope filtering during heightfield ledge filtering when the agent radius is included into a single voxel
#define NAVMESHVER_TILE_RESOLUTIONS_AGENTMAXSTEPHEIGHT 26	// Addition of AgentMaxStepHeight in the resolution params, deprecating the original AgentMaxStepHeight.
#define NAVMESHVER_NAVLINK_JUMP_CONFIGS			27 // Addition of NavLinkJumpConfigs, deprecating the original NavLinkJumpDownConfig
#define NAVMESHVER_TILEVERT_ENCODING			28 // Store vertex data as tile-local floats (dtTileVert) instead of world-space doubles

#define NAVMESHVER_LATEST				NAVMESHVER_TILEVERT_ENCODING
#define NAVMESHVER_MIN_COMPATIBLE		NAVMESHVER_LWCOORDS_OPTIMIZATION
