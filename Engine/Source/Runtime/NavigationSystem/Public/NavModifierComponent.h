// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "NavAreas/NavArea.h"
#include "NavRelevantComponent.h"
#include "NavModifierComponent.generated.h"

struct FNavigationRelevantData;
class UBodySetup;
enum class ENavigationDataResolution : uint8;

UCLASS(ClassGroup = (Navigation), meta = (BlueprintSpawnableComponent), hidecategories = (Activation), config = Engine, defaultconfig, MinimalAPI)
class UNavModifierComponent : public UNavRelevantComponent
{
	GENERATED_UCLASS_BODY()

	/** NavArea to apply inside the defined volume. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Navigation)
	TSubclassOf<UNavArea> AreaClass;

	/** When setting this value, the modifier behavior changes : it will now replace any surface marked by AreaClassToReplace in the volume and replace it with AreaClass. */ 
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Navigation)
	TSubclassOf<UNavArea> AreaClassToReplace;

	/** box extent used ONLY when owning actor doesn't have collision component */
	UPROPERTY(EditAnywhere, Category = Navigation)
	FVector FailsafeExtent;

	/** Experimental: Indicates which navmesh resolution should be used around the actor. */
	UPROPERTY(EditAnywhere, Category = Navigation, AdvancedDisplay)
	ENavigationDataResolution NavMeshResolution;

	/** Setting to 'true' will result in expanding lower bounding box of the nav 
	 *	modifier by agent's height, before applying to navmesh */
	UPROPERTY(config, EditAnywhere, Category = Navigation)
	uint8 bIncludeAgentHeight : 1;


	// Does the actual calculating and caching of the bounds when called by CalcAndCacheBounds
	NAVIGATIONSYSTEM_API virtual void CalculateBounds() const;
	// @Note We might make this function non-virtual in the future in favor of child classes overriding CalculateBounds, see #jira UE-202451
	NAVIGATIONSYSTEM_API virtual void CalcAndCacheBounds() const override;
	NAVIGATIONSYSTEM_API virtual void GetNavigationData(FNavigationRelevantData& Data) const override;

	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	NAVIGATIONSYSTEM_API void SetAreaClass(TSubclassOf<UNavArea> NewAreaClass);
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	NAVIGATIONSYSTEM_API void SetAreaClassToReplace(TSubclassOf<UNavArea> NewAreaClassToReplace);

protected:
	NAVIGATIONSYSTEM_API void OnTransformUpdated(USceneComponent* RootComponent, EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport);

#if WITH_EDITOR
	NAVIGATIONSYSTEM_API void OnNavAreaRegistered(const UWorld& World, const UClass* NavAreaClass);
	NAVIGATIONSYSTEM_API void OnNavAreaUnregistered(const UWorld& World, const UClass* NavAreaClass);
#endif // WITH_EDITOR 

	//~ Begin UActorComponent Interface
	NAVIGATIONSYSTEM_API virtual void OnRegister() override;
	NAVIGATIONSYSTEM_API virtual void OnUnregister() override;
	//~ End UActorComponent Interface

	NAVIGATIONSYSTEM_API void PopulateComponentBounds(FTransform InParentTransform, const UBodySetup& InBodySetup) const;
	
	struct FRotatedBox
	{
		FBox Box;
		FQuat Quat;

		FRotatedBox() {}
		FRotatedBox(const FBox& InBox, const FQuat& InQuat) : Box(InBox), Quat(InQuat) {}
	};

	mutable TArray<FRotatedBox> ComponentBounds;
	mutable FDelegateHandle TransformUpdateHandle;
	/** cached in CalcAndCacheBounds and tested in GetNavigationData to see if
	 *	cached data is still valid */
	mutable FTransform CachedTransform;

#if WITH_EDITOR
	FDelegateHandle OnNavAreaRegisteredDelegateHandle;
	FDelegateHandle OnNavAreaUnregisteredDelegateHandle;
#endif // WITH_EDITOR 
};
