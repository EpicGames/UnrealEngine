// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "AI/Navigation/NavRelevantInterface.h"
#include "AI/Navigation/NavigationTypes.h"
#include "NavAreas/NavArea.h"
#include "GameFramework/Volume.h"
#include "NavModifierVolume.generated.h"

enum class ENavigationDataResolution : uint8;

struct FNavigationRelevantData;

/** 
 *	Allows applying selected AreaClass to navmesh, using Volume's shape
 */
UCLASS(hidecategories=(Navigation), MinimalAPI)
class ANavModifierVolume : public AVolume, public INavRelevantInterface
{
	GENERATED_BODY()

protected:
	/** NavArea to apply inside the defined volume. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Default)
	TSubclassOf<UNavArea> AreaClass;

	/** When setting this value, the modifier volume behavior changes : it will now replace any surface marked by AreaClassToReplace in the volume and replace it with AreaClass. */ 
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Default)
	TSubclassOf<UNavArea> AreaClassToReplace;

	/** Experimental: if set, the 2D space occupied by the volume box will ignore FillCollisionUnderneathForNavmesh */
	UPROPERTY(EditAnywhere, Category = Default, AdvancedDisplay)
	bool bMaskFillCollisionUnderneathForNavmesh;

	/** Experimental: When not set to None, the navmesh tiles touched by the navigation modifier volume will be built
	 * using the highest resolution found. */
	UPROPERTY(EditAnywhere, Category = Default, AdvancedDisplay)
	ENavigationDataResolution NavMeshResolution;

#if WITH_EDITOR
	FDelegateHandle OnNavAreaRegisteredDelegateHandle;
	FDelegateHandle OnNavAreaUnregisteredDelegateHandle;
#endif

public:
	NAVIGATIONSYSTEM_API ANavModifierVolume(const FObjectInitializer& ObjectInitializer);

	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	NAVIGATIONSYSTEM_API void SetAreaClass(TSubclassOf<UNavArea> NewAreaClass = {});
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	NAVIGATIONSYSTEM_API void SetAreaClassToReplace(TSubclassOf<UNavArea> NewAreaClassToReplace = {});

	TSubclassOf<UNavArea> GetAreaClass() const { return AreaClass; }
	TSubclassOf<UNavArea> GetAreaClassToReplace() const { return AreaClassToReplace; }

	NAVIGATIONSYSTEM_API virtual void GetNavigationData(FNavigationRelevantData& Data) const override;
	NAVIGATIONSYSTEM_API virtual FBox GetNavigationBounds() const override;
	NAVIGATIONSYSTEM_API virtual void RebuildNavigationData() override;

#if WITH_EDITOR
	NAVIGATIONSYSTEM_API virtual void PostEditUndo() override;
	NAVIGATIONSYSTEM_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

protected:
	NAVIGATIONSYSTEM_API virtual void PostLoad() override;
	NAVIGATIONSYSTEM_API virtual void PostInitProperties() override;
	NAVIGATIONSYSTEM_API virtual void BeginDestroy() override;

#if WITH_EDITOR
	NAVIGATIONSYSTEM_API virtual void PostRegisterAllComponents() override;
	NAVIGATIONSYSTEM_API virtual void PostUnregisterAllComponents() override;

	NAVIGATIONSYSTEM_API void OnNavAreaRegistered(const UWorld& World, const UClass* NavAreaClass);
	NAVIGATIONSYSTEM_API void OnNavAreaUnregistered(const UWorld& World, const UClass* NavAreaClass);
#endif
};
