// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "NavNodeInterface.generated.h"

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UNavNodeInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class INavNodeInterface
{
	GENERATED_IINTERFACE_BODY()


	/**
	 *	Retrieves pointer to implementation's UNavigationGraphNodeComponent
	 */
	virtual class UNavigationGraphNodeComponent* GetNavNodeComponent() PURE_VIRTUAL(FNavAgentProperties::GetNavNodeComponent,return NULL;);

};
