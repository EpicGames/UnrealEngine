// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Components/ActorComponent.h"
#include "AI/Navigation/NavRelevantInterface.h"
#include "NavRelevantComponent.generated.h"

UCLASS(MinimalAPI)
class UNavRelevantComponent : public UActorComponent, public INavRelevantInterface
{
	GENERATED_UCLASS_BODY()

	//~ Begin UActorComponent Interface
	NAVIGATIONSYSTEM_API virtual void OnRegister() override;
	NAVIGATIONSYSTEM_API virtual void OnUnregister() override;
	//~ End UActorComponent Interface

	//~ Begin INavRelevantInterface Interface
	NAVIGATIONSYSTEM_API virtual FBox GetNavigationBounds() const override;
	NAVIGATIONSYSTEM_API virtual bool IsNavigationRelevant() const override;
	NAVIGATIONSYSTEM_API virtual void UpdateNavigationBounds() override;
	NAVIGATIONSYSTEM_API virtual UObject* GetNavigationParent() const override;
	//~ End INavRelevantInterface Interface
	
	NAVIGATIONSYSTEM_API virtual void CalcAndCacheBounds() const;

	UFUNCTION(BlueprintCallable, Category="AI|Navigation")
	NAVIGATIONSYSTEM_API void SetNavigationRelevancy(bool bRelevant);

	/** force relevancy and skip attaching navigation data to owner's root entry */
	NAVIGATIONSYSTEM_API void ForceNavigationRelevancy(bool bForce);

	/** force refresh in navigation octree */
	NAVIGATIONSYSTEM_API void RefreshNavigationModifiers();
	
protected:

	/** bounds for navigation octree */
	mutable FBox Bounds;
	
	/** attach navigation data to entry for owner's root component (depends on its relevancy) */
	UPROPERTY()
	uint32 bAttachToOwnersRoot : 1;

	mutable uint32 bBoundsInitialized : 1;
	uint32 bNavParentCacheInitialized : 1;

	UPROPERTY(transient)
	TObjectPtr<UObject> CachedNavParent;
};
