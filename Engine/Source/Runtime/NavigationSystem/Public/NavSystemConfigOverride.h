// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "AI/NavigationSystemBase.h"
#include "NavSystemConfigOverride.generated.h"


class UNavigationSystemConfig;

UENUM()
enum class ENavSystemOverridePolicy : uint8
{
	Override, // the pre-existing nav system instance will be destroyed.
	Append, // config information will be added to pre-existing nav system instance
	Skip	// if there's already a NavigationSystem in the world then the overriding config will be ignored
};


UCLASS(hidecategories = (Input, Rendering, Actor, LOD, Cooking), MinimalAPI)
class ANavSystemConfigOverride : public AActor
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
private:
	UPROPERTY()
	TObjectPtr<class UBillboardComponent> SpriteComponent;
#endif // WITH_EDITORONLY_DATA

protected:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Navigation, Instanced,  meta = (NoResetToDefault))
	TObjectPtr<UNavigationSystemConfig> NavigationSystemConfig;

	/** If there's already a NavigationSystem instance in the world how should this nav override behave */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Navigation)
	ENavSystemOverridePolicy OverridePolicy;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Navigation, AdvancedDisplay)
	uint8 bLoadOnClient : 1;

public:
	NAVIGATIONSYSTEM_API ANavSystemConfigOverride(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	//~ Begin UObject Interface
	NAVIGATIONSYSTEM_API virtual void PostInitProperties() override;
#if WITH_EDITOR
	NAVIGATIONSYSTEM_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	//~ Begin AActor Interface
	NAVIGATIONSYSTEM_API virtual void BeginPlay() override;
#if WITH_EDITOR
	NAVIGATIONSYSTEM_API virtual void PostRegisterAllComponents() override;
	NAVIGATIONSYSTEM_API virtual void PostUnregisterAllComponents() override;
#endif
	//~ End AActor Interface

#if WITH_EDITOR
	/** made an explicit function since rebuilding navigation system can be expensive */
	UFUNCTION(Category = Navigation, meta = (CallInEditor = "true"))
	NAVIGATIONSYSTEM_API void ApplyChanges();
	//virtual void CheckForErrors() override;
#endif

protected:
	/** Creates a new navigation system and plugs it into the world. If there's a
	 *	nav system instance already in place it gets destroyed. */
	NAVIGATIONSYSTEM_API virtual void OverrideNavSystem();

	/** Appends non-conflicting information (like supported agents) to a pre-existing 
	 *	nav system instance */
	NAVIGATIONSYSTEM_API virtual void AppendToNavSystem(UNavigationSystemBase& PrevNavSys);

#if WITH_EDITOR
	/** Called only in the editor mode*/
	NAVIGATIONSYSTEM_API void InitializeForWorld(UNavigationSystemBase* NewNavSys, UWorld* World, const FNavigationSystemRunMode RunMode);
#endif // WITH_EDITOR

	NAVIGATIONSYSTEM_API void ApplyConfig();
};
