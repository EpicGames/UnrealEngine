// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "Async/TaskGraphInterfaces.h"
#include "GameFramework/Actor.h"
#include "AI/Navigation/NavigationTypes.h"
#include "NavigationSystemTypes.h"
#include "EngineDefines.h"
#include "AI/Navigation/NavigationDataInterface.h"

#include "NavigationData.generated.h"

class ANavigationData;
class Error;
class FNavDataGenerator;
class INavAgentInterface;
class INavLinkCustomInterface;
class UNavArea;
class UPrimitiveComponent;
class UNavigationQueryFilter;
struct FNavigationDirtyArea;

USTRUCT()
struct FSupportedAreaData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(VisibleAnywhere, Category=Areas)
	FString AreaClassName;

	UPROPERTY(VisibleAnywhere, Category=Areas)
	int32 AreaID;

	UPROPERTY(transient)
	TObjectPtr<const UClass> AreaClass;

	NAVIGATIONSYSTEM_API FSupportedAreaData(TSubclassOf<UNavArea> NavAreaClass = {}, int32 InAreaID = INDEX_NONE);
};

struct FNavPathRecalculationRequest
{
	FNavPathWeakPtr Path;
	ENavPathUpdateType::Type Reason;

	FNavPathRecalculationRequest(const FNavPathSharedPtr& InPath, ENavPathUpdateType::Type InReason)
		: Path(InPath.ToSharedRef()), Reason(InReason)
	{}

	bool operator==(const FNavPathRecalculationRequest& Other) const { return Path == Other.Path;  }
};

struct FPathFindingResult
{
	FNavPathSharedPtr Path;
	ENavigationQueryResult::Type Result;

	FPathFindingResult(ENavigationQueryResult::Type InResult = ENavigationQueryResult::Invalid) : Result(InResult)
	{ }

	inline bool IsSuccessful() const
	{
		return Result == ENavigationQueryResult::Success;
	}
	inline bool IsPartial() const;
};

struct FNavigationRaycastAdditionalResults
{
	/** When the ray is not obstructed, indicates if the projection of RayEnd is located at the end of the explored corridor.
	 *  When bIsRaytEndInCorridor is false, it means that RayEnd failed to project to the NavigationData or on a navigation node that is not part of the explored corridor (e.g. different height)
	 */
	bool bIsRayEndInCorridor = false;
};

struct FNavigationPath : public TSharedFromThis<FNavigationPath, ESPMode::ThreadSafe>
{
	//DECLARE_DELEGATE_OneParam(FPathObserverDelegate, FNavigationPath*);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FPathObserverDelegate, FNavigationPath*, ENavPathEvent::Type);

	NAVIGATIONSYSTEM_API FNavigationPath();
	NAVIGATIONSYSTEM_API FNavigationPath(const TArray<FVector>& Points, AActor* Base = NULL);
	NAVIGATIONSYSTEM_API virtual ~FNavigationPath();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FNavigationPath(const FNavigationPath&) = default;
	FNavigationPath& operator=(const FNavigationPath& Other) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	inline bool IsValid() const
	{
		return bIsReady && PathPoints.Num() > 1 && bUpToDate;
	}
	inline bool IsUpToDate() const
	{
		return bUpToDate;
	}
	inline bool IsReady() const
	{
		return bIsReady;
	}
	inline bool IsPartial() const
	{
		return bIsPartial;
	}
	inline bool DidSearchReachedLimit() const
	{
		return bReachedSearchLimit;
	}

	// True when the path request failed because the start location couldn't be projected on the NavData.
	// False doesn't guarantee that the location is navigable, it's also possible a previous error skipped the projection altogether
	bool IsErrorStartLocationNonNavigable() const
	{
		return bErrorStartLocationNonNavigable;
	}

	// True when the path request failed because the end location couldn't be projected on the NavData.
	// False doesn't guarantee that the location is navigable, it's also possible a previous error skipped the projection altogether
	bool IsErrorEndLocationNonNavigable() const
	{
		return bErrorEndLocationNonNavigable;
	}
	inline bool IsWaitingForRepath() const
	{
		return bWaitingForRepath;
	}
	inline void SetManualRepathWaiting(const bool bInWaitingForRepath)
	{
		bWaitingForRepath = bInWaitingForRepath;
	}
	inline bool ShouldUpdateStartPointOnRepath() const
	{
		return bUpdateStartPointOnRepath;
	}
	inline bool ShouldUpdateEndPointOnRepath() const
	{
		return bUpdateEndPointOnRepath;
	}
	inline FVector GetDestinationLocation() const
	{
		return IsValid() ? PathPoints.Last().Location : INVALID_NAVEXTENT;
	}
	inline FPathObserverDelegate& GetObserver()
	{
		return ObserverDelegate;
	}
	inline FDelegateHandle AddObserver(FPathObserverDelegate::FDelegate NewObserver)
	{
		return ObserverDelegate.Add(NewObserver);
	}
	inline void RemoveObserver(FDelegateHandle HandleOfObserverToRemove)
	{
		ObserverDelegate.Remove(HandleOfObserverToRemove);
	}

	inline void MarkReady()
	{
		bIsReady = true;
	}

	inline void SetNavigationDataUsed(const ANavigationData* const NewData);

	inline ANavigationData* GetNavigationDataUsed() const
	{
		return NavigationDataUsed.Get();
	}
	inline void SetQuerier(const UObject* InQuerier)
	{
		PathFindingQueryData.Owner = InQuerier;
	}
	inline const UObject* GetQuerier() const
	{
		return PathFindingQueryData.Owner.Get();
	}
	inline void SetQueryData(const FPathFindingQueryData& QueryData)
	{
		PathFindingQueryData = QueryData;
	}
	inline FPathFindingQueryData GetQueryData() const
	{
		// return copy of query data
		return PathFindingQueryData;
	}

	//FORCEINLINE void SetObserver(const FPathObserverDelegate& Observer) { ObserverDelegate = Observer; }
	inline void SetIsPartial(const bool bPartial)
	{
		bIsPartial = bPartial;
	}
	inline void SetSearchReachedLimit(const bool bLimited)
	{
		bReachedSearchLimit = bLimited;
	}
	void SetErrorStartLocationNonNavigable(const bool bErrorNonNavigable)
	{
		bErrorStartLocationNonNavigable = bErrorNonNavigable;
	}
	void SetErrorEndLocationNonNavigable(const bool bErrorNonNavigable)
	{
		bErrorEndLocationNonNavigable = bErrorNonNavigable;
	}
	inline void SetFilter(FSharedConstNavQueryFilter InFilter)
	{
		PathFindingQueryData.QueryFilter = InFilter;
		Filter = InFilter;
	}
	inline FSharedConstNavQueryFilter GetFilter() const
	{
		return PathFindingQueryData.QueryFilter;
	}
	inline AActor* GetBaseActor() const
	{
		return Base.Get();
	}

	FVector GetStartLocation() const
	{
		return PathPoints.Num() > 0 ? PathPoints[0].Location : FNavigationSystem::InvalidLocation;
	}
	FVector GetEndLocation() const
	{
		return PathPoints.Num() > 0 ? PathPoints.Last().Location : FNavigationSystem::InvalidLocation;
	}

	inline void DoneUpdating(ENavPathUpdateType::Type UpdateType)
	{
		static const ENavPathEvent::Type PathUpdateTypeToPathEvent[] = {
			ENavPathEvent::UpdatedDueToGoalMoved // GoalMoved,
			, ENavPathEvent::UpdatedDueToNavigationChanged // NavigationChanged,
			, ENavPathEvent::MetaPathUpdate // MetaPathUpdate,
			, ENavPathEvent::Custom // Custom,
		};

		bUpToDate = true;
		bWaitingForRepath = false;

		if (bUseOnPathUpdatedNotify)
		{
			// notify path before observers
			OnPathUpdated(UpdateType);
		}
		
		ObserverDelegate.Broadcast(this, PathUpdateTypeToPathEvent[uint8(UpdateType)]);
	}

	inline double GetTimeStamp() const { return LastUpdateTimeStamp; }
	inline void SetTimeStamp(double TimeStamp) { LastUpdateTimeStamp = TimeStamp; }

	NAVIGATIONSYSTEM_API void Invalidate();
	NAVIGATIONSYSTEM_API void RePathFailed();

	/** Resets all variables describing generated path before attempting new pathfinding call. 
	  * This function will NOT reset setup variables like goal actor, filter, observer, etc */
	NAVIGATIONSYSTEM_API virtual void ResetForRepath();

	/** Remove points that are at the same location. */
	NAVIGATIONSYSTEM_API void RemoveOverlappingPoints(const FVector& Tolerance);

	NAVIGATIONSYSTEM_API virtual void DebugDraw(const ANavigationData* NavData, const FColor PathColor, class UCanvas* Canvas, const bool bPersistent, const float LifeTime, const uint32 NextPathPointIndex = 0) const;
	
#if ENABLE_VISUAL_LOG
	NAVIGATIONSYSTEM_API virtual void DescribeSelfToVisLog(struct FVisualLogEntry* Snapshot) const;
	NAVIGATIONSYSTEM_API virtual FString GetDescription() const;
#endif // ENABLE_VISUAL_LOG

	/** check if path contains specific custom nav link */
	UE_DEPRECATED(5.3, "Use version that takes FNavLinkId instead. This function only returns false.")
	virtual bool ContainsCustomLink(uint32 UniqueLinkId) const final {	return false; }

	NAVIGATIONSYSTEM_API virtual bool ContainsCustomLink(FNavLinkId UniqueLinkId) const;

	/** check if path contains any custom nav link */
	NAVIGATIONSYSTEM_API virtual bool ContainsAnyCustomLink() const;

	/** check if path contains given node */
	NAVIGATIONSYSTEM_API virtual bool ContainsNode(NavNodeRef NodeRef) const;

	/** get cost of path, starting from given point */
	virtual FVector::FReal GetCostFromIndex(int32 PathPointIndex) const
	{
		return 0.;
	}

	/** get cost of path, starting from given node */
	virtual FVector::FReal GetCostFromNode(NavNodeRef PathNode) const
	{
		return 0.;
	}

	inline FVector::FReal GetCost() const
	{
		return GetCostFromIndex(0);
	}

	/** calculates total length of segments from NextPathPoint to the end of path, plus distance from CurrentPosition to NextPathPoint */
	NAVIGATIONSYSTEM_API virtual FVector::FReal GetLengthFromPosition(FVector SegmentStart, uint32 NextPathPointIndex) const;

	inline FVector::FReal GetLength() const
	{
		return PathPoints.Num() ? GetLengthFromPosition(PathPoints[0].Location, 1) : 0.0f;
	}

	static bool GetPathPoint(const FNavigationPath* Path, uint32 PathVertIdx, FNavPathPoint& PathPoint)
	{
		if (Path && Path->GetPathPoints().IsValidIndex((int32)PathVertIdx))
		{
			PathPoint = Path->PathPoints[PathVertIdx];
			return true;
		}

		return false;
	}

	inline const TArray<FNavPathPoint>& GetPathPoints() const
	{
		return PathPoints;
	}
	inline TArray<FNavPathPoint>& GetPathPoints()
	{
		return PathPoints;
	}

	/** get based position of path point */
	NAVIGATIONSYSTEM_API FBasedPosition GetPathPointLocation(uint32 Index) const;

	/** checks if given path, starting from StartingIndex, intersects with given AABB box */
	NAVIGATIONSYSTEM_API virtual bool DoesIntersectBox(const FBox& Box, uint32 StartingIndex = 0, int32* IntersectingSegmentIndex = NULL, FVector* AgentExtent = NULL) const;
	/** checks if given path, starting from StartingIndex, intersects with given AABB box. This version uses AgentLocation as beginning of the path
	 *	with segment between AgentLocation and path's StartingIndex-th node treated as first path segment to check */
	NAVIGATIONSYSTEM_API virtual bool DoesIntersectBox(const FBox& Box, const FVector& AgentLocation, uint32 StartingIndex = 0, int32* IntersectingSegmentIndex = NULL, FVector* AgentExtent = NULL) const;
	/** retrieves normalized direction vector to given path segment
	 *	for '0'-th segment returns same as for 1st segment 
	 */
	NAVIGATIONSYSTEM_API virtual FVector GetSegmentDirection(uint32 SegmentEndIndex) const;

private:
	NAVIGATIONSYSTEM_API bool DoesPathIntersectBoxImplementation(const FBox& Box, const FVector& StartLocation, uint32 StartingIndex, int32* IntersectingSegmentIndex, FVector* AgentExtent) const;

	/** reset variables describing built path, leaves setup variables required for rebuilding */
	NAVIGATIONSYSTEM_API void InternalResetNavigationPath();

public:

	/** type safe casts */
	template<typename PathClass>
	inline const PathClass* CastPath() const
	{
		return PathType.IsA(PathClass::Type) ? static_cast<const PathClass*>(this) : NULL;
	}

	template<typename PathClass>
	inline PathClass* CastPath()
	{
		return PathType.IsA(PathClass::Type) ? static_cast<PathClass*>(this) : NULL;
	}

	/** enables path observing specified AActor's location and update itself if actor changes location */
	NAVIGATIONSYSTEM_API void SetGoalActorObservation(const AActor& ActorToObserve, float TetherDistance);
	/** Modifies distance to the GoalActor at which we'll update the path */
	void SetGoalActorTetherDistance(const float NewTetherDistace) { GoalActorLocationTetherDistanceSq = FMath::Square(NewTetherDistace); }
	/** turns goal actor location's observation */
	NAVIGATIONSYSTEM_API void DisableGoalActorObservation();
	/** set's up the path to use SourceActor's location in case of recalculation */
	NAVIGATIONSYSTEM_API void SetSourceActor(const AActor& InSourceActor);

	const AActor* GetSourceActor() const { return SourceActor.Get(); }
	const INavAgentInterface* GetSourceActorAsNavAgent() const { return SourceActorAsNavAgent; }

	FVector GetLastRepathGoalLocation() const { return GoalActorLastLocation; }
	NAVIGATIONSYSTEM_API void UpdateLastRepathGoalLocation();
	
	double GetLastUpdateTime() const { return LastUpdateTimeStamp; }
	float GetGoalActorTetherDistance() const { return FMath::Sqrt(GoalActorLocationTetherDistanceSq); }

	/** if enabled path will request recalculation if it gets invalidated due to a change to underlying navigation */
	void EnableRecalculationOnInvalidation(bool bShouldAutoUpdate)
	{
		bDoAutoUpdateOnInvalidation = bShouldAutoUpdate;
	}
	bool WillRecalculateOnInvalidation() const
	{
		return bDoAutoUpdateOnInvalidation;
	}

	/** if ignoring, path will stay bUpToDate after being invalidated due to a change to underlying navigation (observer and auto repath will NOT be triggered!) */
	void SetIgnoreInvalidation(bool bShouldIgnore) { bIgnoreInvalidation = bShouldIgnore; }
	bool GetIgnoreInvalidation() const { return bIgnoreInvalidation; }

	NAVIGATIONSYSTEM_API EPathObservationResult::Type TickPathObservation();

	/** If GoalActor is set it retrieved its navigation location, if not retrieved last path point location */
	NAVIGATIONSYSTEM_API FVector GetGoalLocation() const;

	/** retrieved location to start path finding from (in case of path recalculation) */
	NAVIGATIONSYSTEM_API FVector GetPathFindingStartLocation() const;

	const AActor* GetGoalActor() const
	{
		return bObservingGoalActor ? GoalActor.Get() : nullptr;
	}
	const INavAgentInterface* GetGoalActorAsNavAgent() const
	{
		return GoalActor.IsValid() ? GoalActorAsNavAgent : NULL;
	}

	// @todo this is navigation-type specific and should not be implemented here.
	/** additional node refs used during path following shortcuts */
	TArray<NavNodeRef> ShortcutNodeRefs;

protected:
	
	/** optional notify called when path finishes update, before broadcasting to observes - requires bUseOnPathUpdatedNotify flag set */
	virtual void OnPathUpdated(ENavPathUpdateType::Type UpdateType) {};
	
	/**
	* IMPORTANT: path is assumed to be valid if it contains _MORE_ than _ONE_ point
	*	point 0 is path's starting point - if it's the only point on the path then there's no path per se
	*/
	TArray<FNavPathPoint> PathPoints;

	/** base actor, if exist path points locations will be relative to it */
	TWeakObjectPtr<AActor> Base;

private:
	/** if set path will observe GoalActor's location and update itself if goal moves more then
	*	@note only actual navigation paths can use this feature, meaning the ones associated with
	*	a NavigationData instance (meaning NavigationDataUsed != NULL) */
	TWeakObjectPtr<const AActor> GoalActor;

	/** cached result of GoalActor casting to INavAgentInterface */
	const INavAgentInterface* GoalActorAsNavAgent;

	/** if set will be queried for location in case of path's recalculation */
	TWeakObjectPtr<const AActor> SourceActor;

	/** cached result of PathSource casting to INavAgentInterface */
	const INavAgentInterface* SourceActorAsNavAgent;

protected:
	// DEPRECATED: filter used to build this path
	FSharedConstNavQueryFilter Filter;

	/** type of path */
	static NAVIGATIONSYSTEM_API const FNavPathType Type;

	FNavPathType PathType;

	/** A delegate that will be called when path becomes invalid */
	FPathObserverDelegate ObserverDelegate;

	/** "true" until navigation data used to generate this path has been changed/invalidated */
	uint32 bUpToDate : 1;

	/** when false it means path instance has been created, but not filled with data yet */
	uint32 bIsReady : 1;

	/** "true" when path is only partially generated, when goal is unreachable and path represent best guess */
	uint32 bIsPartial : 1;

	/** set to true when path finding algorithm reached a technical limit (like limit of A* nodes).
	*	This generally means path cannot be trusted to lead to requested destination
	*	although it might lead closer to destination. */
	uint32 bReachedSearchLimit : 1;

	/** "true" when the start location could not be projected on the navigation data */
	uint32 bErrorStartLocationNonNavigable : 1 = false;

	/** "true" when the end location could not be projected on the navigation data */
	uint32 bErrorEndLocationNonNavigable : 1 = false;

	/** if true path will request re-pathing if it gets invalidated due to underlying navigation changed */
	uint32 bDoAutoUpdateOnInvalidation : 1;

	/** if true path will keep bUpToDate value after getting invalidated due to underlying navigation changed
	 *  (observer and auto repath will NOT be triggered!)
	 *  it's NOT safe to use if path relies on navigation data references (e.g. poly corridor)
	 */
	uint32 bIgnoreInvalidation : 1;

	/** if true path will use GetPathFindingStartLocation() for updating QueryData before repath */
	uint32 bUpdateStartPointOnRepath : 1;

	/** if true path will use GetGoalLocation() for updating QueryData before repath */
	uint32 bUpdateEndPointOnRepath : 1;

	/** set when path is waiting for recalc from navigation data */
	uint32 bWaitingForRepath : 1;

	/** if true path will call OnPathUpdated notify */
	uint32 bUseOnPathUpdatedNotify : 1;

	/** indicates whether at any point GoalActor was a valid Actor. Used as
	 *	an optimization in FNavigationPath::TickPathObservation */
	uint32 bObservingGoalActor : 1;

	/** navigation data used to generate this path */
	TWeakObjectPtr<ANavigationData> NavigationDataUsed;

	/** essential part of query used to generate this path */
	FPathFindingQueryData PathFindingQueryData;

	/** gets set during path creation and on subsequent path's updates */
	double LastUpdateTimeStamp;

private:
	/* if GoalActor is set this is the distance we'll try to keep GoalActor from end of path. If GoalActor
	* moves more then this from the end of the path we'll recalculate the path */
	float GoalActorLocationTetherDistanceSq;

	/** last location of goal actor that was used for repaths to prevent spamming when path is partial */
	FVector GoalActorLastLocation;
};

/** 
 *  Supported options for runtime navigation data generation
 */
UENUM()
enum class ERuntimeGenerationType : uint8
{
	// No runtime generation, fully static navigation data
	Static,				
	// Supports only navigation modifiers updates
	DynamicModifiersOnly,	
	// Fully dynamic, supports geometry changes along with navigation modifiers
	Dynamic,
	// Only for legacy loading don't use it!
	LegacyGeneration UMETA(Hidden)
};

/** 
 *	Represents abstract Navigation Data (sub-classed as NavMesh, NavGraph, etc)
 *	Used as a common interface for all navigation types handled by NavigationSystem
 */
UCLASS(config=Engine, defaultconfig, NotBlueprintable, abstract, MinimalAPI)
class ANavigationData : public AActor, public INavigationDataInterface
{
	GENERATED_UCLASS_BODY()
	
	UPROPERTY(transient, duplicatetransient)
	TObjectPtr<UPrimitiveComponent> RenderingComp;

protected:
	UPROPERTY()
	FNavDataConfig NavDataConfig;

	/** if set to true then this navigation data will be drawing itself when requested as part of "show navigation" */
	UPROPERTY(Transient, EditAnywhere, Category=Display)
	uint32 bEnableDrawing:1;

	//----------------------------------------------------------------------//
	// game-time config
	//----------------------------------------------------------------------//
	
	/** By default, navigation will skip the first update after being successfully loaded.
	*  Setting bForceRebuildOnLoad to true can override this behavior. */
	UPROPERTY(config, EditAnywhere, Category = Runtime)
	uint32 bForceRebuildOnLoad : 1;

	/** Should this instance auto-destroy when there's no navigation system on
	 *	world when it gets created/loaded */
	UPROPERTY(config, EditAnywhere, Category = Runtime)
	uint32 bAutoDestroyWhenNoNavigation : 1;

	/** If set, navigation data can act as default one in navigation system's queries */
	UPROPERTY(config, EditAnywhere, Category = Runtime, AdvancedDisplay)
	uint32 bCanBeMainNavData : 1;

	/** If set, navigation data will be spawned in persistent level during rebuild if actor doesn't exist */
	UPROPERTY(config, VisibleAnywhere, Category = Runtime, AdvancedDisplay)
	uint32 bCanSpawnOnRebuild : 1;

	/** If true, the NavMesh can be dynamically rebuilt at runtime. */
	UPROPERTY(config)
	uint32 bRebuildAtRuntime_DEPRECATED:1;

	/** Navigation data runtime generation options */
	UPROPERTY(EditAnywhere, Category = Runtime, config)
	ERuntimeGenerationType RuntimeGeneration;

	/** Set when this nav data was spawned at runtime because no serialized
	 *  instance existed in the loaded level. Gates that normally prevent
	 *  rebuilding Static navmesh in game worlds check this flag to allow
	 *  a one-time rebuild so NPCs can pathfind. Cleared once the rebuild
	 *  completes, after which the nav data resumes normal Static behavior. */
	uint32 bRequiresInitialRebuild : 1;

	/** all observed paths will be processed every ObservedPathsTickInterval seconds */
	UPROPERTY(EditAnywhere, Category = Runtime, config)
	float ObservedPathsTickInterval;

public:
	//----------------------------------------------------------------------//
	// Life cycle                                                                
	//----------------------------------------------------------------------//

	//~ Begin UObject/AActor Interface
	NAVIGATIONSYSTEM_API virtual void PostInitProperties() override;
	NAVIGATIONSYSTEM_API virtual void PostInitializeComponents() override;
	NAVIGATIONSYSTEM_API virtual void PostLoad() override;
#if WITH_EDITOR
	NAVIGATIONSYSTEM_API virtual void RerunConstructionScripts() override;
	NAVIGATIONSYSTEM_API virtual void PostEditUndo() override;
	bool IsBuildingOnLoad() const { return bIsBuildingOnLoad; }
	void SetIsBuildingOnLoad(bool bValue) { bIsBuildingOnLoad = bValue; }
#endif // WITH_EDITOR
	NAVIGATIONSYSTEM_API virtual void Destroyed() override;
	NAVIGATIONSYSTEM_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	//~ End UObject Interface
		
	NAVIGATIONSYSTEM_API virtual void CleanUp();
	NAVIGATIONSYSTEM_API virtual void ApplyWorldOffset(const FVector& InOffset, bool bWorldShift) override;

protected:
	NAVIGATIONSYSTEM_API void RequestRegistration();

private:
	/** Simply unregisters self from navigation system and calls CleanUp */
	NAVIGATIONSYSTEM_API void UnregisterAndCleanUp();

public:
	NAVIGATIONSYSTEM_API void DestroyActor();
	NAVIGATIONSYSTEM_API virtual void CleanUpAndMarkPendingKill();

	inline bool IsRegistered() const { return bRegistered; }
	NAVIGATIONSYSTEM_API virtual void OnRegistered();
	NAVIGATIONSYSTEM_API virtual void OnUnregistered();
	
	inline uint16 GetNavDataUniqueID() const { return NavDataUniqueID; }

	NAVIGATIONSYSTEM_API virtual void TickActor(float DeltaTime, enum ELevelTick TickType, FActorTickFunction& ThisTickFunction) override;

	virtual bool NeedsRebuild() const { return false; }
	NAVIGATIONSYSTEM_API virtual bool SupportsRuntimeGeneration() const;
	NAVIGATIONSYSTEM_API virtual bool SupportsStreaming() const;
	NAVIGATIONSYSTEM_API virtual void OnNavigationBoundsChanged();

	virtual void FillNavigationDataChunkActor(const FBox& QueryBounds, class ANavigationDataChunkActor& DataChunkActor, FBox& OutTilesBounds) const {};
	virtual void OnStreamingNavDataAdded(class ANavigationDataChunkActor& InActor) {};
	virtual void OnStreamingNavDataRemoved(class ANavigationDataChunkActor& InActor) {};
	
	virtual void OnStreamingLevelAdded(ULevel* InLevel, UWorld* InWorld) {};
	virtual void OnStreamingLevelRemoved(ULevel* InLevel, UWorld* InWorld) {};

#if WITH_EDITOR 
	virtual double GetWorldPartitionNavigationDataBuilderOverlap() const { return 0; }
#endif

	//----------------------------------------------------------------------//
	// Generation & data access                                                      
	//----------------------------------------------------------------------//
public:
	inline const FNavDataConfig& GetConfig() const { return NavDataConfig; }
	inline ERuntimeGenerationType GetRuntimeGenerationMode() const { return RuntimeGeneration; }
	bool RequiresInitialRebuild() const { return bRequiresInitialRebuild; }
	void MarkRequiresInitialRebuild() { bRequiresInitialRebuild = true; }
	void ClearRequiresInitialRebuild() { bRequiresInitialRebuild = false; }
	/** Populates NavDataConfig and sets NavAgentProperties with the Src config
	 *  Should be used when initially configuring the navdata and not when updating
	 *  values used in NavDataConfig */
	NAVIGATIONSYSTEM_API virtual void SetConfig(const FNavDataConfig& Src);

	void SetSupportsDefaultAgent(bool bIsDefault) { bSupportsDefaultAgent = bIsDefault; SetNavRenderingEnabled(bIsDefault); }
	bool IsSupportingDefaultAgent() const { return bSupportsDefaultAgent; }
	const FNavAgentProperties& GetNavAgentProperties() const
	{
		return NavAgentProperties;
	}

	NAVIGATIONSYSTEM_API virtual bool DoesSupportAgent(const FNavAgentProperties& AgentProps) const;

	virtual void RestrictBuildingToActiveTiles(bool InRestrictBuildingToActiveTiles) {}

	bool CanBeMainNavData() const { return bCanBeMainNavData; }
	bool CanSpawnOnRebuild() const { return bCanSpawnOnRebuild; }
	bool NeedsRebuildOnLoad() const { return bForceRebuildOnLoad || bRequiresInitialRebuild; }

protected:
	virtual void FillConfig(FNavDataConfig& Dest) { Dest = NavDataConfig; }

public:
	/** Creates new generator in case navigation supports it */
	NAVIGATIONSYSTEM_API virtual void ConditionalConstructGenerator();

	/** Any loading before NavDataGenerator->RebuildAll() */
	virtual void LoadBeforeGeneratorRebuild() {}

	/** Runs after LoadBeforeGeneratorRebuild but before the rebuild. */
	virtual void PostLoadPreRebuild() {}
	
	/** Triggers rebuild in case navigation supports it */
	NAVIGATIONSYSTEM_API virtual void RebuildAll();

	/** Blocks until navigation build is complete  */
	NAVIGATIONSYSTEM_API virtual void EnsureBuildCompletion();

	/** Cancels current build  */
	NAVIGATIONSYSTEM_API virtual void CancelBuild();

	/** Ticks navigation build
	 *  If the generator is set to time sliced rebuild then this function will only get called when 
	 *  there is sufficient time (effectively roughly once in n frames where n is the number of time sliced nav data generators currently building)
	 */
	NAVIGATIONSYSTEM_API virtual void TickAsyncBuild(float DeltaSeconds);
	
	/** Retrieves navigation data generator */
	FNavDataGenerator* GetGenerator() { return NavDataGenerator.Get(); }
	const FNavDataGenerator* GetGenerator() const { return NavDataGenerator.Get(); }

	/** Request navigation data update after changes in nav octree */
	NAVIGATIONSYSTEM_API virtual void RebuildDirtyAreas(const TArray<FNavigationDirtyArea>& DirtyAreas);

	/** Configures this NavData instance's navigation generation to be suspended 
	 *	or active. It's active by default. If Suspended then all calls to 
	 *	RebuildDirtyAreas will result in caching the request in SuspendedDirtyAreas 
	 *	until SetRebuildingSuspended(false) gets call at which time all the contents 
	 *	of SuspendedDirtyAreas will get pushed to the nav generator and SuspendedDirtyAreas 
	 *	will be cleaned out. 
	 *	Note that calling SetRebuildingSuspended(true) won't suspend the nav generation 
	 *	already in progress.
	 *	Warning: Leaving generation suspended for extended periods of time can trigger a RebuildAll().
	 *		To prevent the queue from growing out of control, it is limited by ai.navigation.MaxDirtyAreasCountWhileSuspended
	 *		If that limit is reached, upon resuming, the entire navigation data will be rebuilt.
	 */
	NAVIGATIONSYSTEM_API virtual void SetRebuildingSuspended(const bool bNewSuspend);

	/** Retrieves if this NavData instance's navigation generation is suspended */
	virtual bool IsRebuildingSuspended() const { return bRebuildingSuspended; }

	/** Retrieves the number of suspended dirty areas */
	virtual int32 GetNumSuspendedDirtyAreas() const { return SuspendedDirtyAreas.Num(); }
	
	/** releases navigation generator if any has been created */
protected:
	/** register self with navigation system as new NavAreaDefinition(s) observer */
	NAVIGATIONSYSTEM_API void RegisterAsNavAreaClassObserver();

public:
	/** 
	 *	Created an instance of navigation path of specified type.
	 *	PathType needs to derive from FNavigationPath 
	 */
	template<typename PathType>
	FNavPathSharedPtr CreatePathInstance(const FPathFindingQueryData& QueryData) const
	{
		FNavPathSharedPtr SharedPath = MakeShareable(new PathType());
		SharedPath->SetNavigationDataUsed(this);
		SharedPath->SetQueryData(QueryData);
		SharedPath->SetTimeStamp( GetWorldTimeStamp() );

		const_cast<ANavigationData*>(this)->RegisterActivePath(SharedPath);
		return SharedPath;
	}
	
	void RegisterObservedPath(FNavPathSharedPtr SharedPath)
	{
		check(IsInGameThread());
		if (ObservedPaths.Num() == 0)
		{
			NextObservedPathsTickInSeconds = ObservedPathsTickInterval;
		}
		ObservedPaths.Add(SharedPath);
	}

	void RequestRePath(FNavPathSharedPtr Path, ENavPathUpdateType::Type Reason)
	{
		check(IsInGameThread());
		RepathRequests.AddUnique(FNavPathRecalculationRequest(Path, Reason)); 
	}

protected:
	/** removes from ActivePaths all paths that no longer have shared references (and are invalid in fact) */
	NAVIGATIONSYSTEM_API void PurgeUnusedPaths();

	NAVIGATIONSYSTEM_API void RegisterActivePath(FNavPathSharedPtr SharedPath);

public:
	/** Returns bounding box for the navmesh. */
	virtual FBox GetBounds() const PURE_VIRTUAL(ANavigationData::GetBounds,return FBox(););
	
	/** Returns list of navigable bounds. */
	NAVIGATIONSYSTEM_API TArray<FBox> GetNavigableBounds() const;
	
	/** Returns list of navigable bounds that belongs to specific level */
	NAVIGATIONSYSTEM_API TArray<FBox> GetNavigableBoundsInLevel(ULevel* InLevel) const;
	
	//----------------------------------------------------------------------//
	// Debug                                                                
	//----------------------------------------------------------------------//
	NAVIGATIONSYSTEM_API void DrawDebugPath(FNavigationPath* Path, const FColor PathColor = FColor::White, class UCanvas* Canvas = nullptr, const bool bPersistent = true, const float LifeTime = -1.f, const uint32 NextPathPointIndex = 0) const;

	inline bool IsDrawingEnabled() const { return bEnableDrawing; }

	/** @return Total mem counted, including super calls. */
	NAVIGATIONSYSTEM_API virtual uint32 LogMemUsed() const;

	//----------------------------------------------------------------------//
	// Batch processing (important with async rebuilding)
	//----------------------------------------------------------------------//

	/** Starts batch processing and locks access to navigation data from other threads */
	virtual void BeginBatchQuery() const {}

	/** Finishes batch processing and release locks */
	virtual void FinishBatchQuery() const {};

	//----------------------------------------------------------------------//
	// Querying                                                                
	//----------------------------------------------------------------------//
	inline FSharedConstNavQueryFilter GetDefaultQueryFilter() const { return DefaultQueryFilter; }
	inline const class INavigationQueryFilterInterface* GetDefaultQueryFilterImpl() const { return DefaultQueryFilter->GetImplementation(); }	
	inline FVector GetDefaultQueryExtent() const { return NavDataConfig.DefaultQueryExtent; }

	/** 
	 *	Synchronously looks for a path from @StartLocation to @EndLocation for agent with properties @AgentProperties. NavMesh actor appropriate for specified 
	 *	FNavAgentProperties will be found automatically
	 *	@param ResultPath results are put here
	 *	@return true if path has been found, false otherwise
	 *
	 *	@note don't make this function virtual! Look at implementation details and its comments for more info.
	 */
	inline FPathFindingResult FindPath(const FNavAgentProperties& AgentProperties, const FPathFindingQuery& Query) const
	{
		check(FindPathImplementation);
		// this awkward implementation avoids virtual call overhead - it's possible this function will be called a lot
		return (*FindPathImplementation)(AgentProperties, Query);
	}

	/** 
	 *	Synchronously looks for a path from @StartLocation to @EndLocation for agent with properties @AgentProperties. NavMesh actor appropriate for specified 
	 *	FNavAgentProperties will be found automatically
	 *	@param ResultPath results are put here
	 *	@return true if path has been found, false otherwise
	 *
	 *	@note don't make this function virtual! Look at implementation details and its comments for more info.
	 */
	inline FPathFindingResult FindHierarchicalPath(const FNavAgentProperties& AgentProperties, const FPathFindingQuery& Query) const
	{
		check(FindHierarchicalPathImplementation);
		// this awkward implementation avoids virtual call overhead - it's possible this function will be called a lot
		return (*FindHierarchicalPathImplementation)(AgentProperties, Query);
	}

	/** 
	 *	Synchronously checks if path between two points exists
	 *	FNavAgentProperties will be found automatically
	 *	@return true if path has been found, false otherwise
	 *
	 *	@note don't make this function virtual! Look at implementation details and its comments for more info.
	 */
	inline bool TestPath(const FNavAgentProperties& AgentProperties, const FPathFindingQuery& Query, int32* NumVisitedNodes) const
	{
		check(TestPathImplementation);
		// this awkward implementation avoids virtual call overhead - it's possible this function will be called a lot
		return (*TestPathImplementation)(AgentProperties, Query, NumVisitedNodes);
	}

	/** 
	 *	Synchronously checks if path between two points exists using hierarchical graph
	 *	FNavAgentProperties will be found automatically
	 *	@return true if path has been found, false otherwise
	 *
	 *	@note don't make this function virtual! Look at implementation details and its comments for more info.
	 */
	inline bool TestHierarchicalPath(const FNavAgentProperties& AgentProperties, const FPathFindingQuery& Query, int32* NumVisitedNodes) const
	{
		check(TestHierarchicalPathImplementation);
		// this awkward implementation avoids virtual call overhead - it's possible this function will be called a lot
		return (*TestHierarchicalPathImplementation)(AgentProperties, Query, NumVisitedNodes);
	}

	/** 
	 *	Synchronously makes a raycast on navigation data using QueryFilter
	 *	@param HitLocation if line was obstructed this will be set to hit location. Otherwise it contains SegmentEnd
	 *	@return true if line from RayStart to RayEnd is obstructed
	 *
	 *	@note don't make this function virtual! Look at implementation details and its comments for more info.
	 */
	inline bool Raycast(const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, FSharedConstNavQueryFilter QueryFilter, const UObject* Querier = NULL) const
	{
		return Raycast(RayStart, RayEnd, HitLocation, nullptr/*AdditionalResults*/, QueryFilter, Querier);
	}

	/** 
	 *	Synchronously makes a raycast on navigation data using QueryFilter
	 *	@param HitLocation if line was obstructed this will be set to hit location. Otherwise it contains SegmentEnd
	 *	@param AdditionalResults contains more information about the result of the raycast query. See FNavigationRaycastAdditionalResults description for details
	 *	@return true if line from RayStart to RayEnd is obstructed
	 *
	 *	@note don't make this function virtual! Look at implementation details and its comments for more info.
	 */
	inline bool Raycast(const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, FNavigationRaycastAdditionalResults* AdditionalResults, FSharedConstNavQueryFilter QueryFilter, const UObject* Querier = NULL) const
	{
		check(RaycastImplementationWithAdditionalResults);
		// this awkward implementation avoids virtual call overhead - it's possible this function will be called a lot
		return (*RaycastImplementationWithAdditionalResults)(this, RayStart, RayEnd, HitLocation, AdditionalResults, QueryFilter, Querier);
	}

	/** Raycasts batched for efficiency */
	virtual void BatchRaycast(TArray<FNavigationRaycastWork>& Workload, FSharedConstNavQueryFilter QueryFilter, const UObject* Querier = NULL) const PURE_VIRTUAL(ANavigationData::BatchRaycast, );

	/**	Tries to move current nav location towards target constrained to navigable area. Faster than ProjectPointToNavmesh.
	 *	@param OutLocation if successful this variable will be filed with result
	 *	@return true if successful, false otherwise
	 */
	virtual bool FindMoveAlongSurface(const FNavLocation& StartLocation, const FVector& TargetPosition, FNavLocation& OutLocation, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const PURE_VIRTUAL(ANavigationData::FindMoveAlongSurface, return false;);

	/**	Batches FindMoveAlongSurface calls for efficiency.
	 *	@note Refer to FindMoveAlongSurface function documentation for more information
	 */
	virtual void BatchFindMoveAlongSurface(TArray<FNavigationFindMoveAlongSurfaceWork>& Workload, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const PURE_VIRTUAL(ANavigationData::BatchFindMoveAlongSurface, );

	/**	Returns the navmesh edges that touch the convex polygon. The edges are not clipped by the polygon. 
	 *	@param StartLocation a location on the navmesh where to start searching.
	 *	@param ConvexPolygon 2D convex polygon that describes the search area. 
	 *	@param OutEdges result edges, each edge is an adjacent pair of points in the array.
	 *	@param Filter Nav filter to use, or if nullptr, default filter is used. 
	 *	@return true if successful, false otherwise
	 */
	virtual bool FindOverlappingEdges(const FNavLocation& StartLocation, TConstArrayView<FVector> ConvexPolygon, TArray<FVector>& OutEdges, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const PURE_VIRTUAL(ANavigationData::FindOverlappingEdges, return false;);
	
	/**	Searches navmesh edges between the two path points, search up to the convex polygon described in SearchArea. The returned edges are not clipped to the search area polygon.
	 *  @param Path Path where From and To belong to.
	 *	@param StartPoint start location of the path segment.
	 *	@param EndPoint end location of the path segment.
	 *	@param SearchArea 2D convex polygon that describes the search area.
	 *	@param OutEdges result edges, each edge is an adjacent pair of points in the array.
	 *	@param MaxAreaEnterCost if the fixed cost to enter a node is higher than this value, the node is considered unnavigable.
	 *	@param Filter Nav filter to use, or if nullptr, default filter is used. 
	 *	@return true if successful, false otherwise
	 */
	virtual bool GetPathSegmentBoundaryEdges(const FNavigationPath& Path, const FNavPathPoint& StartPoint, const FNavPathPoint& EndPoint, const TConstArrayView<FVector> SearchArea, TArray<FVector>& OutEdges, const float MaxAreaEnterCost, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const PURE_VIRTUAL(ANavigationData::GetPathSegmentBoundaryEdges, return false;);

	virtual FNavLocation GetRandomPoint(FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const PURE_VIRTUAL(ANavigationData::GetRandomPoint, return FNavLocation(););

	/** finds a random location in Radius, reachable from Origin */
	virtual bool GetRandomReachablePointInRadius(const FVector& Origin, float Radius, FNavLocation& OutResult, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const PURE_VIRTUAL(ANavigationData::GetRandomReachablePointInRadius, return false;);

	/** finds a random location in navigable space, in given Radius */
	virtual bool GetRandomPointInNavigableRadius(const FVector& Origin, float Radius, FNavLocation& OutResult, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const PURE_VIRTUAL(ANavigationData::GetRandomPointInNavigableRadius, return false;);
	
	/**	Tries to project given Point to this navigation type, within given Extent.
	 *	@param OutLocation if successful this variable will be filed with result
	 *	@return true if successful, false otherwise
	 */
	virtual bool ProjectPoint(const FVector& Point, FNavLocation& OutLocation, const FVector& Extent, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const PURE_VIRTUAL(ANavigationData::ProjectPoint, return false;);

	/**	batches ProjectPoint's work for efficiency */
	virtual void BatchProjectPoints(TArray<FNavigationProjectionWork>& Workload, const FVector& Extent, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const PURE_VIRTUAL(ANavigationData::BatchProjectPoints, );

	/** Project batch of points using shared search filter. This version is not requiring user to pass in Extent, 
	 *	and is instead relying on FNavigationProjectionWork.ProjectionLimit.
	 *	@note function should assert if item's FNavigationProjectionWork.ProjectionLimit is invalid */
	virtual void BatchProjectPoints(TArray<FNavigationProjectionWork>& Workload, FSharedConstNavQueryFilter Filter = NULL, const UObject* Querier = NULL) const PURE_VIRTUAL(ANavigationData::BatchProjectPoints, );

	/** Calculates path from PathStart to PathEnd and retrieves its cost.
 *	@NOTE this function does not generate string pulled path so the result is an (over-estimated) approximation
 *	@NOTE potentially expensive, so use it with caution */
	virtual ENavigationQueryResult::Type CalcPathCost(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& OutPathCost, FSharedConstNavQueryFilter QueryFilter = NULL, const UObject* Querier = NULL) const PURE_VIRTUAL(ANavigationData::CalcPathCost, return ENavigationQueryResult::Invalid;);

	/** Calculates path from PathStart to PathEnd and retrieves its length.
	 *	@NOTE this function does not generate string pulled path so the result is an (over-estimated) approximation
	 *	@NOTE potentially expensive, so use it with caution */
	virtual ENavigationQueryResult::Type CalcPathLength(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& OutPathLength, FSharedConstNavQueryFilter QueryFilter = NULL, const UObject* Querier = NULL) const PURE_VIRTUAL(ANavigationData::CalcPathLength, return ENavigationQueryResult::Invalid;);

	/** Calculates path from PathStart to PathEnd and retrieves its length.
	 *	@NOTE this function does not generate string pulled path so the result is an (over-estimated) approximation
	 *	@NOTE potentially expensive, so use it with caution */
	virtual ENavigationQueryResult::Type CalcPathLengthAndCost(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& OutPathLength, FVector::FReal& OutPathCost, FSharedConstNavQueryFilter QueryFilter = NULL, const UObject* Querier = NULL) const PURE_VIRTUAL(ANavigationData::CalcPathLengthAndCost, return ENavigationQueryResult::Invalid;);

	/** Checks if specified navigation node contains given location 
	 *	@param Location is expressed in WorldSpace, navigation data is responsible for tansforming if need be */
	virtual bool DoesNodeContainLocation(NavNodeRef NodeRef, const FVector& WorldSpaceLocation) const PURE_VIRTUAL(ANavigationData::DoesNodeContainLocation, return false;);

	NAVIGATIONSYSTEM_API double GetWorldTimeStamp() const;

	//----------------------------------------------------------------------//
	// Areas
	//----------------------------------------------------------------------//

	/** new area was registered in navigation system */
	NAVIGATIONSYSTEM_API virtual void OnNavAreaAdded(const UClass* NavAreaClass, int32 AgentIndex);
	
	/** area was removed from navigation system */
	NAVIGATIONSYSTEM_API virtual void OnNavAreaRemoved(const UClass* NavAreaClass);

	/** called after changes to registered area classes */
	NAVIGATIONSYSTEM_API virtual void OnNavAreaChanged();

	NAVIGATIONSYSTEM_API void OnNavAreaEvent(const UClass* NavAreaClass, ENavAreaEvent::Type Event);

	/** add all existing areas */
	NAVIGATIONSYSTEM_API void ProcessNavAreas(const TSet<const UClass*>& AreaClasses, int32 AgentIndex);

	/** get class associated with AreaID */
	NAVIGATIONSYSTEM_API const UClass* GetAreaClass(int32 AreaID) const;
	
	/** check if AreaID was assigned to class (class itself may not be loaded yet!) */
	NAVIGATIONSYSTEM_API bool IsAreaAssigned(int32 AreaID) const;

	/** get ID assigned to AreaClas or -1 when not assigned */
	NAVIGATIONSYSTEM_API int32 GetAreaID(const UClass* AreaClass) const;

	/** get max areas supported by this navigation data */
	virtual int32 GetMaxSupportedAreas() const { return MAX_int32; }

	/** read all supported areas */
	void GetSupportedAreas(TArray<FSupportedAreaData>& Areas) const { Areas = SupportedAreas; }

	//----------------------------------------------------------------------//
	// Custom navigation links
	//----------------------------------------------------------------------//

	NAVIGATIONSYSTEM_API virtual void UpdateCustomLink(const INavLinkCustomInterface* CustomLink);

	//----------------------------------------------------------------------//
	// Filters
	//----------------------------------------------------------------------//

	/** get cached query filter */
	NAVIGATIONSYSTEM_API FSharedConstNavQueryFilter GetQueryFilter(TSubclassOf<UNavigationQueryFilter> FilterClass) const;

	/** store cached query filter */
	NAVIGATIONSYSTEM_API void StoreQueryFilter(TSubclassOf<UNavigationQueryFilter> FilterClass, FSharedConstNavQueryFilter NavFilter);

	/** removes cached query filter */
	NAVIGATIONSYSTEM_API void RemoveQueryFilter(TSubclassOf<UNavigationQueryFilter> FilterClass);

	//----------------------------------------------------------------------//
	// all the rest                                                                
	//----------------------------------------------------------------------//
	virtual UPrimitiveComponent* ConstructRenderingComponent() { return NULL; }

	/** updates state of rendering component */
	NAVIGATIONSYSTEM_API void SetNavRenderingEnabled(bool bEnable);

#if WITH_EDITOR
	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return false; }
#endif

protected:
	NAVIGATIONSYSTEM_API void InstantiateAndRegisterRenderingComponent();

	/** get ID to assign for newly added area */
	NAVIGATIONSYSTEM_API virtual int32 GetNewAreaID(const UClass* AreaClass) const;
	
protected:
	/** Navigation data versioning. */
	UPROPERTY()
	uint32 DataVersion;

	typedef FPathFindingResult (*FFindPathPtr)(const FNavAgentProperties& AgentProperties, const FPathFindingQuery& Query);
	FFindPathPtr FindPathImplementation;
	FFindPathPtr FindHierarchicalPathImplementation; 
	
	typedef bool (*FTestPathPtr)(const FNavAgentProperties& AgentProperties, const FPathFindingQuery& Query, int32* NumVisitedNodes);
	FTestPathPtr TestPathImplementation;
	FTestPathPtr TestHierarchicalPathImplementation; 

	typedef bool(*FNavRaycastPtr)(const ANavigationData* NavDataInstance, const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, FSharedConstNavQueryFilter QueryFilter, const UObject* Querier);
	UE_DEPRECATED(5.6, "Please use RaycastImplementationWithAdditionalResults instead") 
	FNavRaycastPtr RaycastImplementation;

	typedef bool(*FNavRaycastWithAdditionalResultsPtr)(const ANavigationData* NavDataInstance, const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, FNavigationRaycastAdditionalResults* AdditionalResults, FSharedConstNavQueryFilter QueryFilter, const UObject* Querier);
	FNavRaycastWithAdditionalResultsPtr RaycastImplementationWithAdditionalResults;

protected:
	TSharedPtr<FNavDataGenerator, ESPMode::ThreadSafe> NavDataGenerator;

	/** caches requests to rebuild dirty areas while nav rebuilding is suspended 
	 *	via SetRebuildingSuspended(true) call. Calling SetRebuildingSuspended(false) 
	 *	will result in pushing SuspendedDirtyAreas contents to the nav generator 
	 *	and clearing out of SuspendedDirtyAreas */
	TArray<FNavigationDirtyArea> SuspendedDirtyAreas;

	/** 
	 *	Container for all path objects generated with this Navigation Data instance. 
	 *	Is meant to be added to only on GameThread, and in fact should user should never 
	 *	add items to it manually, @see CreatePathInstance
	 */
	TArray<FNavPathWeakPtr> ActivePaths;

	/** Synchronization object for paths registration from main thread and async pathfinding thread */
	mutable FTransactionallySafeCriticalSection ActivePathsLock;

	/**
	 *	Contains paths that requested observing its goal's location. These paths will be 
	 *	processed on a regular basis (@see ObservedPathsTickInterval) */
	TArray<FNavPathWeakPtr> ObservedPaths;

	/** paths that requested re-calculation */
	TArray<FNavPathRecalculationRequest> RepathRequests;

	/** contains how much time left to the next ObservedPaths processing */
	float NextObservedPathsTickInSeconds;

	/** Query filter used when no other has been passed to relevant functions */
	FSharedNavQueryFilter DefaultQueryFilter;

	/** Map of query filters by UNavigationQueryFilter class */
	TMap<UClass*,FSharedConstNavQueryFilter > QueryFilters;

	/** serialized area class - ID mapping */
	UPROPERTY(VisibleAnywhere, Category=Areas, meta=(TitleProperty="{AreaID}: {AreaClassName}"))
	TArray<FSupportedAreaData> SupportedAreas;

	/** mapping for SupportedAreas */
	TMap<const UClass*, int32> AreaClassToIdMap;

	/** whether this instance is registered with Navigation System*/
	uint32 bRegistered : 1;

	/** was it generated for default agent (SupportedAgents[0]) */
	uint32 bSupportsDefaultAgent : 1;

	/** Set via SetRebuildingSuspended and controlling if RebuildDirtyAreas get 
	 *	passed over to the generator instantly or cached in SuspendedDirtyAreas 
	 *	to be applied at later date with SetRebuildingSuspended(false) call */
	uint32 bRebuildingSuspended : 1;

	/** Becomes true when too many dirty areas have been accumulated while rebuild
	 *	was suspended (see SetRebuildingSuspended).
	 *	If rebuilding is resumed after reaching the max count, this entire navigation
	 *	data will be rebuilt. */
	uint32 bReachedSuspendedAreasMaxCount : 1 = false;

#if WITH_EDITORONLY_DATA
	uint32 bIsBuildingOnLoad : 1;
#endif

private:
	uint16 NavDataUniqueID;

	static NAVIGATIONSYSTEM_API uint16 GetNextUniqueID();

protected:
	/** The exact nav agent properties used when registering this navdata
	 *  in the navigation system. If navdata configuration changes are
	 *  needed, NavDataConfig can be modified and used */
	FNavAgentProperties NavAgentProperties;

	void SetNavAgentProperties(const FNavAgentProperties& InNavAgentProperties)
	{
		NavAgentProperties = InNavAgentProperties;
	}
};

struct FAsyncPathFindingQuery : public FPathFindingQuery
{
	const uint32 QueryID;
	const FNavPathQueryDelegate OnDoneDelegate;
	const TEnumAsByte<EPathFindingMode::Type> Mode;
	FPathFindingResult Result;

	FAsyncPathFindingQuery()
		: QueryID(INVALID_NAVQUERYID)
		, Mode(EPathFindingMode::Regular)
	{ }

	FAsyncPathFindingQuery(const UObject* InOwner, const ANavigationData& InNavData, const FVector& Start, const FVector& End, const FNavPathQueryDelegate& Delegate, FSharedConstNavQueryFilter SourceQueryFilter, const FVector::FReal CostLimit = TNumericLimits<FVector::FReal>::Max());
	FAsyncPathFindingQuery(const FPathFindingQuery& Query, const FNavPathQueryDelegate& Delegate, const EPathFindingMode::Type QueryMode);

protected:
	inline static uint32 GetUniqueID()
	{
		return ++LastPathFindingUniqueID;
	}

	static uint32 LastPathFindingUniqueID;
};

inline bool FPathFindingResult::IsPartial() const
{
	return (Result != ENavigationQueryResult::Error) && Path.IsValid() && Path->IsPartial();
}

inline void FNavigationPath::SetNavigationDataUsed(const ANavigationData* const NavData)
{
	NavigationDataUsed = MakeWeakObjectPtr(const_cast<ANavigationData*>(NavData));
}
