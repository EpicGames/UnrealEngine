// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationOctreeController.h"
#include "NavigationDirtyAreasController.h"

struct FNavigationDirtyElement;
class UNavArea;

struct FNavigationDataHandler
{
	FNavigationOctreeController& OctreeController;
	FNavigationDirtyAreasController& DirtyAreasController;

	NAVIGATIONSYSTEM_API FNavigationDataHandler(FNavigationOctreeController& InOctreeController, FNavigationDirtyAreasController& InDirtyAreasController);

	NAVIGATIONSYSTEM_API void ConstructNavOctree(const FVector& Origin, const double Radius, const ENavDataGatheringModeConfig DataGatheringMode, const float GatheringNavModifiersWarningLimitTime);

	/**
	 * Removes the octree node and the NavigationElementHandle-OctreeElementId pair associated to the specified OctreeElementId.
	 * It will also dirty the area base of the NavigationElement values and the specified update flags.
	 */
	NAVIGATIONSYSTEM_API void RemoveFromNavOctree(const FOctreeElementId2& ElementId, int32 UpdateFlags);

	NAVIGATIONSYSTEM_API FSetElementId RegisterElementWithNavOctree(const TSharedRef<const FNavigationElement>& ElementRef, int32 UpdateFlags);
	NAVIGATIONSYSTEM_API void AddElementToNavOctree(const FNavigationDirtyElement& DirtyElement);

	/**
	 * Removes associated NavOctreeElement and invalidates associated pending updates. Also removes element from the list of children
	 * of the NavigationParent, if any.
	 * @param ElementRef		Navigation element for which we must remove the associated NavOctreeElement
	 * @param UpdateFlags		Flags indicating in which context the method is called to allow/forbid certain operations
	 *
	 * @return True if associated NavOctreeElement has been removed or pending update has been invalidated; false otherwise.
	 */
	NAVIGATIONSYSTEM_API bool UnregisterElementWithNavOctree(const TSharedRef<const FNavigationElement>& ElementRef, int32 UpdateFlags);

	/**
	 * Unregister element associated with the provided handle and register the new element.
	 * Also update any pending update associated to that element.
	 */
	NAVIGATIONSYSTEM_API void UpdateNavOctreeElement(FNavigationElementHandle ElementHandle, const TSharedRef<const FNavigationElement>& UpdatedElement, int32 UpdateFlags);

	NAVIGATIONSYSTEM_API bool UpdateNavOctreeElementBounds(FNavigationElementHandle Element, const FBox& NewBounds, const TConstArrayView<FBox> DirtyAreas);

	NAVIGATIONSYSTEM_API void FindElementsInNavOctree(const FBox& QueryBox, const FNavigationOctreeFilter& Filter, TArray<FNavigationOctreeElement>& Elements);

	NAVIGATIONSYSTEM_API bool ReplaceAreaInOctreeData(FNavigationElementHandle Element, TSubclassOf<UNavArea> OldArea, TSubclassOf<UNavArea> NewArea, bool bReplaceChildClasses) const;

	NAVIGATIONSYSTEM_API void AddLevelCollisionToOctree(ULevel& Level);
	NAVIGATIONSYSTEM_API void RemoveLevelCollisionFromOctree(ULevel& Level);

	NAVIGATIONSYSTEM_API void ProcessPendingOctreeUpdates();

	NAVIGATIONSYSTEM_API void DemandLazyDataGathering(FNavigationRelevantData& ElementData);

private:
	void UpdateNavOctreeParentChain(const TSharedRef<const FNavigationElement>& Element, bool bSkipElementOwnerUpdate = false);
};
