// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Logging/LogMacros.h"
#include "Math/Box.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class UObject;
class ANavigationData;
struct FNavigationDirtyElement;
struct FNavigationElement;
struct FNavigationDirtyArea;
enum class ENavigationDirtyFlag : uint8;

NAVIGATIONSYSTEM_API DECLARE_LOG_CATEGORY_EXTERN(LogNavigationDirtyArea, Warning, All);

struct FNavigationDirtyAreasController
{
	/** update frequency for dirty areas on navmesh */
	float DirtyAreasUpdateFreq = 60.f;

	/** temporary cumulative time to calculate when we need to update dirty areas */
	float DirtyAreasUpdateTime = 0.f;

	/** stores areas marked as dirty throughout the frame, processes them
 *	once a frame in Tick function */
	TArray<FNavigationDirtyArea> DirtyAreas;

	uint8 bCanAccumulateDirtyAreas : 1;
	uint8 bUseWorldPartitionedDynamicMode : 1;

#if !UE_BUILD_SHIPPING
	uint8 bDirtyAreasReportedWhileAccumulationLocked : 1;
private:
	uint8 bCanReportOversizedDirtyArea : 1;
	uint8 bNavigationBuildLocked : 1;

	/** -1 by default, if set to a positive value dirty area with bounds size over that threshold will be logged */
	float DirtyAreaWarningSizeThreshold = -1.f;

	NAVIGATIONSYSTEM_API bool ShouldReportOversizedDirtyArea() const;
#endif // !UE_BUILD_SHIPPING

public:
	NAVIGATIONSYSTEM_API FNavigationDirtyAreasController();

	NAVIGATIONSYSTEM_API void Reset();
	
	/** sets cumulative time to at least one cycle so next tick will rebuild dirty areas */
	NAVIGATIONSYSTEM_API void ForceRebuildOnNextTick();

	NAVIGATIONSYSTEM_API void Tick(float DeltaSeconds, const TArray<ANavigationData*>& NavDataSet, bool bForceRebuilding = false);

	/** Add a dirty area to the queue based on the provided bounds and flags.
	 * Bounds must be valid and non-empty otherwise the request will be ignored and a warning reported.
	 * Accumulation must be allowed and flags valid otherwise the add is ignored.
	 * @param NewArea Bounding box of the affected area
	 * @param Flags Indicates the type of modification applied to the area
	 * @param ElementProviderFunc Optional function to retrieve source element that can be used for error reporting and navmesh exclusion
	 * @param DirtyElement Optional dirty element
	 * @param DebugReason Source of the new area
	 */
	NAVIGATIONSYSTEM_API void AddArea(const FBox& NewArea, const ENavigationDirtyFlag Flags, const TFunction<const TSharedPtr<const FNavigationElement>()>& ElementProviderFunc = nullptr,
		const FNavigationDirtyElement* DirtyElement = nullptr, const FName& DebugReason = NAME_None);

	/** Add non-empty list of dirty areas to the queue based on the provided bounds and flags.
	 * Bounds must be valid and non-empty otherwise the request will be ignored and a warning reported.
	 * Accumulation must be allowed and flags valid otherwise the add is ignored.
	 * A check will be triggered if an empty array is provided.
	 * @param NewAreas Array of bounding boxes of the affected areas
	 * @param Flags Indicates the type of modification applied to the area
	 * @param ElementProviderFunc Optional function to retrieve source element that can be used for error reporting and navmesh exclusion
	 * @param DirtyElement Optional dirty element
	 * @param DebugReason Source of the new area
	 */
	NAVIGATIONSYSTEM_API void AddAreas(const TConstArrayView<FBox> NewAreas, const ENavigationDirtyFlag Flags, const TFunction<const TSharedPtr<const FNavigationElement>()>& ElementProviderFunc = nullptr,
		const FNavigationDirtyElement* DirtyElement = nullptr, const FName& DebugReason = NAME_None);
	
	bool IsDirty() const { return GetNumDirtyAreas() > 0; }
	int32 GetNumDirtyAreas() const { return DirtyAreas.Num(); }

	NAVIGATIONSYSTEM_API void OnNavigationBuildLocked();
	NAVIGATIONSYSTEM_API void OnNavigationBuildUnlocked();

	NAVIGATIONSYSTEM_API void SetUseWorldPartitionedDynamicMode(bool bIsWPDynamic);
	NAVIGATIONSYSTEM_API void SetCanReportOversizedDirtyArea(const bool bCanReport);
	NAVIGATIONSYSTEM_API void SetDirtyAreaWarningSizeThreshold(const float Threshold);

#if !UE_BUILD_SHIPPING
	bool HadDirtyAreasReportedWhileAccumulationLocked() const { return bCanAccumulateDirtyAreas == false && bDirtyAreasReportedWhileAccumulationLocked; }
#endif // UE_BUILD_SHIPPING

	DECLARE_DELEGATE_RetVal_OneParam(bool, FSkipObjectSignature, const UObject& /*Object*/);
	FSkipObjectSignature ShouldSkipObjectPredicate;
};
