// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "AI/Navigation/NavAgentSelector.h"
#include "AI/Navigation/NavigationTypes.h"
#include "Components/ActorComponent.h"
#include "NavigationInvokerComponent.generated.h"

class UNavigationSystemV1;
enum class ENavigationInvokerPriority : uint8;

UCLASS(ClassGroup = (Navigation), meta = (BlueprintSpawnableComponent), MinimalAPI)
class UNavigationInvokerComponent : public UActorComponent
{
	GENERATED_BODY()

protected:

	/** Navigation data is requested within a TileGenerationRadius circle around the component owner. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Navigation, meta = (ClampMin = "0.1", ClampMax = "6400000", UIMin = "0.1", UIMax = "6400000"))
	float TileGenerationRadius;

	/** Navigation data can be discarded when outside of a TileRemovalRadius circle around the component owner.
	 * This is computed for all navigation invokers, so a navigation data must be outside of all navigation invokers TileRemovalRadius circles to be discarded. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Navigation, meta = (ClampMin = "0.1", ClampMax = "6400000", UIMin = "0.1", UIMax = "6400000"))
	float TileRemovalRadius;

	/** restrict navigation generation to specific agents */
	UPROPERTY(EditAnywhere, Category = Navigation)
	FNavAgentSelector SupportedAgents;

	/** Experimental invocation priority. It will modify the order in which invoked tiles are being built if SortPendingTilesMethod is set to SortByPriority. */
	UPROPERTY(EditAnywhere, Category = Navigation)
	ENavigationInvokerPriority Priority;

public:
	NAVIGATIONSYSTEM_API UNavigationInvokerComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	NAVIGATIONSYSTEM_API void RegisterWithNavigationSystem(UNavigationSystemV1& NavSys);

	/** Sets generation/removal ranges. Doesn't force navigation system's update.
	 *	Will get picked up the next time NavigationSystem::UpdateInvokers gets called */
	NAVIGATIONSYSTEM_API void SetGenerationRadii(const float GenerationRadius, const float RemovalRadius);

	float GetGenerationRadius() const { return TileGenerationRadius; }
	float GetRemovalRadius() const { return TileRemovalRadius; }

	NAVIGATIONSYSTEM_API virtual void Activate(bool bReset = false) override;
	NAVIGATIONSYSTEM_API virtual void Deactivate() override;
	NAVIGATIONSYSTEM_API virtual void PostInitProperties() override;

private:
	void OnNavigationInitStart(const class UNavigationSystemBase& NavSys);
};
