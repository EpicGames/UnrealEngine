// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/GenericOctreePublic.h"
#include "NavigationSystemTypes.h"
#include "EngineStats.h"
#include "AI/Navigation/NavigationElement.h"
#include "AI/Navigation/NavigationRelevantData.h"
#include "Math/GenericOctree.h"
#include "HAL/LowLevelMemTracker.h"

class INavRelevantInterface;
class FNavigationOctree;
class UActorComponent;
enum class ENavDataGatheringMode : uint8;
struct FNavigationRelevantDataFilter;
typedef FNavigationRelevantDataFilter FNavigationOctreeFilter;

LLM_DECLARE_TAG(NavigationOctree);

struct FNavigationOctreeElement
{
	FBoxSphereBounds Bounds;
	TSharedRef<FNavigationRelevantData, ESPMode::ThreadSafe> Data;

	explicit FNavigationOctreeElement(const TSharedRef<const FNavigationElement>& SourceObject)
		: Bounds(SourceObject.Get().GetBounds())
		, Data(MakeShared<FNavigationRelevantData>(SourceObject))
	{
	}

	FNavigationOctreeElement(const FNavigationOctreeElement& Other)
		: Bounds(Other.Bounds)
		, Data(Other.Data)
	{}

	FNavigationOctreeElement(FNavigationOctreeElement&& Other) noexcept
		: Bounds(MoveTemp(Other.Bounds))
		, Data(MoveTemp(Other.Data))
	{
	}

	FNavigationOctreeElement& operator=(FNavigationOctreeElement&& Other)
	{
		if (this != &Other)
		{
			this->~FNavigationOctreeElement();
			new(this) FNavigationOctreeElement(Forward<FNavigationOctreeElement>(Other));
		}
		return *this;
	}
	
	FNavigationOctreeElement& operator=(const FNavigationOctreeElement& Other)
	{
		if (this != &Other)
		{
			this->~FNavigationOctreeElement();
			new(this) FNavigationOctreeElement(Other);
		}
		return *this;
	}

	FORCEINLINE bool IsEmpty() const
	{
		const FBox BBox = Bounds.GetBox();
		return Data->IsEmpty() && (BBox.IsValid == 0 || BBox.GetSize().IsNearlyZero());
	}

	FORCEINLINE bool IsMatchingFilter(const FNavigationOctreeFilter& Filter) const
	{
		return Data->IsMatchingFilter(Filter);
	}

	/** 
	 *	Retrieves Modifier if it doesn't contain any "Meta Navigation Areas". 
	 *	If it does, then retrieves a copy with meta areas substituted with
	 *	appropriate non-meta areas, depending on NavAgent.
	 */
	FORCEINLINE FCompositeNavModifier GetModifierForAgent(const FNavAgentProperties* NavAgent = nullptr) const
	{ 
		return Data->GetModifierForAgent(NavAgent);
	}

	FORCEINLINE bool ShouldUseGeometry(const FNavDataConfig& NavConfig) const
	{ 
		return !Data->ShouldUseGeometryDelegate.IsBound() || Data->ShouldUseGeometryDelegate.Execute(&NavConfig);
	}

	FORCEINLINE int32 GetAllocatedSize() const
	{
		return (int32)Data->GetAllocatedSize();
	}

	FORCEINLINE void Shrink()
	{
		Data->Shrink();
	}

	FORCEINLINE void ValidateAndShrink()
	{
		Data->ValidateAndShrink();
	}

	const TSharedRef<const FNavigationElement>& GetSourceElement() const
	{
		return Data->SourceElement;
	}
};

struct FNavigationOctreeSemantics
{
	typedef TOctree2<FNavigationOctreeElement, FNavigationOctreeSemantics> FOctree;
	enum { MaxElementsPerLeaf = 16 };
	enum { MinInclusiveElementsPerNode = 7 };
	enum { MaxNodeDepth = 12 };

	typedef TInlineAllocator<MaxElementsPerLeaf> ElementAllocator;

	FORCEINLINE static const FBoxSphereBounds& GetBoundingBox(const FNavigationOctreeElement& NavData)
	{
		return NavData.Bounds;
	}

	FORCEINLINE static bool AreElementsEqual(const FNavigationOctreeElement& A, const FNavigationOctreeElement& B)
	{
		return A.Data.Get().SourceElement.Get().GetHandle() == B.Data.Get().SourceElement.Get().GetHandle();
	}

	FORCEINLINE static void ApplyOffset(FNavigationOctreeElement& Element, const FVector& InOffset)
	{
		ensureMsgf(false, TEXT("Not implemented yet"));
	}

#if NAVSYS_DEBUG
	FORCENOINLINE 
#endif // NAVSYS_DEBUG
	static void SetElementId(FOctree& OctreeOwner, const FNavigationOctreeElement& Element, FOctreeElementId2 Id);
};

class FNavigationOctree : public TOctree2<FNavigationOctreeElement, FNavigationOctreeSemantics>, public TSharedFromThis<FNavigationOctree, ESPMode::ThreadSafe>
{
public:
	DECLARE_DELEGATE_TwoParams(FGeometryExportDelegate, const FNavigationElement&, FNavigationRelevantData&);
    FGeometryExportDelegate GeometryExportDelegate;

	enum ENavGeometryStoringMode {
		SkipNavGeometry,
		StoreNavGeometry,
	};

	/**
	 * Adds an element to the octree.
	 * @param Element - The element to add.
	 */
	inline void AddElement(const FNavigationOctreeElement& Element)
	{
		LLM_SCOPE_BYTAG(NavigationOctree);

		DEC_MEMORY_STAT_BY(STAT_NavigationMemory, OctreeSizeBytes);
		DEC_MEMORY_STAT_BY(STAT_Navigation_CollisionTreeMemory, OctreeSizeBytes);
		TOctree2<FNavigationOctreeElement, FNavigationOctreeSemantics>::AddElement(Element);
		OctreeSizeBytes = GetSizeBytes();
		INC_MEMORY_STAT_BY(STAT_NavigationMemory, OctreeSizeBytes);
		INC_MEMORY_STAT_BY(STAT_Navigation_CollisionTreeMemory, OctreeSizeBytes);
	}

	/**
	 * Removes an element from the octree.
	 * @param ElementId - The element to remove from the octree.
	 */
	inline void RemoveElement(FOctreeElementId2 ElementId)
	{
		DEC_MEMORY_STAT_BY(STAT_NavigationMemory, OctreeSizeBytes);
		DEC_MEMORY_STAT_BY(STAT_Navigation_CollisionTreeMemory, OctreeSizeBytes);
		TOctree2<FNavigationOctreeElement, FNavigationOctreeSemantics>::RemoveElement(ElementId);
		OctreeSizeBytes = GetSizeBytes();
		INC_MEMORY_STAT_BY(STAT_NavigationMemory, OctreeSizeBytes);
		INC_MEMORY_STAT_BY(STAT_Navigation_CollisionTreeMemory, OctreeSizeBytes);
	}

	NAVIGATIONSYSTEM_API FNavigationOctree(const FVector& Origin, FVector::FReal Radius);
	NAVIGATIONSYSTEM_API virtual ~FNavigationOctree();

	/** Add new node and fill it with navigation export data */
	NAVIGATIONSYSTEM_API void AddNode(const FBox& Bounds, FNavigationOctreeElement& OctreeElement);

	/** Append new data to existing node */
	NAVIGATIONSYSTEM_API void AppendToNode(const FOctreeElementId2& Id, const TSharedRef<const FNavigationElement>& ElementRef, const FBox& Bounds, FNavigationOctreeElement& Data);

	/** Updates element bounds remove/add operation */
	NAVIGATIONSYSTEM_API void UpdateNode(const FOctreeElementId2& Id, const FBox& NewBounds);

	/** Remove node */
	NAVIGATIONSYSTEM_API void RemoveNode(const FOctreeElementId2& Id);

	NAVIGATIONSYSTEM_API void SetNavigableGeometryStoringMode(ENavGeometryStoringMode NavGeometryMode);

	NAVIGATIONSYSTEM_API const FNavigationRelevantData* GetDataForID(const FOctreeElementId2& Id) const;

	NAVIGATIONSYSTEM_API FNavigationRelevantData* GetMutableDataForID(const FOctreeElementId2& Id);

	ENavGeometryStoringMode GetNavGeometryStoringMode() const
	{
		return bGatherGeometry ? StoreNavGeometry : SkipNavGeometry;
	}

	NAVIGATIONSYSTEM_API void SetDataGatheringMode(ENavDataGatheringModeConfig Mode);

	// Lazy data gathering methods
	NAVIGATIONSYSTEM_API bool IsLazyGathering(const FNavigationElement& NavigationElement) const;

	NAVIGATIONSYSTEM_API void DemandLazyDataGathering(FNavigationRelevantData& ElementData);

	NAVIGATIONSYSTEM_API void DemandChildLazyDataGathering(FNavigationRelevantData& ElementData, const FNavigationElement& ChildElement) const;

#if !UE_BUILD_SHIPPING
	NAVIGATIONSYSTEM_API void SetGatheringNavModifiersTimeLimitWarning(const float Threshold);
#endif // !UE_BUILD_SHIPPING
protected:
	friend struct FNavigationOctreeController;
	friend struct FNavigationOctreeSemantics;

	NAVIGATIONSYSTEM_API void SetElementIdImpl(FNavigationElementHandle ElementHandle, FOctreeElementId2 Id);

	TMap<FNavigationElementHandle, FOctreeElementId2> ElementToOctreeId;
	ENavDataGatheringMode DefaultGeometryGatheringMode;
	uint32 bGatherGeometry : 1;
	uint32 NodesMemory;
#if !UE_BUILD_SHIPPING
	float GatheringNavModifiersTimeLimitWarning;
#endif // !UE_BUILD_SHIPPING
private:
	SIZE_T OctreeSizeBytes = 0;
};
