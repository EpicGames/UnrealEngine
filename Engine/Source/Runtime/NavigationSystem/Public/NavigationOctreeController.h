// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "NavigationOctree.h"
#include "AI/Navigation/NavigationDirtyElement.h"


struct FNavigationDirtyElementKeyFunctions : BaseKeyFuncs<FNavigationDirtyElement, FNavigationElementHandle, /*bInAllowDuplicateKeys*/false>
{
	static FNavigationElementHandle GetSetKey(ElementInitType Element)
	{
		return Element.NavigationElement->GetHandle();
	}

	static bool Matches(KeyInitType A, KeyInitType B)
	{
		return A == B;
	}

	static uint32 GetKeyHash(KeyInitType Key)
	{
		return GetTypeHash(Key);
	}
};

struct FNavigationOctreeController
{
	enum EOctreeUpdateMode
	{
		OctreeUpdate_Default = 0,						// regular update, mark dirty areas depending on exported content
		OctreeUpdate_Geometry = 1,						// full update, mark dirty areas for geometry rebuild
		OctreeUpdate_Modifiers = 2,						// quick update, mark dirty areas for modifier rebuild
		OctreeUpdate_Refresh = 4,						// update is used for refresh, don't invalidate pending queue
		OctreeUpdate_ParentChain = 8,					// update child nodes, don't remove anything
	};

	TSet<FNavigationDirtyElement, FNavigationDirtyElementKeyFunctions> PendingUpdates;
	TSharedPtr<FNavigationOctree, ESPMode::ThreadSafe> NavOctree;

	/** if set, navoctree updates are ignored, use with caution! */
	uint8 bNavOctreeLock : 1 = false;

	inline void SetNavigationOctreeLock(bool bLock);

	NAVIGATIONSYSTEM_API bool HasPendingUpdateForElement(FNavigationElementHandle Element) const;

	inline void RemoveNode(FOctreeElementId2 ElementId, FNavigationElementHandle GetHandle);

	NAVIGATIONSYSTEM_API void SetNavigableGeometryStoringMode(FNavigationOctree::ENavGeometryStoringMode NavGeometryMode);

	NAVIGATIONSYSTEM_API void Reset();

	inline const FNavigationOctree* GetOctree() const;
	inline FNavigationOctree* GetMutableOctree();

	inline const FOctreeElementId2* GetNavOctreeIdForElement(FNavigationElementHandle Element) const;

	NAVIGATIONSYSTEM_API bool GetNavOctreeElementData(FNavigationElementHandle Element, ENavigationDirtyFlag& DirtyFlags, FBox& DirtyBounds);

	NAVIGATIONSYSTEM_API const FNavigationRelevantData* GetDataForElement(FNavigationElementHandle Element) const;

	NAVIGATIONSYSTEM_API FNavigationRelevantData* GetMutableDataForElement(FNavigationElementHandle Element);

	inline bool HasElementNavOctreeId(const FNavigationElementHandle Element) const;

	inline bool IsNavigationOctreeLocked() const;

	/** basically says if navoctree has been created already */
	bool IsValid() const { return NavOctree.IsValid(); }

	inline bool IsValidElement(const FOctreeElementId2* ElementId) const;
	inline bool IsValidElement(const FOctreeElementId2& ElementId) const;

	bool IsEmpty() const
	{
		return (IsValid() == false) || NavOctree->GetSizeBytes() == 0;
	}

	inline void AddChild(FNavigationElementHandle Parent, const TSharedRef<const FNavigationElement>& Child);
	inline void RemoveChild(FNavigationElementHandle Parent, const TSharedRef<const FNavigationElement>& Child);
	inline void GetChildren(FNavigationElementHandle Parent, TArray<const TSharedRef<const FNavigationElement>>& OutChildren) const;

private:
	/** Map of all elements that are tied to indexed navigation parent */
	TMultiMap<FNavigationElementHandle, const TSharedRef<const FNavigationElement>> OctreeParentChildNodesMap;
};

//----------------------------------------------------------------------//
// inlines
//----------------------------------------------------------------------//

inline const FOctreeElementId2* FNavigationOctreeController::GetNavOctreeIdForElement(const FNavigationElementHandle Element) const
{ 
	return NavOctree.IsValid()
		? NavOctree->ElementToOctreeId.Find(Element)
		: nullptr;
}

inline bool FNavigationOctreeController::HasElementNavOctreeId(const FNavigationElementHandle Element) const
{
	return NavOctree.IsValid() && (NavOctree->ElementToOctreeId.Find(Element) != nullptr);
}

inline void FNavigationOctreeController::RemoveNode(const FOctreeElementId2 ElementId, const FNavigationElementHandle ElementHandle)
{ 
	if (NavOctree.IsValid())
	{
		NavOctree->RemoveNode(ElementId);
		NavOctree->ElementToOctreeId.Remove(ElementHandle);
	}
}

inline const FNavigationOctree* FNavigationOctreeController::GetOctree() const
{ 
	return NavOctree.Get(); 
}

inline FNavigationOctree* FNavigationOctreeController::GetMutableOctree()
{ 
	return NavOctree.Get(); 
}

inline bool FNavigationOctreeController::IsNavigationOctreeLocked() const
{ 
	return bNavOctreeLock; 
}

inline void FNavigationOctreeController::SetNavigationOctreeLock(bool bLock) 
{ 
	bNavOctreeLock = bLock; 
}

inline bool FNavigationOctreeController::IsValidElement(const FOctreeElementId2* ElementId) const
{
	return ElementId && IsValidElement(*ElementId);
}

inline bool FNavigationOctreeController::IsValidElement(const FOctreeElementId2& ElementId) const
{
	return IsValid() && NavOctree->IsValidElementId(ElementId);
}

void FNavigationOctreeController::AddChild(const FNavigationElementHandle Parent, const TSharedRef<const FNavigationElement>& Child)
{
	OctreeParentChildNodesMap.AddUnique(Parent, Child);
}

void FNavigationOctreeController::RemoveChild(const FNavigationElementHandle Parent, const TSharedRef<const FNavigationElement>& Child)
{
	// Compare by handle rather than pointer identity so that callers can pass a
	// newly-allocated element that represents the same navigation object.
	const FNavigationElementHandle ChildHandle = Child->GetHandle();
	for (auto It = OctreeParentChildNodesMap.CreateKeyIterator(Parent); It; ++It)
	{
		if (It.Value()->GetHandle() == ChildHandle)
		{
			It.RemoveCurrent();
			return;
		}
	}
}

void FNavigationOctreeController::GetChildren(const FNavigationElementHandle Parent, TArray<const TSharedRef<const FNavigationElement>>& OutChildren) const
{
	OctreeParentChildNodesMap.MultiFind(Parent, OutChildren);
}
