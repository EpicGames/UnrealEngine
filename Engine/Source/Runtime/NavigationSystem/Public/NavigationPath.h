// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "NavigationData.h"
#include "NavigationPath.generated.h"

class APlayerController;
class UCanvas;
class UNavigationPath;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnNavigationPathUpdated, UNavigationPath*, AffectedPath, TEnumAsByte<ENavPathEvent::Type>, PathEvent);

/**
 *	UObject wrapper for FNavigationPath
 */

UCLASS(BlueprintType, MinimalAPI)
class UNavigationPath : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintAssignable)
	FOnNavigationPathUpdated PathUpdatedNotifier;

	UPROPERTY(BlueprintReadOnly, Category = Navigation)
	TArray<FVector> PathPoints;

	UPROPERTY(BlueprintReadOnly, Category = Navigation)
	ENavigationOptionFlag RecalculateOnInvalidation = ENavigationOptionFlag::Default;

private:	
	uint32 bIsValid : 1;
	uint32 bDebugDrawingEnabled : 1;
	FColor DebugDrawingColor;

	FDelegateHandle DrawDebugDelegateHandle;

protected:
	FNavPathSharedPtr SharedPath;

	FNavigationPath::FPathObserverDelegate::FDelegate PathObserver;
	FDelegateHandle PathObserverDelegateHandle;

public:

	// UObject begin
	NAVIGATIONSYSTEM_API virtual void BeginDestroy() override;
	// UObject end

	UFUNCTION(BlueprintCallable, Category = "AI|Debug")
	NAVIGATIONSYSTEM_API FString GetDebugString() const;

	UFUNCTION(BlueprintCallable, Category = "AI|Debug")
	NAVIGATIONSYSTEM_API void EnableDebugDrawing(bool bShouldDrawDebugData, FLinearColor PathColor = FLinearColor::White);

	/** if enabled path will request recalculation if it gets invalidated due to a change to underlying navigation */
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	NAVIGATIONSYSTEM_API void EnableRecalculationOnInvalidation(ENavigationOptionFlag DoRecalculation);

	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	NAVIGATIONSYSTEM_API double GetPathLength() const;

	/** returns the remaining length of the path from Position, including the distance from Position to the NextPathPoint */
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	NAVIGATIONSYSTEM_API double GetLengthFromPosition(FVector Position, int32 NextPathPointIndex) const;

	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	NAVIGATIONSYSTEM_API double GetPathCost() const;

	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	NAVIGATIONSYSTEM_API bool IsPartial() const;

	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	NAVIGATIONSYSTEM_API bool IsValid() const;

	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	NAVIGATIONSYSTEM_API bool IsStringPulled() const;
		
	NAVIGATIONSYSTEM_API void SetPath(FNavPathSharedPtr NewSharedPath);
	FNavPathSharedPtr GetPath() { return SharedPath; }

protected:
	NAVIGATIONSYSTEM_API void DrawDebug(UCanvas* Canvas, APlayerController*);
	NAVIGATIONSYSTEM_API void OnPathEvent(FNavigationPath* Path, ENavPathEvent::Type PathEvent);

	NAVIGATIONSYSTEM_API void SetPathPointsFromPath(FNavigationPath& NativePath);
};
