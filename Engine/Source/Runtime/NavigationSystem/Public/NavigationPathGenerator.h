// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "AI/Navigation/NavigationTypes.h"
#include "NavigationPathGenerator.generated.h"

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UNavigationPathGenerator : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class INavigationPathGenerator
{
	GENERATED_IINTERFACE_BODY()


	/** 
	 *	Retrieved path generated for specified navigation Agent
	 */
	virtual FNavPathSharedPtr GetGeneratedPath(class INavAgentInterface* Agent) PURE_VIRTUAL(INavigationPathGenerator::GetGeneratedPath, return FNavPathSharedPtr(NULL););
};
