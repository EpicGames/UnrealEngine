// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "UObject/WeakObjectPtr.h"
#include "Misc/CoreMisc.h"
#include "Misc/CoreDelegates.h"
#include "AI/Navigation/NavigationTypes.h"
#include "AI/Navigation/NavigationDirtyElement.h"
#include "AI/Navigation/NavigationInvokerPriority.h"
#include "NavigationSystemTypes.h"
#include "NavigationData.h"
#include "AI/NavigationSystemBase.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "NavigationOctree.h"
#include "AI/NavigationSystemConfig.h"
#include "NavigationOctreeController.h"
#include "NavigationDirtyAreasController.h"
#include "Math/MovingWindowAverageFast.h"
#include "AI/Navigation/NavigationBounds.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "NavigationSystem.generated.h"


class AController;
class ANavMeshBoundsVolume;
class AWorldSettings;
class FEdMode;
class FNavDataGenerator;
class INavigationInvokerInterface;
class INavLinkCustomInterface;
class INavRelevantInterface;
class UCrowdManagerBase;
class UNavArea;
class UNavigationPath;
class UNavigationObjectRepository;
class UNavigationSystemModuleConfig;
struct FDebugText;
struct FNavigationRelevantData;
struct FNavigationOctreeElement;
struct FNavigationElement;
struct FAssetCompileData;

#if !UE_BUILD_SHIPPING
#define ALLOW_TIME_SLICE_DEBUG 1
#else
#define ALLOW_TIME_SLICE_DEBUG 0
#endif

/** delegate to let interested parties know that new nav area class has been registered */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnNavAreaChanged, const UClass* /*AreaClass*/);

/** Delegate to let interested parties know that Nav Data has been registered */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnNavDataGenericEvent, ANavigationData*, NavData);

DECLARE_MULTICAST_DELEGATE(FOnNavigationInitDone);

namespace NavigationDebugDrawing
{
	extern const NAVIGATIONSYSTEM_API float PathLineThickness;
	extern const NAVIGATIONSYSTEM_API FVector PathOffset;
	extern const NAVIGATIONSYSTEM_API FVector PathNodeBoxExtent;
}

namespace FNavigationSystem
{	
	/** 
	 * Used to construct an ANavigationData instance for specified navigation data agent 
	 */
	typedef ANavigationData* (*FNavigationDataInstanceCreator)(UWorld*, const FNavDataConfig&);

	struct FCustomLinkOwnerInfo
	{
		FWeakObjectPtr LinkOwner;
		INavLinkCustomInterface* LinkInterface;

		FCustomLinkOwnerInfo() : LinkInterface(nullptr) {}
		NAVIGATIONSYSTEM_API FCustomLinkOwnerInfo(INavLinkCustomInterface* Link);

		bool IsValid() const { return LinkOwner.IsValid(); }
	};

	bool NAVIGATIONSYSTEM_API ShouldLoadNavigationOnClient(ANavigationData& NavData);

	void NAVIGATIONSYSTEM_API MakeAllComponentsNeverAffectNav(AActor& Actor);

#if ALLOW_TIME_SLICE_DEBUG
	extern NAVIGATIONSYSTEM_API const FName DebugTimeSliceDefaultSectionName;
#endif // ALLOW_TIME_SLICE_DEBUG
}

struct FNavigationSystemExec: public FSelfRegisteringExec
{
protected:
	//~ Begin FExec Interface
	virtual bool Exec_Runtime(UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	//~ End FExec Interface
};

namespace ENavigationBuildLock
{
	enum Type
	{
		NoUpdateInEditor = 1 << 1,		// editor doesn't allow automatic updates
		NoUpdateInPIE = 1 << 2,			// PIE doesn't allow automatic updates
		InitialLock = 1 << 3,			// initial lock, release manually after levels are ready for rebuild (e.g. streaming)
		Custom = 1 << 4,
		AsyncLoadLock = 1 << 5			// used in editor worlds to prevent automatic update when loading a map using async loading 
	};
}


// Use this just  before a call to TestTimeSliceFinished() to allow the time slice logging to use a debug name for the section of code being timed.
#if ALLOW_TIME_SLICE_DEBUG
#define MARK_TIMESLICE_SECTION_DEBUG(TIME_SLICER, TIME_SLICE_FNAME) \
static const FName TIME_SLICE_FNAME(TEXT(#TIME_SLICE_FNAME)); \
TIME_SLICER.DebugSetSectionName(TIME_SLICE_FNAME);
#else
#define MARK_TIMESLICE_SECTION_DEBUG(TIME_SLICER, TIME_SLICE_FNAME) ;
#endif

class FNavRegenTimeSlicer
{
public:
	/** Setup the initial values for a time slice. This can be called on an instance after TestTimeSliceFinished() has returned true and EndTimeSliceAndAdjustDuration() has been called */
	NAVIGATIONSYSTEM_API void SetupTimeSlice(double SliceDuration);

	/** 
	 *  Starts the time slice, this can be called multiple times as long as EndTimeSliceAndAdjustDuration() is called between each call.
	 *  StartTimeSlice should not be called after TestTimeSliceFinished() has returned true
	 */
	NAVIGATIONSYSTEM_API void StartTimeSlice();

	/** 
	 *  Useful when multiple sections of code need to be timesliced per frame using the same time slice duration that do not necessarily occur concurrently.
	 *  This ends the time sliced code section and adjusts the RemainingDuration based on the time used between calls to StartTimeSlice and the last call to TestTimeSliceFinished.
	 *  Note the actual time slice is not tested in this function. Thats done in TestTimeSliceFinished!
	 *  This can be called multiple times as long as StartTimeSlice() is called before EndTimeSliceAndAdjustDuration().
	 *  EndTimeSliceAndAdjustDuration can be called after TestTimeSliceFinished() has returned true in this case the RemainingDuration will just be zero
	 */
	NAVIGATIONSYSTEM_API void EndTimeSliceAndAdjustDuration();
	double GetStartTime() const { return StartTime; }
	NAVIGATIONSYSTEM_API bool TestTimeSliceFinished() const;

	//* Returns the cached result of calling TestTimeSliceFinished, false by default */
	bool IsTimeSliceFinishedCached() const { return bTimeSliceFinishedCached; }
	double GetRemainingDuration() const { return RemainingDuration; }
	double GetRemainingDurationFraction() const { return OriginalDuration > 0. ? RemainingDuration / OriginalDuration : 0.; }


#if ALLOW_TIME_SLICE_DEBUG
	/*
	 * Sets data used for debugging time slices that are taking too long to process.
	 * @param LongTimeSliceFunction function called when a time slice is taking too long to process.
	 * @param LongTimeSliceDuration when a time slice takes longer than this duration LongTimeSliceFunction will be called.
	 */
	NAVIGATIONSYSTEM_API void DebugSetLongTimeSliceData(TFunction<void(FName, double)> LongTimeSliceFunction, double LongTimeSliceDuration) const;
	NAVIGATIONSYSTEM_API void DebugResetLongTimeSliceFunction() const;

	/**
	 *  Sets the debug name for a time sliced section of code.
	 *  Do not call this directly use MARK_TIMESLICE_SECTION_DEBUG
	 */
	void DebugSetSectionName(FName InDebugSectionName) const
	{
		DebugSectionName = InDebugSectionName;
	}
#endif // ALLOW_TIME_SLICE_DEBUG

protected:
	double OriginalDuration = 0.;
	double RemainingDuration = 0.;
	double StartTime = 0.;
	mutable double TimeLastTested = 0.;
	mutable bool bTimeSliceFinishedCached = false;

#if ALLOW_TIME_SLICE_DEBUG
	mutable TFunction<void(FName, double)> DebugLongTimeSliceFunction;
	mutable double DebugLongTimeSliceDuration = 0.;
	mutable FName DebugSectionName = FNavigationSystem::DebugTimeSliceDefaultSectionName;
#endif
};

#if !UE_BUILD_SHIPPING
struct FTileHistoryData
{
	/** Tile coordinates **/
	int32 TileX = 0;
	int32 TileY = 0;
	
	/** Accumulated time spent processing the tile (seconds). */
	float TileRegenTime = 0.f;

	/** Time between the tile was requested and when it's finished building (seconds). */
	float TileWaitTime = 0.f;

	/** Frame when tile processing started. */
	int64 StartRegenFrame = 0;

	/** Frame when tile processing ended. */
	int64 EndRegenFrame = 0;
};
#endif // !UE_BUILD_SHIPPING

class FNavRegenTimeSliceManager
{
public:
	NAVIGATIONSYSTEM_API FNavRegenTimeSliceManager();

	void PushTileRegenTime(double NewTime) { MovingWindowTileRegenTime.PushValue(NewTime);  }

	double GetAverageTileRegenTime() const { return MovingWindowTileRegenTime.GetAverage();  }

	double GetAverageDeltaTime() const { return MovingWindowDeltaTime.GetAverage(); }

	NAVIGATIONSYSTEM_API void ResetTileWaitTimeArrays(const TArray<TObjectPtr<ANavigationData>>& NavDataSet);
	NAVIGATIONSYSTEM_API void PushTileWaitTime(const int32 NavDataIndex, const double NewTime);

#if !UE_BUILD_SHIPPING
	NAVIGATIONSYSTEM_API void ResetTileHistoryData(const TArray<TObjectPtr<ANavigationData>>& NavDataSet);
	NAVIGATIONSYSTEM_API void PushTileHistoryData(const int32 NavDataIndex, const FTileHistoryData& TileData);
#endif // UE_BUILD_SHIPPING

	NAVIGATIONSYSTEM_API double GetAverageTileWaitTime(const int32 NavDataIndex) const;
	NAVIGATIONSYSTEM_API void ResetTileWaitTime(const int32 NavDataIndex);

	bool DoTimeSlicedUpdate() const { return bDoTimeSlicedUpdate; }

	NAVIGATIONSYSTEM_API void CalcAverageDeltaTime(uint64 FrameNum);

	NAVIGATIONSYSTEM_API void CalcTimeSliceDuration(const TArray<TObjectPtr<ANavigationData>>& NavDataSet, int32 NumTilesToRegen, const TArray<double>& CurrentTileRegenDurations);

	NAVIGATIONSYSTEM_API void SetMinTimeSliceDuration(double NewMinTimeSliceDuration);

	NAVIGATIONSYSTEM_API void SetMaxTimeSliceDuration(double NewMaxTimeSliceDuration);

	NAVIGATIONSYSTEM_API void SetMaxDesiredTileRegenDuration(float NewMaxDesiredTileRegenDuration);

	int32 GetNavDataIdx() const { return NavDataIdx;  }
	void SetNavDataIdx(int32 InNavDataIdx) { NavDataIdx = InNavDataIdx; }

	FNavRegenTimeSlicer& GetTimeSlicer() { return TimeSlicer; }
	const FNavRegenTimeSlicer& GetTimeSlicer() const { return TimeSlicer; }

#if !UE_BUILD_SHIPPING	
	NAVIGATIONSYSTEM_API void LogTileStatistics(const TArray<TObjectPtr<ANavigationData>>& NavDataSet) const;
#endif // !UE_BUILD_SHIPPING
	
protected:
	FNavRegenTimeSlicer TimeSlicer;

	/** Used to calculate the moving window average of the actual time spent inside functions used to regenerate a tile, this is processing time not actual time over multiple frames */
	FMovingWindowAverageFast<double, 256> MovingWindowTileRegenTime;

	/** Used to calculate the actual moving window delta time */
	FMovingWindowAverageFast<double, 256> MovingWindowDeltaTime;

	/** Average tile wait time per NavDataIndex */
	TArray<TArray<double>> TileWaitTimes;

#if !UE_BUILD_SHIPPING
	/** Tile processing time per NavDataIndex */
	TArray<TArray<FTileHistoryData>> TileHistoryData;
	double TileHistoryStartTime = 0;
#endif // UE_BUILD_SHIPPING

	/** If there are enough tiles to process this in the Min Time Slice Duration */
	double MinTimeSliceDuration;

	/** The max Desired Time Slice Duration */
	double MaxTimeSliceDuration;

	uint64 FrameNumOld;

	/** The max real world desired time to Regen all the tiles in PendingDirtyTiles,
	 *  Note it could take longer than this, as the time slice is clamped per frame between
	 *	MinTimeSliceDuration and MaxTimeSliceDuration.
	 */
	float MaxDesiredTileRegenDuration;

	double TimeLastCall;

	int32 NavDataIdx;

	bool bDoTimeSlicedUpdate;
};

UCLASS(Within=World, config=Engine, defaultconfig, MinimalAPI)
class UNavigationSystemV1 : public UNavigationSystemBase
{
	GENERATED_BODY()

	friend UNavigationSystemModuleConfig;

public:
	NAVIGATIONSYSTEM_API UNavigationSystemV1(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	virtual ~UNavigationSystemV1() = default;

	NAVIGATIONSYSTEM_API void GatherDebugLabels(TArray<FString>& InOutDebugLabels) const;

	UPROPERTY(Transient)
	TObjectPtr<ANavigationData> MainNavData;

	/** special navigation data for managing direct paths, not part of NavDataSet! */
	UPROPERTY(Transient)
	TObjectPtr<ANavigationData> AbstractNavData;

protected:
	/** If not None indicates which of navigation datas and supported agents are
	 * going to be used as the default ones. If navigation agent of this type does
	 * not exist or is not enabled then the first available nav data will be used
	 * as the default one */
	UPROPERTY(config, EditAnywhere, BlueprintReadOnly, Category = Navigation)
	FName DefaultAgentName;

	UPROPERTY(config, EditAnywhere, BlueprintReadOnly, Category = Navigation)
	TSoftClassPtr<UCrowdManagerBase> CrowdManagerClass;

	/** Should navigation system spawn default Navigation Data when there's none and there are navigation bounds present? */
	UPROPERTY(config, EditAnywhere, Category=NavigationSystem)
	uint32 bAutoCreateNavigationData:1;

	/** If true will try to spawn the navigation data instance in the sublevel with navigation bounds, if false it will spawn in the persistent level */
	UPROPERTY(config, EditAnywhere, Category = NavigationSystem)
	uint32 bSpawnNavDataInNavBoundsLevel:1;

	/** If false, will not create nav collision when connecting as a client */
	UPROPERTY(config, EditAnywhere, Category=NavigationSystem)
	uint32 bAllowClientSideNavigation:1;

	/** If true, games should ignore navigation data inside loaded sublevels */
	UPROPERTY(config, EditAnywhere, Category = NavigationSystem)
	uint32 bShouldDiscardSubLevelNavData:1;

	/** If true, will update navigation even when the game is paused */
	UPROPERTY(config, EditAnywhere, Category=NavigationSystem)
	uint32 bTickWhilePaused:1;

	/** gets set to true if gathering navigation data (like in navoctree) is required due to the need of navigation generation 
	 *	Is always true in Editor Mode. In other modes it depends on bRebuildAtRuntime of every required NavigationData class' CDO
	 */
	UPROPERTY()
	uint32 bSupportRebuilding : 1; 

	UPROPERTY()
	uint32 bDirtyNavDataPackageOnNavigationGenerationFinished : 1;

#if WITH_EDITORONLY_DATA	
	/** In editor worlds, wait for async loading to finish before allowing to build navigation automatically (true by default). */
	UPROPERTY(config, EditAnywhere, Category=NavigationSystem)
	uint32 bWaitForAsyncLoadingBeforeBuildingNavigationAutomatically : 1 = true;
#endif	

public:
	/** if set to true will result navigation system not rebuild navigation until 
	 *	a call to ReleaseInitialBuildingLock() is called. Does not influence 
	 *	editor-time generation (i.e. does influence PIE and Game).
	 *	Defaults to false.*/
	UPROPERTY(config, EditAnywhere, Category=NavigationSystem)
	uint32 bInitialBuildingLocked:1;

	/** If set to true (default) navigation will be generated only within special navigation 
	 *	bounds volumes (like ANavMeshBoundsVolume). Set to false means navigation should be generated
	 *	everywhere.
	 */
	// @todo removing it from edition since it's currently broken and I'm not sure we want that at all
	// since I'm not sure we can make it efficient in a generic case
	//UPROPERTY(config, EditAnywhere, Category=NavigationSystem)
	uint32 bWholeWorldNavigable:1;

	/** false by default, if set to true will result in not caring about nav agent height 
	 *	when trying to match navigation data to passed in nav agent */
	UPROPERTY(config, EditAnywhere, Category=NavigationSystem)
	uint32 bSkipAgentHeightCheckWhenPickingNavData:1;

	/** Warnings are logged if exporting the navigation collision for an object exceed this triangle count.
	 * Use -1 to disable. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = NavigationSystem)
	int32 GeometryExportTriangleCountWarningThreshold = 200000;
	
protected:
	/** If set to true navigation will be generated only around registered "navigation enforcers"
	*	This has a range of consequences (including how navigation octree operates) so it needs to
	*	be a conscious decision.
	*	Once enabled results in whole world being navigable.
	*	@see RegisterNavigationInvoker
	*/
	UPROPERTY(EditDefaultsOnly, Category = "Navigation Enforcing", config)
	uint32 bGenerateNavigationOnlyAroundNavigationInvokers:1;

	/** Minimal time, in seconds, between active tiles set update */
	UPROPERTY(EditAnywhere, Category = "Navigation Enforcing", meta = (ClampMin = "0.1", UIMin = "0.1", EditCondition = "bGenerateNavigationOnlyAroundNavigationInvokers"), config)
	float ActiveTilesUpdateInterval;

	/** When in use, invokers farther away from any invoker seed will be ignored (set to -1 to disable). */
	UPROPERTY(EditAnywhere, Category = "Navigation Enforcing", meta = (EditCondition = "bGenerateNavigationOnlyAroundNavigationInvokers"), config)
	double InvokersMaximumDistanceFromSeed = -1;
	
	/** Sets how navigation data should be gathered when building collision information */
	UPROPERTY(EditDefaultsOnly, Category = "NavigationSystem", config)
	ENavDataGatheringModeConfig DataGatheringMode;

	/** -1 by default, if set to a positive value dirty areas with any dimensions in 2d over the threshold created at runtime will be logged */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = NavigationSystem, meta = (ClampMin = "-1.0", UIMin = "-1.0"))
	float DirtyAreaWarningSizeThreshold;

	/** -1.0f by default, if set to a positive value, all calls to GetNavigationData will be timed and compared to it. 
	*	Over the limit calls will be logged as warnings. 
	*	In seconds. Non-shipping build only.
	*/
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = NavigationSystem, meta = (ClampMin = "-1.0", UIMin = "-1.0"))
	float GatheringNavModifiersWarningLimitTime;

	/** List of agents types supported by this navigation system */
	UPROPERTY(config, EditAnywhere, Category = Agents)
	TArray<FNavDataConfig> SupportedAgents;

	/** NavigationSystem's properties in Project Settings define all possible supported agents,
	 *	but a specific navigation system can choose to support only a subset of agents. Set via 
	 *	NavigationSystemConfig */
	UPROPERTY(config, EditAnywhere, Category = Agents)
	FNavAgentSelector SupportedAgentsMask;

public:
	/** Bounds of tiles to be built */
	UPROPERTY(Transient)
	FBox BuildBounds;

	UPROPERTY(Transient)
	TArray<TObjectPtr<ANavigationData>> NavDataSet;

	UPROPERTY(Transient)
	TArray<TObjectPtr<ANavigationData>> NavDataRegistrationQueue;

	// List of pending navigation bounds update requests (add, remove, update size)
	TArray<FNavigationBoundsUpdateRequest> PendingNavBoundsUpdates;

 	UPROPERTY(/*BlueprintAssignable, */Transient)
	FOnNavDataGenericEvent OnNavDataRegisteredEvent;

	UPROPERTY(BlueprintAssignable, Transient, meta = (displayname = OnNavigationGenerationFinished))
	FOnNavDataGenericEvent OnNavigationGenerationFinishedDelegate;

	FOnNavigationInitDone OnNavigationInitDone;
	
private:
	TWeakObjectPtr<UCrowdManagerBase> CrowdManager;
	
	/** set to true when navigation processing was blocked due to missing nav bounds */
	uint32 bNavDataRemovedDueToMissingNavBounds : 1;

protected:	
	/** All areas where we build/have navigation */
	TSet<FNavigationBounds> RegisteredNavBounds;

private:
	TMap<UObject*, FNavigationInvoker> Invokers;
	/** Contains pre-digested and cached invokers' info. Generated by UpdateInvokers */
	TArray<FNavigationInvokerRaw> InvokerLocations;

	TArray<FBox> InvokersSeedBounds;

	double NextInvokersUpdateTime;
	NAVIGATIONSYSTEM_API void UpdateInvokers();

public:
#if !UE_BUILD_SHIPPING
	void DebugLogInvokers(FOutputDevice& OutputDevice) const;
#endif // !UE_BUILD_SHIPPING

protected:
	NAVIGATIONSYSTEM_API virtual void UpdateNavDataActiveTiles();

private:
	NAVIGATIONSYSTEM_API void DirtyTilesInBuildBounds();

	void UnregisterInvoker_Internal(const UObject& Invoker);

	/** Registers to the navigation objects repository subsystem delegates to get notified on object registration */
	void RegisterToRepositoryDelegates();

	/** Unregisters from the navigation objects repository subsystem delegates */
	void UnregisterFromRepositoryDelegates() const;

#if WITH_EDITOR	
	void UpdateDelayedUnlockeRegistration(const TArray<FAssetCompileData>& AssetCompileData);
	void RegisterDelayedUnlock(float DelayInSeconds);
	void UnregisterDelayedUnlock();

	FDelegateHandle OnAssetPostCompileEventHandle;
	FTSTicker::FDelegateHandle DelayedUnlockHandle;
#endif // WITH_EDITOR

public:
	//----------------------------------------------------------------------//
	// Blueprint functions
	//----------------------------------------------------------------------//

	UFUNCTION(BlueprintPure, Category = "AI|Navigation", meta = (WorldContext = "WorldContextObject"))
	static NAVIGATIONSYSTEM_API UNavigationSystemV1* GetNavigationSystem(UObject* WorldContextObject);
	
	/** Project a point onto the NavigationData */
	UFUNCTION(BlueprintPure, Category = "AI|Navigation", meta = (WorldContext = "WorldContextObject", DisplayName = "Project Point to Navigation", ScriptName = "ProjectPointToNavigation"))
	static NAVIGATIONSYSTEM_API bool K2_ProjectPointToNavigation(UObject* WorldContextObject, const FVector& Point, FVector& ProjectedLocation, ANavigationData* NavData, TSubclassOf<UNavigationQueryFilter> FilterClass, const FVector QueryExtent = FVector::ZeroVector);

	/** Generates a random location reachable from given Origin location.
	 *	@return Return Value represents if the call was successful */
	UFUNCTION(BlueprintPure, Category = "AI|Navigation", meta = (WorldContext = "WorldContextObject", DisplayName = "Get Random Reachable Point in Radius", ScriptName = "GetRandomReachablePointInRadius"))
	static NAVIGATIONSYSTEM_API bool K2_GetRandomReachablePointInRadius(UObject* WorldContextObject, const FVector& Origin, FVector& RandomLocation, float Radius, ANavigationData* NavData = NULL, TSubclassOf<UNavigationQueryFilter> FilterClass = {});

	/** Generates a random location in navigable space within given radius of Origin.
	 *	@return Return Value represents if the call was successful */
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation", meta = (WorldContext = "WorldContextObject", DisplayName = "Get Random Location in Navigable Radius", ScriptName = "GetRandomLocationInNavigableRadius"))
	static NAVIGATIONSYSTEM_API bool K2_GetRandomLocationInNavigableRadius(UObject* WorldContextObject, const FVector& Origin, FVector& RandomLocation, float Radius, ANavigationData* NavData = NULL, TSubclassOf<UNavigationQueryFilter> FilterClass = {});

	/** Potentially expensive. Use with caution. Consider using UPathFollowingComponent::GetRemainingPathCost instead */
	UFUNCTION(BlueprintPure, Category = "AI|Navigation", meta = (WorldContext = "WorldContextObject"))
	static NAVIGATIONSYSTEM_API ENavigationQueryResult::Type GetPathCost(UObject* WorldContextObject, const FVector& PathStart, const FVector& PathEnd, double& PathCost, ANavigationData* NavData = NULL, TSubclassOf<UNavigationQueryFilter> FilterClass = {});

	/** Potentially expensive. Use with caution */
	UFUNCTION(BlueprintPure, Category = "AI|Navigation", meta = (WorldContext = "WorldContextObject"))
	static NAVIGATIONSYSTEM_API ENavigationQueryResult::Type GetPathLength(UObject* WorldContextObject, const FVector& PathStart, const FVector& PathEnd, double& PathLength, ANavigationData* NavData = NULL, TSubclassOf<UNavigationQueryFilter> FilterClass = {});

	UFUNCTION(BlueprintPure, Category="AI|Navigation", meta=(WorldContext="WorldContextObject" ) )
	static NAVIGATIONSYSTEM_API bool IsNavigationBeingBuilt(UObject* WorldContextObject);

	UFUNCTION(BlueprintPure, Category = "AI|Navigation", meta = (WorldContext = "WorldContextObject"))
	static NAVIGATIONSYSTEM_API bool IsNavigationBeingBuiltOrLocked(UObject* WorldContextObject);

	/** Finds path instantly, in a FindPath Synchronously. 
	 *	@param PathfindingContext could be one of following: NavigationData (like Navmesh actor), Pawn or Controller. This parameter determines parameters of specific pathfinding query */
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation", meta = (WorldContext="WorldContextObject"))
	static NAVIGATIONSYSTEM_API UNavigationPath* FindPathToLocationSynchronously(UObject* WorldContextObject, const FVector& PathStart, const FVector& PathEnd, AActor* PathfindingContext = NULL, TSubclassOf<UNavigationQueryFilter> FilterClass = {});

	/** Finds path instantly, in a FindPath Synchronously. Main advantage over FindPathToLocationSynchronously is that 
	 *	the resulting path will automatically get updated if goal actor moves more than TetherDistance away from last path node
	 *	@param PathfindingContext could be one of following: NavigationData (like Navmesh actor), Pawn or Controller. This parameter determines parameters of specific pathfinding query */
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation", meta = (WorldContext="WorldContextObject"))
	static NAVIGATIONSYSTEM_API UNavigationPath* FindPathToActorSynchronously(UObject* WorldContextObject, const FVector& PathStart, AActor* GoalActor, float TetherDistance = 50.f, AActor* PathfindingContext = NULL, TSubclassOf<UNavigationQueryFilter> FilterClass = {});

	/** Performs navigation raycast on NavigationData appropriate for given Querier.
	 *	@param Querier if not passed default navigation data will be used
	 *	@param HitLocation if line was obstructed this will be set to hit location. Otherwise it contains SegmentEnd
	 *	@return true if line from RayStart to RayEnd was obstructed. Also, true when no navigation data present */
	UFUNCTION(BlueprintCallable, Category="AI|Navigation", meta=(WorldContext="WorldContextObject" ))
	static NAVIGATIONSYSTEM_API bool NavigationRaycast(UObject* WorldContextObject, const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, TSubclassOf<UNavigationQueryFilter> FilterClass = {}, AController* Querier = NULL);

	/** Performs navigation raycast on NavigationData appropriate for given Querier.
	 *	@param Querier if not passed default navigation data will be used
	 *	@param HitLocation if line was obstructed this will be set to hit location. Otherwise it contains SegmentEnd
	 *	@param AdditionalResults contains more information about the result of the raycast query. See FNavigationRaycastAdditionalResults description for details
	 *	@return true if line from RayStart to RayEnd was obstructed. Also, true when no navigation data present */
	static NAVIGATIONSYSTEM_API bool NavigationRaycastWithAdditionalResults(UObject* WorldContextObject, const FVector& RayStart, const FVector& RayEnd, FVector& HitLocation, FNavigationRaycastAdditionalResults* AdditionalResults, TSubclassOf<UNavigationQueryFilter> FilterClass = {}, AController* Querier = NULL);

	/** will limit the number of simultaneously running navmesh tile generation jobs to specified number.
	 *	@param MaxNumberOfJobs gets trimmed to be at least 1. You cannot use this function to pause navmesh generation */
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	NAVIGATIONSYSTEM_API void SetMaxSimultaneousTileGenerationJobsCount(int32 MaxNumberOfJobs);
	
	/** Brings limit of simultaneous navmesh tile generation jobs back to Project Setting's default value */
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	NAVIGATIONSYSTEM_API void ResetMaxSimultaneousTileGenerationJobsCount();

	/** Registers given actor as a "navigation enforcer" which means navigation system will
	 *	make sure navigation is being generated in specified radius around it.
	 *	@note: you need NavigationSystem's GenerateNavigationOnlyAroundNavigationInvokers to be set to true
	 *		to take advantage of this feature
	 */
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	NAVIGATIONSYSTEM_API void RegisterNavigationInvoker(AActor* Invoker, float TileGenerationRadius = 3000, float TileRemovalRadius = 5000);

	/** Removes given actor from the list of active navigation enforcers.
	 *	@see RegisterNavigationInvoker for more details */
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	NAVIGATIONSYSTEM_API void UnregisterNavigationInvoker(AActor* Invoker);

	UFUNCTION(BlueprintCallable, Category = "AI|Navigation|Generation")
	NAVIGATIONSYSTEM_API void SetGeometryGatheringMode(ENavDataGatheringModeConfig NewMode);

	UFUNCTION(BlueprintCallable, Category = "AI|Navigation", meta=(DisplayName="Replace Area in Octree Data"))
	NAVIGATIONSYSTEM_API bool K2_ReplaceAreaInOctreeData(const UObject* Object, TSubclassOf<UNavArea> OldArea, TSubclassOf<UNavArea> NewArea);

	inline bool IsActiveTilesGenerationEnabled() const{ return bGenerateNavigationOnlyAroundNavigationInvokers; }
	
	/** delegate type for events that dirty the navigation data ( Params: const FBox& DirtyBounds ) */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnNavigationDirty, const FBox&);
	/** called after navigation influencing event takes place*/
	static NAVIGATIONSYSTEM_API FOnNavigationDirty NavigationDirtyEvent;

	enum ERegistrationResult
	{
		RegistrationError,
		RegistrationFailed_DataPendingKill,			// means navigation data being registered is marked as pending kill
		RegistrationFailed_AgentAlreadySupported,	// this means that navigation agent supported by given nav data is already handled by some other, previously registered instance
		RegistrationFailed_AgentNotValid,			// given instance contains navmesh that doesn't support any of expected agent types, or instance doesn't specify any agent
		RegistrationFailed_NotSuitable,				// given instance had been considered unsuitable by current navigation system instance itself. NOTE: this value is not currently being used by the engine-supplied navigation system classes
		RegistrationSuccessful,
	};

	// EOctreeUpdateMode is deprecated. Use FNavigationOctreeController::EOctreeUpdateMode instead
	enum EOctreeUpdateMode
	{
		OctreeUpdate_Default = 0,						// regular update, mark dirty areas depending on exported content
		OctreeUpdate_Geometry = 1,						// full update, mark dirty areas for geometry rebuild
		OctreeUpdate_Modifiers = 2,						// quick update, mark dirty areas for modifier rebuild
		OctreeUpdate_Refresh = 4,						// update is used for refresh, don't invalidate pending queue
		OctreeUpdate_ParentChain = 8,					// update child nodes, don't remove anything
	};

	//~ Begin UObject Interface
	NAVIGATIONSYSTEM_API virtual void PostInitProperties() override;
	NAVIGATIONSYSTEM_API virtual void FinishDestroy() override;
	static NAVIGATIONSYSTEM_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
#if WITH_EDITOR
	NAVIGATIONSYSTEM_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	NAVIGATIONSYSTEM_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

	NAVIGATIONSYSTEM_API virtual void Tick(float DeltaSeconds) override;	

	UWorld* GetWorld() const override { return GetOuterUWorld(); }

	UCrowdManagerBase* GetCrowdManager() const { return CrowdManager.Get(); }

protected:
	NAVIGATIONSYSTEM_API void CalcTimeSlicedUpdateData(TArray<double>& OutCurrentTimeSlicedBuildTaskDurations, TArray<bool>& OutIsTimeSlicingArray, bool& bOutAnyNonTimeSlicedGenerators, TArray<int32, TInlineAllocator<8>>& OutNumTimeSlicedRemainingBuildTasksArray);

	/** spawn new crowd manager */
	NAVIGATIONSYSTEM_API virtual void CreateCrowdManager();

	/** Used to properly set navigation class for indicated agent and propagate information to other places
	 *	(like project settings) that may need this information 
	 */
	NAVIGATIONSYSTEM_API void SetSupportedAgentsNavigationClass(int32 AgentIndex, TSubclassOf<ANavigationData> NavigationDataClass);

public:
	//----------------------------------------------------------------------//
	//~ Begin Public querying Interface                                                                
	//----------------------------------------------------------------------//
	/** 
	 *	Synchronously looks for a path from @fLocation to @EndLocation for agent with properties @AgentProperties. NavData actor appropriate for specified 
	 *	FNavAgentProperties will be found automatically
	 *	@param ResultPath results are put here
	 *	@param NavData optional navigation data that will be used instead of the one that would be deducted from AgentProperties
	 *  @param Mode switch between normal and hierarchical path finding algorithms
	 */
	NAVIGATIONSYSTEM_API FPathFindingResult FindPathSync(const FNavAgentProperties& AgentProperties, FPathFindingQuery Query, EPathFindingMode::Type Mode = EPathFindingMode::Regular);

	/** 
	 *	Does a simple path finding from @StartLocation to @EndLocation on specified NavData. If none passed MainNavData will be used
	 *	Result gets placed in ResultPath
	 *	@param NavData optional navigation data that will be used instead main navigation data
	 *  @param Mode switch between normal and hierarchical path finding algorithms
	 */
	NAVIGATIONSYSTEM_API FPathFindingResult FindPathSync(FPathFindingQuery Query, EPathFindingMode::Type Mode = EPathFindingMode::Regular);

	/** 
	 *	Asynchronously looks for a path from @StartLocation to @EndLocation for agent with properties @AgentProperties. NavData actor appropriate for specified 
	 *	FNavAgentProperties will be found automatically
	 *	@param AgentProperties optional agent properties used to determine the NavData to use, only when the NavData inside the FPathFindingQuery was left empty.
	 *	@param ResultDelegate delegate that will be called once query has been processed and finished. Will be called even if query fails - in such case see comments for delegate's params
	 *  @param Mode switch between normal and hierarchical path finding algorithms
	 *	@return request ID
	 */
	NAVIGATIONSYSTEM_API uint32 FindPathAsync(const FNavAgentProperties& AgentProperties, FPathFindingQuery Query, const FNavPathQueryDelegate& ResultDelegate, EPathFindingMode::Type Mode = EPathFindingMode::Regular);

	/** Removes query indicated by given ID from queue of path finding requests to process. */
	NAVIGATIONSYSTEM_API void AbortAsyncFindPathRequest(uint32 AsynPathQueryID);
	
	/** 
	 *	Synchronously check if path between two points exists
	 *  Does not return path object, but will run faster (especially in hierarchical mode)
	 *  @param Mode switch between normal and hierarchical path finding algorithms. @note Hierarchical mode ignores QueryFilter
	 *	@return true if path exists
	 */
	NAVIGATIONSYSTEM_API bool TestPathSync(FPathFindingQuery Query, EPathFindingMode::Type Mode = EPathFindingMode::Regular, int32* NumVisitedNodes = NULL) const;

	/** Finds random point in navigable space
	 *	@param ResultLocation Found point is put here
	 *	@param NavData If NavData == NULL then MainNavData is used.
	 *	@return true if any location found, false otherwise */
	NAVIGATIONSYSTEM_API bool GetRandomPoint(FNavLocation& ResultLocation, ANavigationData* NavData = NULL, FSharedConstNavQueryFilter QueryFilter = NULL);

	/** Finds random, reachable point in navigable space restricted to Radius around Origin
	 *	@param ResultLocation Found point is put here
	 *	@param NavData If NavData == NULL then MainNavData is used.
	 *	@return true if any location found, false otherwise */
	NAVIGATIONSYSTEM_API bool GetRandomReachablePointInRadius(const FVector& Origin, float Radius, FNavLocation& ResultLocation, ANavigationData* NavData = NULL, FSharedConstNavQueryFilter QueryFilter = NULL) const;

	/** Finds random, point in navigable space restricted to Radius around Origin. Resulting location is not tested for reachability from the Origin
	 *	@param ResultLocation Found point is put here
	 *	@param NavData If NavData == NULL then MainNavData is used.
	 *	@return true if any location found, false otherwise */
	NAVIGATIONSYSTEM_API bool GetRandomPointInNavigableRadius(const FVector& Origin, float Radius, FNavLocation& ResultLocation, ANavigationData* NavData = NULL, FSharedConstNavQueryFilter QueryFilter = NULL) const;

	/** Calculates a path from PathStart to PathEnd and retrieves its cost.
	 *	@NOTE potentially expensive, so use it with caution */
	NAVIGATIONSYSTEM_API ENavigationQueryResult::Type GetPathCost(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& PathCost, const ANavigationData* NavData = NULL, FSharedConstNavQueryFilter QueryFilter = NULL) const;

	/** Calculates a path from PathStart to PathEnd and retrieves its overestimated length.
	 *	@NOTE potentially expensive, so use it with caution */
	NAVIGATIONSYSTEM_API ENavigationQueryResult::Type GetPathLength(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& PathLength, const ANavigationData* NavData = NULL, FSharedConstNavQueryFilter QueryFilter = NULL) const;

	/** Calculates a path from PathStart to PathEnd and retrieves its overestimated length and cost.
	 *	@NOTE potentially expensive, so use it with caution */
	NAVIGATIONSYSTEM_API ENavigationQueryResult::Type GetPathLengthAndCost(const FVector& PathStart, const FVector& PathEnd, FVector::FReal& PathLength, FVector::FReal& PathCost, const ANavigationData* NavData = NULL, FSharedConstNavQueryFilter QueryFilter = NULL) const;

	// @todo document
	bool ProjectPointToNavigation(const FVector& Point, FNavLocation& OutLocation, const FVector& Extent = INVALID_NAVEXTENT, const FNavAgentProperties* AgentProperties = NULL, FSharedConstNavQueryFilter QueryFilter = NULL)
	{
		return ProjectPointToNavigation(Point, OutLocation, Extent, AgentProperties != NULL ? GetNavDataForProps(*AgentProperties, Point) : GetDefaultNavDataInstance(FNavigationSystem::DontCreate), QueryFilter);
	}

	// @todo document
	NAVIGATIONSYSTEM_API bool ProjectPointToNavigation(const FVector& Point, FNavLocation& OutLocation, const FVector& Extent = INVALID_NAVEXTENT, const ANavigationData* NavData = NULL, FSharedConstNavQueryFilter QueryFilter = NULL) const;

	/** 
	 * Looks for NavData generated for specified movement properties and returns it. NULL if not found;
	 */
	NAVIGATIONSYSTEM_API virtual ANavigationData* GetNavDataForProps(const FNavAgentProperties& AgentProperties);

	/** 
	 * Looks for NavData generated for specified movement properties and returns it. NULL if not found; Const version.
	 */
	NAVIGATIONSYSTEM_API virtual const ANavigationData* GetNavDataForProps(const FNavAgentProperties& AgentProperties) const;

	/** Goes through all registered NavigationData instances and retrieves the one 
	 *	supporting agent named AgentName */
	NAVIGATIONSYSTEM_API virtual ANavigationData* GetNavDataForAgentName(const FName AgentName) const;

	/**
	 * Looks up NavData appropriate for specified movement properties and returns it. NULL if not found;
	 * This is the encouraged way of querying for the appropriate NavData. It makes no difference for NavigationSystemV1
	 *	(AgentLocation and Extent parameters not being used) but NaV2 will take advantage of it and all engine-level
	 * AI navigation code is going to use call this flavor.
	 */
	NAVIGATIONSYSTEM_API virtual ANavigationData* GetNavDataForProps(const FNavAgentProperties& AgentProperties, const FVector& AgentLocation, const FVector& Extent = INVALID_NAVEXTENT) const;

	/** Returns the world default navigation data instance. Creates one if it doesn't exist. */
	NAVIGATIONSYSTEM_API ANavigationData* GetDefaultNavDataInstance(FNavigationSystem::ECreateIfMissing CreateNewIfNoneFound);
	/** Returns the world default navigation data instance. */
	virtual INavigationDataInterface* GetMainNavData() const override { return Cast<INavigationDataInterface>(GetDefaultNavDataInstance()); }
	ANavigationData& GetMainNavDataChecked() const { check(MainNavData); return *MainNavData; }

	/** Set limiting bounds to be used when building navigation data. */
	NAVIGATIONSYSTEM_API virtual void SetBuildBounds(const FBox& Bounds) override;

	NAVIGATIONSYSTEM_API virtual FBox GetNavigableWorldBounds() const override;

	NAVIGATIONSYSTEM_API virtual bool ContainsNavData(const FBox& Bounds) const override;
	NAVIGATIONSYSTEM_API virtual FBox ComputeNavDataBounds() const override;
	NAVIGATIONSYSTEM_API virtual void AddNavigationDataChunk(class ANavigationDataChunkActor& DataChunkActor) override;
	NAVIGATIONSYSTEM_API virtual void RemoveNavigationDataChunk(class ANavigationDataChunkActor& DataChunkActor) override;
	NAVIGATIONSYSTEM_API virtual void FillNavigationDataChunkActor(const FBox& QueryBounds, class ANavigationDataChunkActor& DataChunkActor, FBox& OutTilesBounds) override;

	ANavigationData* GetDefaultNavDataInstance() const { return MainNavData; }

	ANavigationData* GetAbstractNavData() const { return AbstractNavData; }

	/** constructs a navigation data instance of specified NavDataClass, in passed Level
	 *	for supplied NavConfig. If Level == null and bSpawnNavDataInNavBoundsLevel == true
	 *	then the first volume actor in RegisteredNavBounds will be used to source the level. 
	 *	Otherwise the navdata instance will be spawned in NavigationSystem's world */
	NAVIGATIONSYSTEM_API virtual ANavigationData* CreateNavigationDataInstanceInLevel(const FNavDataConfig& NavConfig, ULevel* SpawnLevel);

	NAVIGATIONSYSTEM_API FSharedNavQueryFilter CreateDefaultQueryFilterCopy() const;

	/** Super-hacky safety feature for threaded navmesh building. Will be gone once figure out why keeping TSharedPointer to Navigation Generator doesn't 
	 *	guarantee its existence */
	NAVIGATIONSYSTEM_API bool ShouldGeneratorRun(const FNavDataGenerator* Generator) const;

	NAVIGATIONSYSTEM_API virtual bool IsNavigationBuilt(const AWorldSettings* Settings) const override;

	NAVIGATIONSYSTEM_API virtual bool IsThereAnywhereToBuildNavigation() const;

	bool ShouldGenerateNavigationEverywhere() const { return bWholeWorldNavigable; }

	bool ShouldAllowClientSideNavigation() const { return bAllowClientSideNavigation; }
	virtual bool ShouldLoadNavigationOnClient(ANavigationData* NavData = nullptr) const { return bAllowClientSideNavigation; }
	virtual bool ShouldDiscardSubLevelNavData(ANavigationData* NavData = nullptr) const { return bShouldDiscardSubLevelNavData; }

	NAVIGATIONSYSTEM_API FBox GetWorldBounds() const;
	
	NAVIGATIONSYSTEM_API FBox GetLevelBounds(ULevel* InLevel) const;

	NAVIGATIONSYSTEM_API bool IsNavigationRelevant(const AActor* TestActor) const;

	NAVIGATIONSYSTEM_API const TSet<FNavigationBounds>& GetNavigationBounds() const;

	static NAVIGATIONSYSTEM_API const FNavDataConfig& GetDefaultSupportedAgent();
	static NAVIGATIONSYSTEM_API const FNavDataConfig& GetBiggestSupportedAgent(const UWorld* World);
	
#if WITH_EDITOR
	static NAVIGATIONSYSTEM_API double GetWorldPartitionNavigationDataBuilderOverlap(const UWorld& World);
#endif
	
	NAVIGATIONSYSTEM_API const FNavDataConfig& GetDefaultSupportedAgentConfig() const;
	inline const TArray<FNavDataConfig>& GetSupportedAgents() const { return SupportedAgents; }
	NAVIGATIONSYSTEM_API void OverrideSupportedAgents(const TArray<FNavDataConfig>& NewSupportedAgents);
	NAVIGATIONSYSTEM_API void SetSupportedAgentsMask(const FNavAgentSelector& InSupportedAgentsMask);
	FNavAgentSelector GetSupportedAgentsMask() const { return SupportedAgentsMask; }

	NAVIGATIONSYSTEM_API virtual void ApplyWorldOffset(const FVector& InOffset, bool bWorldShift) override;

	/** checks if navigation/navmesh is dirty and needs to be rebuilt */
	NAVIGATIONSYSTEM_API bool IsNavigationDirty() const;

	/** checks if dirty navigation data can rebuild itself */
	NAVIGATIONSYSTEM_API bool CanRebuildDirtyNavigation() const;

	bool SupportsNavigationGeneration() const { return bSupportRebuilding; }

	bool DirtyNavDataPackageOnNavigationGenerationFinished() const;

	static NAVIGATIONSYSTEM_API bool DoesPathIntersectBox(const FNavigationPath* Path, const FBox& Box, uint32 StartingIndex = 0, FVector* AgentExtent = NULL);
	static NAVIGATIONSYSTEM_API bool DoesPathIntersectBox(const FNavigationPath* Path, const FBox& Box, const FVector& AgentLocation, uint32 StartingIndex = 0, FVector* AgentExtent = NULL);
	
	//----------------------------------------------------------------------//
	// Active tiles
	//----------------------------------------------------------------------//
	NAVIGATIONSYSTEM_API virtual void RegisterInvoker(AActor& Invoker, float TileGenerationRadius, float TileRemovalRadius, const FNavAgentSelector& Agents, ENavigationInvokerPriority InPriority);

	NAVIGATIONSYSTEM_API virtual void RegisterInvoker(const TWeakInterfacePtr<INavigationInvokerInterface>& Invoker, float TileGenerationRadius, float TileRemovalRadius, const FNavAgentSelector& Agents, ENavigationInvokerPriority InPriority);

	NAVIGATIONSYSTEM_API virtual void UnregisterInvoker(AActor& Invoker);
	
	NAVIGATIONSYSTEM_API virtual void UnregisterInvoker(const TWeakInterfacePtr<INavigationInvokerInterface>& Invoker);

	static NAVIGATIONSYSTEM_API void RegisterNavigationInvoker(AActor& Invoker, float TileGenerationRadius, float TileRemovalRadius, 
		const FNavAgentSelector& Agents = FNavAgentSelector(), ENavigationInvokerPriority Priority = ENavigationInvokerPriority::Default);
	static NAVIGATIONSYSTEM_API void UnregisterNavigationInvoker(AActor& Invoker);

	const TArray<FNavigationInvokerRaw>& GetInvokerLocations() const { return InvokerLocations; }
	const TArray<FBox>& GetInvokersSeedBounds() const { return InvokersSeedBounds; }

	//----------------------------------------------------------------------//
	// Bookkeeping 
	//----------------------------------------------------------------------//
	
	// @todo document
	NAVIGATIONSYSTEM_API virtual void UnregisterNavData(ANavigationData* NavData);

	/** Traverses SupportedAgents and for all agents not supported (i.e. filtered
	 *	out by SupportedAgentsMask) checks if there's a currently registered
	 *	NavigationData instance for that agent, and if so it unregisters that agent */
	NAVIGATIONSYSTEM_API virtual void UnregisterUnusedNavData();

	/** Adds NavData to registration candidates queue - NavDataRegistrationQueue*/
	NAVIGATIONSYSTEM_API virtual void RequestRegistrationDeferred(ANavigationData& NavData);

protected:
	NAVIGATIONSYSTEM_API void ApplySupportedAgentsFilter();

	/** Processes all NavigationData instances in UWorld owning navigation system instance, and registers
	 *	all previously unregistered */
	NAVIGATIONSYSTEM_API void RegisterNavigationDataInstances();

	/** called in places where we need to spawn the NavOctree, but is checking additional conditions if we really want to do that
	 *	depending on navigation data setup among others 
	 *	@return true if NavOctree instance has been created, or if one is already present */
	NAVIGATIONSYSTEM_API virtual bool ConditionalPopulateNavOctree();

	/** called to instantiate NavigationSystem's NavOctree instance */
	NAVIGATIONSYSTEM_API virtual void ConstructNavOctree();

	/** Processes registration of candidates queues via RequestRegistration and stored in NavDataRegistrationQueue */
	NAVIGATIONSYSTEM_API virtual void ProcessRegistrationCandidates();

	/** Registers custom navigation links awaiting registration in the navigation object repository */
	NAVIGATIONSYSTEM_API void ProcessCustomLinkPendingRegistration();

	/** used to apply updates of nav volumes in navigation system's tick */
	NAVIGATIONSYSTEM_API virtual void PerformNavigationBoundsUpdate(const TArray<FNavigationBoundsUpdateRequest>& UpdateRequests);
	
	/** adds data to RegisteredNavBounds */
	NAVIGATIONSYSTEM_API void AddNavigationBounds(const FNavigationBounds& NewBounds);

	/** Searches for all valid navigation bounds in the world and stores them */
	NAVIGATIONSYSTEM_API virtual void GatherNavigationBounds();

	/** Get seed locations for invokers, @see InvokersMaximumDistanceFromSeed 
	 *	By default these are the player pawn locations. If a player controller has no pawn assigned, the player's camera location will be used instead. */ 
	NAVIGATIONSYSTEM_API virtual void GetInvokerSeedLocations(const UWorld& InWorld, TArray<FVector, TInlineAllocator<32>>& OutSeedLocations);

	/** @return pointer to ANavigationData instance of given ID, or NULL if it was not found. Note it looks only through registered navigation data */
	NAVIGATIONSYSTEM_API ANavigationData* GetNavDataWithID(const uint16 NavDataID) const;

	/** @return Valid shared pointer on a FNavigationElement if the provided UObject is registered in the navigation system, an invalid one otherwise. */
	NAVIGATIONSYSTEM_API TSharedPtr<const FNavigationElement> GetNavigationElementForUObject(const UObject*) const;

	/** @return Valid handle if the provided UObject is registered in the navigation system, FNavigationElementHandle::Invalid otherwise. */
	NAVIGATIONSYSTEM_API FNavigationElementHandle GetNavigationElementHandleForUObject(const UObject*) const;

	static NAVIGATIONSYSTEM_API void RegisterComponentToNavOctree(UActorComponent* Comp);
	static NAVIGATIONSYSTEM_API void UnregisterComponentToNavOctree(UActorComponent* Comp);

public:
	NAVIGATIONSYSTEM_API virtual void ReleaseInitialBuildingLock();

	//----------------------------------------------------------------------//
	// navigation octree related functions
	//----------------------------------------------------------------------//
	static NAVIGATIONSYSTEM_API bool SupportsDynamicChanges(UWorld* World);
	static NAVIGATIONSYSTEM_API FNavigationElementHandle AddNavigationElement(UWorld* World, FNavigationElement&& Element);
	static NAVIGATIONSYSTEM_API void RemoveNavigationElement(UWorld* World, FNavigationElementHandle ElementHandle);
	static NAVIGATIONSYSTEM_API void OnNavigationElementUpdated(UWorld* World, FNavigationElementHandle ElementHandle, FNavigationElement&& Element);

	static NAVIGATIONSYSTEM_API void OnNavRelevantObjectRegistered(UObject& Object);
	static NAVIGATIONSYSTEM_API void UpdateNavRelevantObjectInNavOctree(UObject& Object);
	static NAVIGATIONSYSTEM_API void OnNavRelevantObjectUnregistered(UObject& Object);

	static NAVIGATIONSYSTEM_API void OnComponentRegistered(UActorComponent* Comp);
	static NAVIGATIONSYSTEM_API void OnComponentUnregistered(UActorComponent* Comp);
	static NAVIGATIONSYSTEM_API void RegisterComponent(UActorComponent* Comp);
	static NAVIGATIONSYSTEM_API void UnregisterComponent(UActorComponent* Comp);
	static NAVIGATIONSYSTEM_API void OnActorRegistered(AActor* Actor);
	static NAVIGATIONSYSTEM_API void OnActorUnregistered(AActor* Actor);

	/** update navoctree entry for specified actor/component */
	static NAVIGATIONSYSTEM_API void UpdateActorInNavOctree(AActor& Actor);
	static NAVIGATIONSYSTEM_API void UpdateComponentInNavOctree(UActorComponent& Comp);
	/** update all navoctree entries for actor and its components */
	static NAVIGATIONSYSTEM_API void UpdateActorAndComponentsInNavOctree(AActor& Actor, bool bUpdateAttachedActors = true);
	/** update all navoctree entries for actor and its non scene components after root movement */
	static NAVIGATIONSYSTEM_API void UpdateNavOctreeAfterMove(USceneComponent* Comp);

protected:
	/** Fetches the repository from the provided object and registers the object with it. */
	static NAVIGATIONSYSTEM_API void RegisterNavRelevantObjectStatic(const INavRelevantInterface& NavRelevantObject, const UObject& Object);

	/** Similar to UnregisterNavRelevantObjectStatic but uses the cached repository. */
	NAVIGATIONSYSTEM_API void RegisterNavRelevantObjectInternal(const INavRelevantInterface& NavRelevantObject, const UObject& Object);

	/** Fetches the repository from the provided object and unregisters the object with it. */
	static NAVIGATIONSYSTEM_API void UnregisterNavRelevantObjectStatic(const UObject& Object);

	/** Similar to UnregisterNavRelevantObjectStatic but uses the cached repository. */
	NAVIGATIONSYSTEM_API void UnregisterNavRelevantObjectInternal(const UObject& Object);

	/**
	 * In actor/component code paths it is possible that updates get called without a previous call to register (wasn't relevant at that time).
	 * In those cases we create a new, or an updated, navigation element if needed.
	 * @note Method expects only navigation relevant objects and will ensure otherwise.
	 */
	static NAVIGATIONSYSTEM_API void UpdateNavRelevantObjectInNavOctreeStatic(
		const INavRelevantInterface& InNavRelevantObject,
		const UObject& InObject,
		UNavigationSystemV1* InNavigationSystem,
		TFunctionRef<void(UNavigationSystemV1&, const TSharedRef<const FNavigationElement>&)> InCallback);

	/** A helper function that gathers all actors attached to RootActor and fetches 
	 *	them back. The function does consider multi-level attachments.
	 *	@param RootActor The reference actor to get the attached actors from.
	 *	@param OutAttachedActors is getting reset at the beginning of the function. 
	 *		When done it is guaranteed to contain unique, non-null ptrs. It 
	 *		will not include RootActor.
	 *	@return the number of unique attached actors */
	static NAVIGATIONSYSTEM_API int32 GetAllAttachedActors(const AActor& RootActor, TArray<AActor*>& OutAttachedActors);

	/** updates navoctree information on actors attached to RootActor */
	static NAVIGATIONSYSTEM_API void UpdateAttachedActorsInNavOctree(AActor& RootActor);

public:
	/** removes all navoctree entries for actor and its components */
	static NAVIGATIONSYSTEM_API void ClearNavOctreeAll(AActor* Actor);

	/** updates bounds of all components implementing INavRelevantInterface */
	static NAVIGATIONSYSTEM_API void UpdateNavOctreeBounds(AActor* Actor);

	NAVIGATIONSYSTEM_API void AddDirtyArea(const FBox& NewArea, ENavigationDirtyFlag Flags, const FName& DebugReason = NAME_None);
	NAVIGATIONSYSTEM_API void AddDirtyArea(const FBox& NewArea, ENavigationDirtyFlag Flags, const TFunction<const TSharedPtr<const FNavigationElement>()>& ElementProviderFunc, const FName& DebugReason = NAME_None);
	NAVIGATIONSYSTEM_API void AddDirtyAreas(const TArray<FBox>& NewAreas, ENavigationDirtyFlag Flags, const FName& DebugReason = NAME_None);
	NAVIGATIONSYSTEM_API bool HasDirtyAreasQueued() const;
	NAVIGATIONSYSTEM_API int32 GetNumDirtyAreas() const;
	float GetDirtyAreaWarningSizeThreshold() const { return DirtyAreaWarningSizeThreshold; }

	const FNavigationOctree* GetNavOctree() const { return DefaultOctreeController.GetOctree(); }
	FNavigationOctree* GetMutableNavOctree() { return DefaultOctreeController.GetMutableOctree(); }

	inline const FOctreeElementId2* GetNavOctreeIdForElement(FNavigationElementHandle Element) const
	{
		return DefaultOctreeController.GetNavOctreeIdForElement(Element);
	}

	inline bool HasPendingUpdateForElement(FNavigationElementHandle Element) const
	{
		return DefaultOctreeController.HasPendingUpdateForElement(Element);
	}

	NAVIGATIONSYSTEM_API void FlushPendingOperations();

	/** Update any pending FNavigationDirtyElement in the navigation system default navigation octree. */
	NAVIGATIONSYSTEM_API void ProcessPendingOctreeUpdates();
	
protected:
	NAVIGATIONSYSTEM_API void ResetDefaultDirtyAreasController();

	NAVIGATIONSYSTEM_API void RemoveFromNavOctree(const FOctreeElementId2& ElementId, int32 UpdateFlags);

public:
	NAVIGATIONSYSTEM_API const FNavigationRelevantData* GetDataForElement(FNavigationElementHandle Element) const;

	NAVIGATIONSYSTEM_API FNavigationRelevantData* GetMutableDataForElement(FNavigationElementHandle Element);

	/** find all elements in navigation octree within given box (intersection) */
	NAVIGATIONSYSTEM_API void FindElementsInNavOctree(const FBox& QueryBox, const FNavigationOctreeFilter& Filter, TArray<FNavigationOctreeElement>& Elements);

	/** update single element in navoctree */
	NAVIGATIONSYSTEM_API void UpdateNavOctreeElement(FNavigationElementHandle Handle, const TSharedRef<const FNavigationElement>& Element, int32 UpdateFlags);

	/** update element's bounds in navigation octree and mark only specified area as dirty, doesn't re-export geometry */
	NAVIGATIONSYSTEM_API bool UpdateNavOctreeElementBounds(FNavigationElementHandle Handle, const FBox& NewBounds, TConstArrayView<FBox> DirtyAreas);

	/** fetched element's data from the octree and replaces occurrences of OldArea with NewArea */
	NAVIGATIONSYSTEM_API bool ReplaceAreaInOctreeData(FNavigationElementHandle Handle, TSubclassOf<UNavArea> OldArea, TSubclassOf<UNavArea> NewArea, bool bReplaceChildClasses = false);

	//----------------------------------------------------------------------//
	// Custom navigation links
	//----------------------------------------------------------------------//
	NAVIGATIONSYSTEM_API virtual void RegisterCustomLink(INavLinkCustomInterface& CustomLink);
	NAVIGATIONSYSTEM_API void UnregisterCustomLink(INavLinkCustomInterface& CustomLink);
	int32 GetNumCustomLinks() const { return CustomNavLinksMap.Num(); }

	static NAVIGATIONSYSTEM_API void RequestCustomLinkRegistering(INavLinkCustomInterface& CustomLink, UObject* OwnerOb);
	static NAVIGATIONSYSTEM_API void RequestCustomLinkUnregistering(INavLinkCustomInterface& CustomLink, UObject* ObjectOb);

	UE_DEPRECATED(5.3, "LinkIds are now based on FNavLinkId call the version of this function that takes FNavLinkId. This function only returns nullptr.")
	INavLinkCustomInterface* GetCustomLink(uint32 UniqueLinkId) const { return nullptr; }

	/** find custom link by unique ID */
	NAVIGATIONSYSTEM_API INavLinkCustomInterface* GetCustomLink(FNavLinkId UniqueLinkId) const;

	/** updates custom link for all active navigation data instances */
	NAVIGATIONSYSTEM_API void UpdateCustomLink(const INavLinkCustomInterface* CustomLink);

	/** Return a Bounding Box containing the navlink points */
	static NAVIGATIONSYSTEM_API FBox ComputeCustomLinkBounds(const INavLinkCustomInterface& CustomLink);

	//----------------------------------------------------------------------//
	// Areas
	//----------------------------------------------------------------------//
	static NAVIGATIONSYSTEM_API void RequestAreaRegistering(UClass* NavAreaClass);
	static NAVIGATIONSYSTEM_API void RequestAreaUnregistering(UClass* NavAreaClass);

	/** find index in SupportedAgents array for given navigation data */
	NAVIGATIONSYSTEM_API int32 GetSupportedAgentIndex(const ANavigationData* NavData) const;

	/** find index in SupportedAgents array for agent type */
	NAVIGATIONSYSTEM_API int32 GetSupportedAgentIndex(const FNavAgentProperties& NavAgent) const;

	//----------------------------------------------------------------------//
	// Filters
	//----------------------------------------------------------------------//
	
	/** prepare descriptions of navigation flags in UNavigationQueryFilter class: using enum */
	NAVIGATIONSYSTEM_API void DescribeFilterFlags(UEnum* FlagsEnum) const;

	/** prepare descriptions of navigation flags in UNavigationQueryFilter class: using array */
	NAVIGATIONSYSTEM_API void DescribeFilterFlags(const TArray<FString>& FlagsDesc) const;

	/** removes cached filters from currently registered navigation data */
	NAVIGATIONSYSTEM_API void ResetCachedFilter(TSubclassOf<UNavigationQueryFilter> FilterClass);

	//----------------------------------------------------------------------//
	// building
	//----------------------------------------------------------------------//
	
	/** Triggers navigation building on all eligible navigation data. */
	NAVIGATIONSYSTEM_API virtual void Build();

	/** Cancels all currently running navigation builds */
	NAVIGATIONSYSTEM_API virtual void CancelBuild();

	// @todo document
	NAVIGATIONSYSTEM_API void OnPIEStart();
	// @todo document
	NAVIGATIONSYSTEM_API void OnPIEEnd();
	
	// @todo document
	inline bool IsNavigationBuildingLocked(uint8 Flags = ~0) const { return (NavBuildingLockFlags & Flags) != 0; }

	/** check if building is permanently locked to avoid showing navmesh building notify (due to queued dirty areas) */
	inline bool IsNavigationBuildingPermanentlyLocked() const
	{
		return (NavBuildingLockFlags & ~ENavigationBuildLock::InitialLock) != 0; 
	}

	/** check if navigation octree updates are currently ignored */
	inline bool IsNavigationOctreeLocked() const { return DefaultOctreeController.IsNavigationOctreeLocked(); }

	// @todo document
	UFUNCTION(BlueprintCallable, Category = "AI|Navigation")
	NAVIGATIONSYSTEM_API void OnNavigationBoundsUpdated(ANavMeshBoundsVolume* NavVolume);
	NAVIGATIONSYSTEM_API virtual void OnNavigationBoundsAdded(ANavMeshBoundsVolume* NavVolume);
	NAVIGATIONSYSTEM_API virtual void OnNavigationBoundsRemoved(ANavMeshBoundsVolume* NavVolume);

	/** determines whether any generator is performing navigation building actions at the moment, dirty areas are also checked */
	NAVIGATIONSYSTEM_API bool IsNavigationBuildInProgress();
	
	NAVIGATIONSYSTEM_API virtual void OnNavigationGenerationFinished(ANavigationData& NavData);

	/** Used to display "navigation building in progress" counter */
	NAVIGATIONSYSTEM_API int32 GetNumRemainingBuildTasks() const;

	/** Number of currently running tasks */
	NAVIGATIONSYSTEM_API int32 GetNumRunningBuildTasks() const;

protected:
	/** Sets up SuportedAgents and NavigationDataCreators. Override it to add additional setup, but make sure to call Super implementation */
	NAVIGATIONSYSTEM_API virtual void DoInitialSetup();

	/** Find or create abstract nav data */
	NAVIGATIONSYSTEM_API virtual void UpdateAbstractNavData();
	
	/** Called during ConditionalPopulateNavOctree and gives subclassess a chance
	 *	to influence what gets added */
	NAVIGATIONSYSTEM_API virtual void AddLevelToOctree(ULevel& Level);

	/** Called as part of UWorld::BeginTearingDown */
	NAVIGATIONSYSTEM_API virtual void OnBeginTearingDown(UWorld* World);
	
public:
	/** Called upon UWorld destruction to release what needs to be released */
	NAVIGATIONSYSTEM_API virtual void CleanUp(const FNavigationSystem::ECleanupMode Mode = FNavigationSystem::ECleanupMode::CleanupUnsafe) override;

	/** 
	 *	Called when owner-UWorld initializes actors
	 */
	NAVIGATIONSYSTEM_API virtual void OnInitializeActors() override;

	/** */
	NAVIGATIONSYSTEM_API virtual void OnWorldInitDone(FNavigationSystemRunMode Mode);

	/** Returns true if world has been initialized. */
	inline virtual bool IsWorldInitDone() const override { return bWorldInitDone; }
	inline bool IsInitialized() const { return bWorldInitDone; }

	inline FNavigationSystemRunMode GetRunMode() const { return OperationMode; }

	/** adds BSP collisions of currently streamed in levels to octree */
	NAVIGATIONSYSTEM_API void InitializeLevelCollisions();

	enum class ELockRemovalRebuildAction
	{
		Rebuild,
		RebuildIfNotInEditor,
		NoRebuild
	};
	NAVIGATIONSYSTEM_API void AddNavigationBuildLock(uint8 Flags);
	NAVIGATIONSYSTEM_API void RemoveNavigationBuildLock(uint8 Flags, const ELockRemovalRebuildAction RebuildAction = ELockRemovalRebuildAction::Rebuild);

	NAVIGATIONSYSTEM_API void SetNavigationOctreeLock(bool bLock);

	/** checks if auto-rebuilding navigation data is enabled. Defaults to bNavigationAutoUpdateEnabled
	*	value, but can be overridden per nav sys instance */
	virtual bool GetIsAutoUpdateEnabled() const { return bNavigationAutoUpdateEnabled; }

#if WITH_EDITOR
	/** allow editor to toggle whether seamless navigation building is enabled */
	static NAVIGATIONSYSTEM_API void SetNavigationAutoUpdateEnabled(bool bNewEnable, UNavigationSystemBase* InNavigationSystem);

	inline bool IsNavigationRegisterLocked() const { return NavUpdateLockFlags != 0; }
	inline bool IsNavigationUnregisterLocked() const { return NavUpdateLockFlags && !(NavUpdateLockFlags & ENavigationLockReason::AllowUnregister); }
	inline bool IsNavigationUpdateLocked() const { return IsNavigationRegisterLocked(); }
	inline void AddNavigationUpdateLock(uint8 Flags) { NavUpdateLockFlags |= Flags; }
	inline void RemoveNavigationUpdateLock(uint8 Flags) { NavUpdateLockFlags &= ~Flags; }

	NAVIGATIONSYSTEM_API void UpdateLevelCollision(ULevel* InLevel);

#endif // WITH_EDITOR

	inline bool IsSetUpForLazyGeometryExporting() const { return bGenerateNavigationOnlyAroundNavigationInvokers; }

	static NAVIGATIONSYSTEM_API UNavigationSystemV1* GetCurrent(UWorld* World);
	static NAVIGATIONSYSTEM_API UNavigationSystemV1* GetCurrent(UObject* WorldContextObject);
	
	NAVIGATIONSYSTEM_API virtual void InitializeForWorld(UWorld& World, FNavigationSystemRunMode Mode) override;

	// Fetch the array of all nav-agent properties.
	NAVIGATIONSYSTEM_API void GetNavAgentPropertiesArray(TArray<FNavAgentProperties>& OutNavAgentProperties) const;

	static inline bool ShouldUpdateNavOctreeOnComponentChange()
	{
		return (bUpdateNavOctreeOnComponentChange && !bStaticRuntimeNavigation)
#if WITH_EDITOR
			|| (GIsEditor && !GIsPlayInEditorWorld)
#endif
			;
	}

	/**
	 * Indicates whether navigation is allowed for a given world.
	 * @param World The world in which the navigation would be used
	 * @return whether the navigation system can be used in the specified world or not
	 */
	NAVIGATIONSYSTEM_API virtual bool ShouldCreateNavigationSystemInstance(const UWorld* World) const override;

	static inline bool IsNavigationSystemStatic()
	{
#if WITH_EDITOR
		return (bStaticRuntimeNavigation && !(GIsEditor && !GIsPlayInEditorWorld))
			|| IsRunningCookCommandlet(); // consider navigation static when cooking to eliminate useless operations
#else
	return bStaticRuntimeNavigation;
#endif // WITH_EDITOR
	}

	/**	call with bEnableStatic == true to signal the NavigationSystem it doesn't need to store
	 *	any navigation-generation-related data at game runtime, because 
	 *	nothing is going to use it anyway. This will short-circuit all code related 
	 *	to navmesh rebuilding, so use it only if you have fully static navigation in 
	 *	your game. bEnableStatic = false will reset the mechanism.
	 *	Note: this is not a runtime switch. It's highly advisable not to call it manually and 
	 *	use UNavigationSystemModuleConfig.bStrictlyStatic instead */
	static NAVIGATIONSYSTEM_API void ConfigureAsStatic(bool bEnableStatic = true);

	static NAVIGATIONSYSTEM_API void SetUpdateNavOctreeOnComponentChange(bool bNewUpdateOnComponentChange);

	/** 
	 * Exec command handlers
	 */
	NAVIGATIONSYSTEM_API bool HandleCycleNavDrawnCommand( const TCHAR* Cmd, FOutputDevice& Ar );
	NAVIGATIONSYSTEM_API bool HandleCountNavMemCommand();
	
	//----------------------------------------------------------------------//
	// debug
	//----------------------------------------------------------------------//
	NAVIGATIONSYSTEM_API void CycleNavigationDataDrawn();

	FNavRegenTimeSliceManager& GetMutableNavRegenTimeSliceManager() { return NavRegenTimeSliceManager; }

	FNavigationSystemRunMode GetOperationMode() const { return OperationMode; }

protected:

	UPROPERTY()
	FNavigationSystemRunMode OperationMode;

	/** Queued async pathfinding queries to process in the next update. */
	TArray<FAsyncPathFindingQuery> AsyncPathFindingQueries;

	/** Queued async pathfinding results computed by the dedicated task in the last frame and ready to dispatch in the next update. */
	TArray<FAsyncPathFindingQuery> AsyncPathFindingCompletedQueries;

	/** Graph event that the main thread will wait for to synchronize with the async pathfinding task, if any. */
	FGraphEventRef AsyncPathFindingTask;

	/** Flag used by main thread to ask the async pathfinding task to stop and postpone remaining queries, if any. */
	TAtomic<bool> bAbortAsyncQueriesRequested;

	FCriticalSection NavDataRegistration;

	TMap<FNavAgentProperties, TWeakObjectPtr<ANavigationData> > AgentToNavDataMap;
	
	FNavigationOctreeController DefaultOctreeController;

	TMap<FNavLinkId, FNavigationSystem::FCustomLinkOwnerInfo> CustomNavLinksMap;

	FNavigationDirtyAreasController DefaultDirtyAreasController;

	UPROPERTY(transient)
	TObjectPtr<UNavigationObjectRepository> Repository = nullptr;

	TArray<TWeakObjectPtr<ANavigationData>> NavigationDataToRemove;

	// async queries
	FCriticalSection NavDataRegistrationSection;

#if WITH_EDITOR
	uint8 NavUpdateLockFlags;
#endif
	uint8 NavBuildingLockFlags;

	/** set of locking flags applied on startup of navigation system */
	uint8 InitialNavBuildingLockFlags;
	
	uint8 bInitialSetupHasBeenPerformed : 1;
	uint8 bInitialLevelsAdded : 1;
	uint8 bWorldInitDone : 1;
	/** set when the NavSys instance has been cleaned up. This is an irreversible state */
	uint8 bCleanUpDone : 1;
	uint8 bAsyncBuildPaused : 1; // mz@todo remove, replaced by bIsPIEActive and IsGameWorld
protected:

	/** cached navigable world bounding box*/
	mutable FBox NavigableWorldBounds;

	/** indicates which of multiple navigation data instances to draw*/
	int32 CurrentlyDrawnNavDataIndex;
	
#if !UE_BUILD_SHIPPING
	/** self-registering exec command to handle nav sys console commands */
	static NAVIGATIONSYSTEM_API FNavigationSystemExec ExecHandler;
#endif // !UE_BUILD_SHIPPING
	
	/** whether seamless navigation building is enabled */
	static NAVIGATIONSYSTEM_API bool bNavigationAutoUpdateEnabled;
	static NAVIGATIONSYSTEM_API bool bUpdateNavOctreeOnComponentChange;
	static NAVIGATIONSYSTEM_API bool bStaticRuntimeNavigation;
	static NAVIGATIONSYSTEM_API bool bIsPIEActive;

	TSet<TObjectPtr<const UClass>> NavAreaClasses;

	FNavRegenTimeSliceManager NavRegenTimeSliceManager;

	/** delegate handler for PostLoadMap event */
	NAVIGATIONSYSTEM_API void OnPostLoadMap(UWorld* LoadedWorld);
	
#if WITH_EDITOR
	void OnMapOpened(const FString& Filename, bool bAsTemplate);
	
	/** delegate handler for ActorMoved events */
	NAVIGATIONSYSTEM_API void OnActorMoved(AActor* Actor);
#endif // WITH_EDITOR

	/** delegate handler called when navigation is dirtied*/
	NAVIGATIONSYSTEM_API void OnNavigationDirtied(const FBox& Bounds);

	FDelegateHandle ReloadCompleteDelegateHandle;

	/** called to notify NavigationSystem about finished reload */
	NAVIGATIONSYSTEM_API virtual void OnReloadComplete(EReloadCompleteReason Reason);

	/** Registers given navigation data with this Navigation System.
	 *	@return RegistrationSuccessful if registration was successful, other results mean it failed
	 *	@see ERegistrationResult
	 */
	NAVIGATIONSYSTEM_API virtual ERegistrationResult RegisterNavData(ANavigationData* NavData);

	/** tries to register navigation area */
	NAVIGATIONSYSTEM_API void RegisterNavAreaClass(UClass* NavAreaClass);

	/** tries to unregister navigation area */
	NAVIGATIONSYSTEM_API void UnregisterNavAreaClass(UClass* NavAreaClass);

	NAVIGATIONSYSTEM_API void OnNavigationAreaEvent(UClass* AreaClass, ENavAreaEvent::Type Event);

	NAVIGATIONSYSTEM_API FSetElementId RegisterNavigationElementWithNavOctree(const TSharedRef<const FNavigationElement>& Element, int32 UpdateFlags);

	NAVIGATIONSYSTEM_API void UnregisterNavigationElementWithOctree(const TSharedRef<const FNavigationElement>& Element, int32 UpdateFlags);

	/** read element data from navigation octree */
	NAVIGATIONSYSTEM_API bool GetNavOctreeElementData(FNavigationElementHandle Element, ENavigationDirtyFlag& OutDirtyFlags, FBox& OutDirtyBounds);

	/** Adds given element to NavOctree. No check for owner's validity are performed, 
	 *	nor its presence in NavOctree - function assumes callee responsibility 
	 *	in this regard **/
	NAVIGATIONSYSTEM_API void AddElementToNavOctree(const FNavigationDirtyElement& DirtyElement);

	NAVIGATIONSYSTEM_API void SetCrowdManager(UCrowdManagerBase* NewCrowdManager);

	/** Add BSP collision data to navigation octree */
	NAVIGATIONSYSTEM_API void AddLevelCollisionToOctree(ULevel* Level);
	
	/** Remove BSP collision data from navigation octree */
	NAVIGATIONSYSTEM_API void RemoveLevelCollisionFromOctree(ULevel* Level);

	NAVIGATIONSYSTEM_API virtual void SpawnMissingNavigationData();

	/** 
	 * Fills a mask indicating which navigation data associated to the supported agent mask are already instantiated.
	 * @param OutInstantiatedMask The mask that will represent already instantiated navigation data.
	 * @param InLevel If specified will be used to search for navigation data; will use the owning world otherwise.
	 * @return Number of instantiated navigation data.
	 */
	NAVIGATIONSYSTEM_API uint8 FillInstantiatedDataMask(TBitArray<>& OutInstantiatedMask, ULevel* InLevel = nullptr);

	/**
	 * Spawns missing navigation data.
	 * @param InInstantiatedMask The mask representing already instantiated navigation data.
	 * @param InLevel Level in which the new data must be added. See CreateNavigationDataInstanceInLevel doc.
	 */
	NAVIGATIONSYSTEM_API void SpawnMissingNavigationDataInLevel(const TBitArray<>& InInstantiatedMask, ULevel* InLevel = nullptr);

public:
	NAVIGATIONSYSTEM_API void DemandLazyDataGathering(FNavigationRelevantData& ElementData);

private:
	// Reduces a nav bounds only to regions overlapping loaded world partition regions (if any exist)
	void CheckToLimitNavigationBoundsToLoadedRegions(FNavigationBounds& OutBounds) const;

protected:
	NAVIGATIONSYSTEM_API virtual void RebuildDirtyAreas(float DeltaSeconds);
	
	// adds navigation bounds update request to a pending list
	NAVIGATIONSYSTEM_API void AddNavigationBoundsUpdateRequest(const FNavigationBoundsUpdateRequest& UpdateRequest);

	/** Triggers navigation building on all eligible navigation data. */
	NAVIGATIONSYSTEM_API virtual void RebuildAll(bool bIsLoadTime = false);
		 
	/** Handler for FWorldDelegates::LevelAddedToWorld event */
	 NAVIGATIONSYSTEM_API void OnLevelAddedToWorld(ULevel* InLevel, UWorld* InWorld);
	 
	/** Handler for FWorldDelegates::LevelRemovedFromWorld event */
	NAVIGATIONSYSTEM_API void OnLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld);

	/** Handler for FWorldDelegates::OnWorldPostActorTick event */
	void OnWorldPostActorTick(UWorld* World, ELevelTick TickType, float DeltaTime)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UNavigationSystemV1::OnWorldPostActorTick)
		PostponeAsyncQueries();
	}

	/** Adds given request to requests queue. Note it's to be called only on game thread only */
	NAVIGATIONSYSTEM_API void AddAsyncQuery(const FAsyncPathFindingQuery& Query);
		 
	/** spawns a non-game-thread task to process requests given in PathFindingQueries.
	 *	In the process PathFindingQueries gets copied. */
	NAVIGATIONSYSTEM_API void TriggerAsyncQueries(TArray<FAsyncPathFindingQuery>& PathFindingQueries);

	/** Processes pathfinding requests given in PathFindingQueries.*/
	NAVIGATIONSYSTEM_API void PerformAsyncQueries(TArray<FAsyncPathFindingQuery> PathFindingQueries);

	/** Broadcasts completion delegate for all completed async pathfinding requests. */
	NAVIGATIONSYSTEM_API void DispatchAsyncQueriesResults(const TArray<FAsyncPathFindingQuery>& PathFindingQueries) const;

	/**
	 * Requests the async pathfinding task to abort and waits for it to complete
	 * before resuming the main thread. Pathfind task will postpone remaining queries to next frame.
	 */
	NAVIGATIONSYSTEM_API void PostponeAsyncQueries();

	/** */
	NAVIGATIONSYSTEM_API virtual void DestroyNavOctree();

	/** Whether Navigation system needs to populate nav octree. 
	 *  Depends on runtime generation settings of each navigation data, always true in the editor
	 */
	NAVIGATIONSYSTEM_API bool RequiresNavOctree() const;

	/** Return "Strongest" runtime generation type required by registered navigation data objects
	 *  Depends on runtime generation settings of each navigation data, always ERuntimeGenerationType::Dynamic in the editor world
	 */
	NAVIGATIONSYSTEM_API ERuntimeGenerationType GetRuntimeGenerationType() const;

	/** Returns true if any registered navigation data supports runtime generation.
	 *  Unlike GetRuntimeGenerationType which returns the raw config enum,
	 *  this also accounts for nav data that was spawned at runtime and needs rebuilding. */
	NAVIGATIONSYSTEM_API bool DoesAnyNavDataSupportRuntimeGeneration() const;

	/** Returns true if any registered navigation data requires geometry to be stored in the nav octree.
	 *  This is true for Dynamic nav data, or nav data that was spawned at runtime (which needs a full rebuild).
	 *  DynamicModifiersOnly does NOT require geometry storage. */
	NAVIGATIONSYSTEM_API bool DoesAnyNavDataRequireGeometryStorage() const;

	UE_DEPRECATED(5.8, "Use LogNavDataRegistrationResult receiving ANavigationData* instead.")
	NAVIGATIONSYSTEM_API void LogNavDataRegistrationResult(ERegistrationResult);
	
	NAVIGATIONSYSTEM_API void LogNavDataRegistrationResult(ERegistrationResult, const ANavigationData* NavigationData) const;

	/** Whether Navigation System is allowed to rebuild the navmesh
	 *  Depends on runtime generation settings of each navigation data, always true in the editor
	 */
	NAVIGATIONSYSTEM_API bool IsAllowedToRebuild() const;

	/** Handle forwarding the information where needed when the setting is changed*/
	NAVIGATIONSYSTEM_API void OnGenerateNavigationOnlyAroundNavigationInvokersChanged();
	
	//----------------------------------------------------------------------//
	// new stuff
	//----------------------------------------------------------------------//
public:
	NAVIGATIONSYSTEM_API void VerifyNavigationRenderingComponents(const bool bShow);
	/** @param InLevel Optional parameter to limit the navigation bounds to consider to the ones from that level */
	NAVIGATIONSYSTEM_API virtual int GetNavigationBoundsForNavData(const ANavigationData& NavData, TArray<FBox>& OutBounds, ULevel* InLevel = nullptr) const;
	static NAVIGATIONSYSTEM_API INavigationDataInterface* GetNavDataForActor(const AActor& Actor);
	NAVIGATIONSYSTEM_API virtual void Configure(const UNavigationSystemConfig& Config) override;
	NAVIGATIONSYSTEM_API virtual void AppendConfig(const UNavigationSystemConfig& NewConfig) override;
	
#if !UE_BUILD_SHIPPING
	NAVIGATIONSYSTEM_API void GetOnScreenMessages(TMultiMap<FCoreDelegates::EOnScreenMessageSeverity, FText>& OutMessages);
#endif // !UE_BUILD_SHIPPING

	//----------------------------------------------------------------------//
	// DEPRECATED
	//----------------------------------------------------------------------//
public:
	// Note that this function was only deprecated for blueprint in 5.1
	// Note2: originally deprecated as 4.22, bumped up to 5.1 as per comment above.
	UE_DEPRECATED(5.1, "This version is deprecated.  Please use GetRandomLocationInNavigableRadius instead")
	UFUNCTION(BlueprintPure, Category = "AI|Navigation", meta = (WorldContext = "WorldContextObject", DisplayName = "GetRandomPointInNavigableRadius", ScriptName = "GetRandomPointInNavigableRadius", DeprecatedFunction, DeprecationMessage = "GetRandomPointInNavigableRadius is deprecated. Use GetRandomLocationInNavigableRadius instead"))
	static NAVIGATIONSYSTEM_API bool K2_GetRandomPointInNavigableRadius(UObject* WorldContextObject, const FVector& Origin, FVector& RandomLocation, float Radius, ANavigationData* NavData = NULL, TSubclassOf<UNavigationQueryFilter> FilterClass = {});
};

//----------------------------------------------------------------------//
// UNavigationSystemModuleConfig 
//----------------------------------------------------------------------//
UCLASS(MinimalAPI)
class UNavigationSystemModuleConfig : public UNavigationSystemConfig
{
	GENERATED_BODY()

public:
	UPROPERTY()
	uint32 bDirtyNavDataPackageOnNavigationGenerationFinished : 1;
	
protected:
	/** Whether at game runtime we expect any kind of dynamic navigation generation */
	UPROPERTY(EditAnywhere, Category = Navigation)
	uint32 bStrictlyStatic : 1;

	UPROPERTY(EditAnywhere, Category = Navigation)
	uint32 bCreateOnClient : 1;

	UPROPERTY(EditAnywhere, Category = Navigation)
	uint32 bAutoSpawnMissingNavData : 1;

	UPROPERTY(EditAnywhere, Category = Navigation)
	uint32 bSpawnNavDataInNavBoundsLevel : 1;

public:
	NAVIGATIONSYSTEM_API UNavigationSystemModuleConfig(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	NAVIGATIONSYSTEM_API virtual void PostInitProperties() override;

	NAVIGATIONSYSTEM_API virtual UNavigationSystemBase* CreateAndConfigureNavigationSystem(UWorld& World) const override;

#if WITH_EDITOR
	NAVIGATIONSYSTEM_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

protected:
#if WITH_EDITOR
	friend UNavigationSystemV1;
#endif // WITH_EDITOR
	NAVIGATIONSYSTEM_API void UpdateWithNavSysCDO(const UNavigationSystemV1& NavSysCDO);
};
