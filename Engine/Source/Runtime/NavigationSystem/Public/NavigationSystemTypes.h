// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AI/Navigation/NavigationTypes.h"
#include "AI/Navigation/NavAgentSelector.h"
#include "UObject/WeakInterfacePtr.h"

#define NAVSYS_DEBUG (0 && UE_BUILD_DEBUG)

#define RECAST_INTERNAL_DEBUG_DATA (!UE_BUILD_SHIPPING)

enum class ENavigationInvokerPriority : uint8;
class UBodySetup;
class UNavCollision;
struct FKAggregateGeom;
class FNavigationOctree;
class UNavigationPath;
class ANavigationData;
class INavigationInvokerInterface;
struct FCompositeNavModifier;
struct FNavigationLink;
struct FNavigationSegmentLink;
struct FNavigationQueryFilter;
typedef TSharedPtr<const FNavigationQueryFilter, ESPMode::ThreadSafe> FSharedConstNavQueryFilter;
typedef TSharedPtr<struct FNavigationPath, ESPMode::ThreadSafe> FNavPathSharedPtr;

struct FPathFindingQueryData
{
	TWeakObjectPtr<const UObject> Owner;
	FVector StartLocation;
	FVector EndLocation;
	FSharedConstNavQueryFilter QueryFilter;

	/** cost limit of nodes allowed to be added to the open list */
	FVector::FReal CostLimit;
	
	/** additional flags passed to navigation data handling request */
	int32 NavDataFlags;

	/** if set, allow partial paths as a result */
	uint32 bAllowPartialPaths : 1;

	/** if set, require the end location to be linked to the navigation data */
	uint32 bRequireNavigableEndLocation : 1;

	FPathFindingQueryData() : StartLocation(FNavigationSystem::InvalidLocation), EndLocation(FNavigationSystem::InvalidLocation), CostLimit(TNumericLimits<FVector::FReal>::Max()), NavDataFlags(0), bAllowPartialPaths(true), bRequireNavigableEndLocation(true) {}

	FPathFindingQueryData(const UObject* InOwner, const FVector& InStartLocation, const FVector& InEndLocation, FSharedConstNavQueryFilter InQueryFilter = nullptr, int32 InNavDataFlags = 0, bool bInAllowPartialPaths = true, const  FVector::FReal InCostLimit = TNumericLimits<FVector::FReal>::Max(), const bool bInRequireNavigableEndLocation = true) :
		Owner(InOwner), StartLocation(InStartLocation), EndLocation(InEndLocation), QueryFilter(InQueryFilter), CostLimit(InCostLimit), NavDataFlags(InNavDataFlags), bAllowPartialPaths(bInAllowPartialPaths), bRequireNavigableEndLocation(bInRequireNavigableEndLocation) {}
};

struct FPathFindingQuery : public FPathFindingQueryData
{
	TWeakObjectPtr<const ANavigationData> NavData;
	FNavPathSharedPtr PathInstanceToFill;
	FNavAgentProperties NavAgentProperties;

	FPathFindingQuery() : FPathFindingQueryData() {}
	NAVIGATIONSYSTEM_API FPathFindingQuery(const UObject* InOwner, const ANavigationData& InNavData, const FVector& Start, const FVector& End, FSharedConstNavQueryFilter SourceQueryFilter = NULL, FNavPathSharedPtr InPathInstanceToFill = NULL, const FVector::FReal CostLimit = TNumericLimits<FVector::FReal>::Max(), const bool bInRequireNavigableEndLocation = true);
	NAVIGATIONSYSTEM_API FPathFindingQuery(const INavAgentInterface& InNavAgent, const ANavigationData& InNavData, const FVector& Start, const FVector& End, FSharedConstNavQueryFilter SourceQueryFilter = NULL, FNavPathSharedPtr InPathInstanceToFill = NULL, const FVector::FReal CostLimit = TNumericLimits<FVector::FReal>::Max(), const bool bInRequireNavigableEndLocation = true);

	NAVIGATIONSYSTEM_API explicit FPathFindingQuery(FNavPathSharedRef PathToRecalculate, const ANavigationData* NavDataOverride = NULL);

	FPathFindingQuery& SetPathInstanceToUpdate(FNavPathSharedPtr InPathInstanceToFill) { PathInstanceToFill = InPathInstanceToFill; return *this; }
	FPathFindingQuery& SetAllowPartialPaths(const bool bAllow) { bAllowPartialPaths = bAllow; return *this; }
	FPathFindingQuery& SetRequireNavigableEndLocation(const bool bRequire) { bRequireNavigableEndLocation = bRequire; return *this; }
	FPathFindingQuery& SetNavAgentProperties(const FNavAgentProperties& InNavAgentProperties) { NavAgentProperties = InNavAgentProperties; return *this; }

	/** utility function to compute a cost limit using an Euclidean heuristic, an heuristic scale and a cost limit factor
	*	CostLimitFactor: multiplier used to compute the cost limit value from the initial heuristic
	*	MinimumCostLimit: minimum clamping value used to prevent low cost limit for short path query */
	static NAVIGATIONSYSTEM_API FVector::FReal ComputeCostLimitFromHeuristic(const FVector& StartPos, const FVector& EndPos, const FVector::FReal HeuristicScale, const FVector::FReal CostLimitFactor, const FVector::FReal MinimumCostLimit);
};

namespace EPathFindingMode
{
	enum Type
	{
		Regular,
		Hierarchical,
	};
};

////////////////////////////////////////////////////////////////////////////
//// Custom path following data
//
///** Custom data passed to movement requests. */
struct FMoveRequestCustomData
{
};

typedef TSharedPtr<FMoveRequestCustomData, ESPMode::ThreadSafe> FCustomMoveSharedPtr;
typedef TWeakPtr<FMoveRequestCustomData, ESPMode::ThreadSafe> FCustomMoveWeakPtr;

//----------------------------------------------------------------------//
// Active tiles 
//----------------------------------------------------------------------//
struct FNavigationInvokerRaw
{
	FVector Location;
	float RadiusMin;
	float RadiusMax;
	FNavAgentSelector SupportedAgents;
	ENavigationInvokerPriority Priority;

	FNavigationInvokerRaw(const FVector& InLocation, float Min, float Max, const FNavAgentSelector& InSupportedAgents, ENavigationInvokerPriority InPriority);
};

class AActor;

struct FNavigationInvoker
{
	/** The Invoker source should be either an Actor or an Object. Thus only 1 of those member should be set. We'll use IsExplicitlyNull to know which one to use */
	TWeakObjectPtr<AActor> Actor;
	TWeakInterfacePtr<INavigationInvokerInterface> Object;

	/** tiles GenerationRadius away or close will be generated if they're not already present */
	float GenerationRadius;

	/** tiles over RemovalRadius will get removed.
	*	@Note needs to be >= GenerationRadius or will get clamped */
	float RemovalRadius;

	/** restrict navigation generation to specific agents */
	FNavAgentSelector SupportedAgents;

	/** invoker Priority used when dirtying tiles */
	ENavigationInvokerPriority Priority;

	FNavigationInvoker();
	FNavigationInvoker(AActor& InActor, float InGenerationRadius, float InRemovalRadius, const FNavAgentSelector& InSupportedAgents, ENavigationInvokerPriority InPriority);
	FNavigationInvoker(INavigationInvokerInterface& InObject, float InGenerationRadius, float InRemovalRadius, const FNavAgentSelector& InSupportedAgents, ENavigationInvokerPriority InPriority);

	FString GetName() const;
	bool GetLocation(FVector& OutLocation) const;
};

namespace NavigationHelper
{
	void GatherCollision(UBodySetup* RigidBody, TNavStatArray<FVector>& OutVertexBuffer, TNavStatArray<int32>& OutIndexBuffer, const FTransform& ComponentToWorld = FTransform::Identity);
	void GatherCollision(UBodySetup* RigidBody, UNavCollision* NavCollision);

	/** gather collisions from aggregated geom, convex and tri mesh elements are not supported - use override with full UBodySetup param instead */
	void GatherCollision(const FKAggregateGeom& AggGeom, UNavCollision& NavCollision);
}

namespace FNavigationSystem
{
	enum ECreateIfMissing
	{
		Invalid = -1,
		DontCreate = 0,
		Create = 1,
	};

	typedef ECreateIfMissing ECreateIfEmpty;
}

//----------------------------------------------------------------------//
// 
//----------------------------------------------------------------------//

namespace NavigationHelper
{
	struct FNavLinkOwnerData
	{
		const AActor* Actor;
		FTransform LinkToWorld;

		FNavLinkOwnerData() : Actor(nullptr) {}
		NAVIGATIONSYSTEM_API FNavLinkOwnerData(const AActor& InActor);
		NAVIGATIONSYSTEM_API FNavLinkOwnerData(const USceneComponent& InComponent);
	};

	DECLARE_DELEGATE_ThreeParams(FNavLinkProcessorDelegate, FCompositeNavModifier*, const AActor*, const TArray<FNavigationLink>&);
	DECLARE_DELEGATE_ThreeParams(FNavLinkSegmentProcessorDelegate, FCompositeNavModifier*, const AActor*, const TArray<FNavigationSegmentLink>&);

	DECLARE_DELEGATE_ThreeParams(FNavLinkProcessorDataDelegate, FCompositeNavModifier*, const FNavLinkOwnerData&, const TArray<FNavigationLink>&);
	DECLARE_DELEGATE_ThreeParams(FNavLinkSegmentProcessorDataDelegate, FCompositeNavModifier*, const FNavLinkOwnerData&, const TArray<FNavigationSegmentLink>&);

	/** Set new implementation of nav link processor, a function that will be
	*	be used to process/transform links before adding them to CompositeModifier.
	*	This function is supposed to be called once during the engine/game
	*	setup phase. Not intended to be toggled at runtime */
	NAVIGATIONSYSTEM_API void SetNavLinkProcessorDelegate(const FNavLinkProcessorDataDelegate& NewDelegate);
	NAVIGATIONSYSTEM_API void SetNavLinkSegmentProcessorDelegate(const FNavLinkSegmentProcessorDataDelegate& NewDelegate);

	/** called to do any necessary processing on NavLinks and put results in CompositeModifier */
	NAVIGATIONSYSTEM_API void ProcessNavLinkAndAppend(FCompositeNavModifier* OUT CompositeModifier, const AActor* Actor, const TArray<FNavigationLink>& IN NavLinks);
	NAVIGATIONSYSTEM_API void ProcessNavLinkAndAppend(FCompositeNavModifier* OUT CompositeModifier, const FNavLinkOwnerData& OwnerData, const TArray<FNavigationLink>& IN NavLinks);

	/** called to do any necessary processing on NavLinks and put results in CompositeModifier */
	NAVIGATIONSYSTEM_API void ProcessNavLinkSegmentAndAppend(FCompositeNavModifier* OUT CompositeModifier, const AActor* Actor, const TArray<FNavigationSegmentLink>& IN NavLinks);
	NAVIGATIONSYSTEM_API void ProcessNavLinkSegmentAndAppend(FCompositeNavModifier* OUT CompositeModifier, const FNavLinkOwnerData& OwnerData, const TArray<FNavigationSegmentLink>& IN NavLinks);

	NAVIGATIONSYSTEM_API void DefaultNavLinkProcessorImpl(FCompositeNavModifier* OUT CompositeModifier, const FNavLinkOwnerData& OwnerData, const TArray<FNavigationLink>& IN NavLinks);
	NAVIGATIONSYSTEM_API void DefaultNavLinkSegmentProcessorImpl(FCompositeNavModifier* OUT CompositeModifier, const FNavLinkOwnerData& OwnerData, const TArray<FNavigationSegmentLink>& IN NavLinks);

	NAVIGATIONSYSTEM_API bool IsBodyNavigationRelevant(const UBodySetup& IN BodySetup);
}
