// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "AI/Navigation/NavigationTypes.h"
#include "AI/Navigation/NavAgentInterface.h"
#include "NavigationData.h"
#include "Tickable.h"
#include "AI/Navigation/NavPathObserverInterface.h"
#include "NavMesh/RecastNavMesh.h"
#include "NavigationTestingActor.generated.h"

class ANavigationTestingActor;
class UNavigationInvokerComponent;

struct FNavTestTickHelper : FTickableGameObject
{
	TWeakObjectPtr<ANavigationTestingActor> Owner;

	FNavTestTickHelper() : Owner(NULL) {}
	virtual void Tick(float DeltaTime);
	virtual bool IsTickable() const { return Owner.IsValid(); }
	virtual bool IsTickableInEditor() const { return true; }
	virtual TStatId GetStatId() const ;
};

UENUM()
namespace ENavCostDisplay
{
	enum Type : int
	{
		TotalCost,
		HeuristicOnly,
		RealCostOnly,
	};
}

UCLASS(hidecategories=(Object, Actor, Input, Rendering, Replication, HLOD, Cooking), showcategories=("Input|MouseInput", "Input|TouchInput"), Blueprintable, MinimalAPI)
class ANavigationTestingActor : public AActor, public INavAgentInterface, public INavPathObserverInterface
{
	GENERATED_UCLASS_BODY()

private:
	UPROPERTY()
	TObjectPtr<class UCapsuleComponent> CapsuleComponent;

#if WITH_EDITORONLY_DATA
	/** Editor Preview */
	UPROPERTY()
	TObjectPtr<class UNavTestRenderingComponent> EdRenderComp;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, Category = Navigation, meta=(EditCondition="bActAsNavigationInvoker"))
	TObjectPtr<UNavigationInvokerComponent> InvokerComponent;

	UPROPERTY(EditAnywhere, Category = Navigation, meta=(InlineEditConditionToggle))
	uint32 bActAsNavigationInvoker : 1;

public:

	/** @todo document */
	UPROPERTY(EditAnywhere, Category=Agent)
	FNavAgentProperties NavAgentProps;

	UPROPERTY(EditAnywhere, Category=Agent)
	FVector QueryingExtent;

	UPROPERTY(transient)
	TObjectPtr<ANavigationData> MyNavData;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=AgentStatus)
	FVector ProjectedLocation;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=AgentStatus)
	uint32 bProjectedLocationValid : 1;

	/** if set, start the search from this actor, else start the search from the other actor */
	UPROPERTY(EditAnywhere, Category=Pathfinding)
	uint32 bSearchStart : 1;

	/** this multiplier is used to compute a max node cost allowed to the open list
	 *	(cost limit = CostLimitFactor*InitialHeuristicEstimate) */
	UPROPERTY(EditAnywhere, Category=Pathfinding, meta = (ClampMin = "0", UIMin = "0"))
	float CostLimitFactor;

	/** minimum cost limit clamping value (in cost units)
	 *	used to allow large deviation in short paths */
	UPROPERTY(EditAnywhere, Category = Pathfinding, meta = (ClampMin = "0", UIMin = "0"))
	float MinimumCostLimit;

	/** Instead of regular pathfinding from source to target location do
	 *	a 'backwards' search that searches from the source, but as if the allowed
	 *	movement direction was coming from the target. Meaningful only for paths
	 *	containing one-direction nav links. */
	UPROPERTY(EditAnywhere, Category = Pathfinding)
	uint32 bBacktracking : 1;

	UPROPERTY(EditAnywhere, Category=Pathfinding)
	uint32 bUseHierarchicalPathfinding : 1;

	/** if set, all steps of A* algorithm will be accessible for debugging */
	UPROPERTY(EditAnywhere, Category=Pathfinding)
	uint32 bGatherDetailedInfo : 1;

	/** if set, require the end location to be close to the navigation data. The tolerance is controlled by QueryingExtent */
	UPROPERTY(EditAnywhere, Category=Pathfinding)
	uint32 bRequireNavigableEndLocation : 1;

	UPROPERTY(EditAnywhere, Category=Query)
	uint32 bDrawDistanceToWall : 1;

	/** If set, a cylinder is drawn to indicate if the navigation data is ready (has been generated) for the given radius (green when ready, red otherwise). */
	UPROPERTY(EditAnywhere, Category=Query)
	uint32 bDrawIfNavDataIsReadyInRadius : 1;

	/** If set, a capsule is drawn to indicate if the navigation data is ready (has been generated) for the given radius from the current actor to the query target (green when ready, red otherwise). */
	UPROPERTY(EditAnywhere, Category=Query)
	uint32 bDrawIfNavDataIsReadyToQueryTargetActor : 1;

	/** If set, a line is drawn to indicate to result of a ray cast on the navigation data between the current actor and the QueryTargetActor location
	 * (red when there is a hit, green when there is no hit and the ray end is on the explored corridor, orange otherwise). */
	UPROPERTY(EditAnywhere, Category=Query)
	uint32 bDrawRaycastToQueryTargetActor : 1;

	/** Actor to use as a target for navigation data queries */
	UPROPERTY(EditAnywhere, Category=Query)
	TObjectPtr<AActor> QueryTargetActor;

	/** show polys from open (orange) and closed (yellow) sets */
	UPROPERTY(EditAnywhere, Category=Debug)
	uint32 bShowNodePool : 1;

	/** show current best path */
	UPROPERTY(EditAnywhere, Category=Debug)
	uint32 bShowBestPath : 1;

	/** show which nodes were modified in current A* step */
	UPROPERTY(EditAnywhere, Category=Debug)
	uint32 bShowDiffWithPreviousStep : 1;

	UPROPERTY(EditAnywhere, Category=Debug)
	uint32 bShouldBeVisibleInGame : 1;

	/** NavData must be ready for all tiles within radius. When using 0, NavData must be ready at the actor location. */
	UPROPERTY(EditAnywhere, Category=Query)
	float RadiusUsedToValidateNavData = 0;
	
	/** determines which cost will be shown*/
	UPROPERTY(EditAnywhere, Category=Debug)
	TEnumAsByte<ENavCostDisplay::Type> CostDisplayMode;

	/** text canvas offset to apply */
	UPROPERTY(EditAnywhere, Category=Debug)
	FVector2D TextCanvasOffset;

	UPROPERTY(transient, VisibleAnywhere, BlueprintReadOnly, Category=PathfindingStatus)
	uint32 bPathExist : 1;

	UPROPERTY(transient, VisibleAnywhere, BlueprintReadOnly, Category=PathfindingStatus)
	uint32 bPathIsPartial : 1;
	
	UPROPERTY(transient, VisibleAnywhere, BlueprintReadOnly, Category=PathfindingStatus)
	uint32 bPathSearchOutOfNodes : 1;

	/** Time in micro seconds */
	UPROPERTY(transient, VisibleAnywhere, BlueprintReadOnly, Category=PathfindingStatus)
	float PathfindingTime;

	UPROPERTY(transient, VisibleAnywhere, BlueprintReadOnly, Category=PathfindingStatus)
	double PathCost;

	UPROPERTY(transient, VisibleAnywhere, BlueprintReadOnly, Category=PathfindingStatus)
	int32 PathfindingSteps;

	UPROPERTY(EditAnywhere, Category=Pathfinding)
	TObjectPtr<ANavigationTestingActor> OtherActor;

	/** "None" will result in default filter being used. This filter is used by the PathFind and Raycast queries. */
	UPROPERTY(EditAnywhere, Category=Query)
	TSubclassOf<class UNavigationQueryFilter> FilterClass;

	/** Show debug steps up to this index. Use -1 to disable. */
	UPROPERTY(EditInstanceOnly, Category=Debug, meta=(ClampMin="-1", UIMin="-1"))
	int32 ShowStepIndex;

	UPROPERTY(EditAnywhere, Category=Pathfinding)
	float OffsetFromCornersDistance;

	FVector ClosestWallLocation;
	FVector RaycastHitLocation;

	bool bNavDataIsReadyInRadius;
	bool bNavDataIsReadyToQueryTargetActor;
	bool bRaycastToQueryTargetActorResult;
	bool bRaycastToQueryTargetEndsInCorridor;

#if WITH_RECAST && WITH_EDITORONLY_DATA
	/** detail data gathered from each step of regular A* algorithm */
	TArray<struct FRecastDebugPathfindingData> DebugSteps;

	FNavTestTickHelper* TickHelper;
#endif

	FNavPathSharedPtr LastPath;
	FNavigationPath::FPathObserverDelegate::FDelegate PathObserver;

	/** Dtor */
	NAVIGATIONSYSTEM_API virtual ~ANavigationTestingActor();

	NAVIGATIONSYSTEM_API virtual void BeginDestroy() override;

#if WITH_EDITOR
	NAVIGATIONSYSTEM_API virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	NAVIGATIONSYSTEM_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	NAVIGATIONSYSTEM_API virtual void PostEditMove(bool bFinished) override;
	
	NAVIGATIONSYSTEM_API virtual void PostLoad() override;
	NAVIGATIONSYSTEM_API void TickMe();
#endif // WITH_EDITOR

	//~ Begin INavAgentInterface Interface
	virtual const FNavAgentProperties& GetNavAgentPropertiesRef() const override { return NavAgentProps; }
	NAVIGATIONSYSTEM_API virtual FVector GetNavAgentLocation() const override;
	virtual void GetMoveGoalReachTest(const AActor* MovingActor, const FVector& MoveOffset, FVector& GoalOffset, float& GoalRadius, float& GoalHalfHeight) const override {}
	//~ End INavAgentInterface Interface

	//~ Begin INavPathObserverInterface Interface
	virtual void OnPathUpdated(class INavigationPathGenerator* PathGenerator) override {};
	virtual void OnPathInvalid(class INavigationPathGenerator* PathGenerator) override {};
	virtual void OnPathFailed(class INavigationPathGenerator* PathGenerator) override {};
	//~ End INavPathObserverInterface Interface	

	NAVIGATIONSYSTEM_API void UpdateNavData();
	NAVIGATIONSYSTEM_API void UpdatePathfinding();
	NAVIGATIONSYSTEM_API virtual void SearchPathTo(ANavigationTestingActor* Goal);

	/*	Called when given path becomes invalid (via @see PathObserverDelegate)
	 *	NOTE: InvalidatedPath doesn't have to be instance's current Path
	 */
	NAVIGATIONSYSTEM_API void OnPathEvent(FNavigationPath* InvalidatedPath, ENavPathEvent::Type Event);

	// Virtual method to override if you want to customize the query being 
	// constructed for the path find (e.g. change the filter or add 
	// constraints/goal evaluators).
	NAVIGATIONSYSTEM_API virtual FPathFindingQuery BuildPathFindingQuery(const ANavigationTestingActor* Goal) const;

	/** Returns CapsuleComponent subobject **/
	class UCapsuleComponent* GetCapsuleComponent() const { return CapsuleComponent; }
#if WITH_EDITORONLY_DATA
	/** Returns EdRenderComp subobject **/
	class UNavTestRenderingComponent* GetEdRenderComp() const { return EdRenderComp; }
#endif

protected:
	void UpdateLocalQueries();
	void UpdateTargetActorQueries();

	NAVIGATIONSYSTEM_API FVector FindClosestWallLocation() const;
	bool CheckIfNavDataIsReadyInRadius();
	bool CheckIfNavDataIsReadyToActor(const AActor* TargetActor);
	bool CheckRaycastToActor(const AActor* TargetActor, FVector& OutHitLocation, bool& bOutIsRaycastEndInCorridor);
	void OnQueryTargetActorTransformUpdated(USceneComponent* InRootComponent, EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport);
};
