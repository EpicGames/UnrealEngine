// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NavModifierComponent.h"

#include "SplineNavModifierComponent.generated.h"

#define INVALID_SPLINE_VERSION MIN_int32

struct FNavigationRelevantData;

UENUM()
enum class ESubdivisionLOD
{
	Low,
	Medium,
	High,
	Ultra,
};

/**
 *	Used to assign a chosen NavArea to the nav mesh in the vicinity of a chosen spline.
 *	A tube is constructed around the spline and intersected with the nav mesh. Set its dimensions with StrokeWidth and StrokeHeight.
 */
UCLASS(Blueprintable, MinimalAPI, Meta = (BlueprintSpawnableComponent), hidecategories = (Variable, Tags, Cooking, Collision))
class USplineNavModifierComponent : public UNavModifierComponent
{
	GENERATED_BODY()

	/**
	 * If true, any changes to Spline Components on this actor will cause this component to update the nav mesh.
	 * This will be slow if the spline has many points, or the nav mesh is sufficiently large.
	 */ 
	UPROPERTY(EditAnywhere, Category = Navigation)
	bool bUpdateNavDataOnSplineChange = true;

	/** The SplineComponent which will modify the nav mesh; it must also be attached to this component's owner actor */ 
	UPROPERTY(EditAnywhere, Category = Navigation, Meta = (UseComponentPicker, AllowedClasses = "/Script/Engine.SplineComponent", DisplayName = "Nav Modifier Spline"))
	FComponentReference AttachedSpline;

	/** Cross-sectional width of the tube enclosing the spline */
	UPROPERTY(EditAnywhere, Category = Navigation, Meta=(UIMin="10", ClampMin="10"))
	double StrokeWidth = 500.0f;

	/** Cross-sectional height of the tube enclosing the spline */
	UPROPERTY(EditAnywhere, Category = Navigation, Meta=(UIMin="10", ClampMin="10"))
	double StrokeHeight = 500.0f;

	/** Higher LOD will capture finer details in the spline */
	UPROPERTY(EditAnywhere, Category = Navigation)
	ESubdivisionLOD SubdivisionLOD = ESubdivisionLOD::Medium;

	USplineNavModifierComponent(const FObjectInitializer& ObjectInitializer);

	/**
	 * Recalculates bounds, then re-computes the NavModifierVolumes and re-marks the nav mesh.
	 * Disable UpdateNavDataOnSplineChange and use this to manually update when either the spline or nav mesh is too large to handle rapid updates.
	 *
	 * Does nothing in non-editor builds
	 */
	UFUNCTION(CallInEditor, Category = Navigation, Meta=(DisplayName="UpdateNavigationData"))
	void UpdateNavigationWithComponentData();

protected:
#if WITH_EDITORONLY_DATA
	virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	virtual bool IsComponentTickEnabled() const override;
#endif // WITH_EDITORONLY_DATA

	virtual void CalculateBounds() const override;
	virtual void GetNavigationData(FNavigationRelevantData& Data) const override;

	float GetSubdivisionThreshold() const;

private:
#if WITH_EDITORONLY_DATA
	// Used to check against attached spline's version each tick for changes
	uint32 SplineVersion = INVALID_SPLINE_VERSION;
#endif // WITH_EDITORONLY_DATA

	// Used for bounds calculation and to check against attached spline's transform each tick for changes
	FTransform SplineTransform = FTransform();
};
